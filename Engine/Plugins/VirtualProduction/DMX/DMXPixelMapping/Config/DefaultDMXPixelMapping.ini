; Used during 4.26 development only. Not needed to upgrade 4.25 projects and projects created after the 21 September 2020
[CoreRedirects]
+ClassRedirects=(OldName="DMXPixelMappingMatrixPixelComponent", NewName="DMXPixelMappingMatrixCellComponent")


+PropertyRedirects=(OldName="DMXPixelMappingScreenComponent.NumXPanels",NewName="NumXCells")
+PropertyRedirects=(OldName="DMXPixelMappingScreenComponent.NumYPanels",NewName="NumYCells")
