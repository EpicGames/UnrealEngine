{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DMX Pixel Mapping",
	"Description": "Tools set for map LED digital pixel strip or fixture arrays regardless of shape or size",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "DMXPixelMappingCore",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXPixelMappingRuntime",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXPixelMappingEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXPixelMappingEditorWidgets",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXPixelMappingBlueprintGraph",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DMXPixelMappingRenderer",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "DMXProtocol",
			"Enabled": true
		},
		{
			"Name": "DMXEngine",
			"Enabled": true
		}
	],
	"IsExperimentalVersion": false,
	"Installed": false
}