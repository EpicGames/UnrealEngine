// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMapping.ush"

/*=============================================================================
 DMXPixelMapping.usf: Draw downsampled texture
 =============================================================================*/


/* Vertex shader
=============================================================================*/

OutputVS DMXPixelMappingVS(InputVS IN)
{
	OutputVS OUT;
	
	// Used for calculating vertex positions and UVs when drawing with DrawRectangleForPixelMapping
	DrawRectangleForPixelMapping(IN.Position, IN.UV, OUT.Position, OUT.UV.xy);

	return OUT;
}

/* Pixel shader
=============================================================================*/

#define PIXELBLENDING_QUALITY_LOW		0
#define PIXELBLENDING_QUALITY_MEDIUM	1
#define PIXELBLENDING_QUALITY_HIGH		2

#ifndef PIXELBLENDING_QUALITY
#error PIXELBLENDING_QUALITY is not defined.
#endif

OutputPS DMXPixelMappingPS(OutputVS IN)
{
	OutputPS OUT;
	float2 UV = IN.UV.xy;
	
	// Draw colors only within 0...1 X and Y coordinates
	if (UV.x < 0.f ||
		UV.x > 1.f ||
		UV.y < 0.f ||
		UV.y > 1.f
	)
	{
		OUT.Color = float4(0.f, 0.f, 0.f, 1.f);
	}
	else
	{
		// Sample the center color
		OUT.Color = Texture2DSample(InputTexture, InputSampler, UV);

		// For higher qualities, average the final color from the center color and additional samples on the sides.
#if PIXELBLENDING_QUALITY == PIXELBLENDING_QUALITY_MEDIUM
		#define NUM_CELLS 4
		float2 UVCells[NUM_CELLS];
		
		// 1/8 from the vertices results in an even color distribution with 5 samples
		UVCells[0] = UVTopLeftRotated + (UV - UVTopLeftRotated) / 8.f;			// Upper Left 
		UVCells[1] = UVTopRightRotated + (UV - UVTopRightRotated) / 8.f;		// Upper Right
		UVCells[2] = UV + UV - UVCells[0];										// Bottom Right
		UVCells[3] = UV + UV - UVCells[1];										// Bottom Left

		for(uint i = 0; i < NUM_CELLS; ++i)
		{
			// Only add 1/4 of the color to avoid bias towards the sides
			OUT.Color += Texture2DSample(InputTexture, InputSampler, clamp(UVCells[i], 0, 1)) / 4.f;
		}
		
		// The average of the center color + 1/4 of the color on each side
		OUT.Color /= 2.f;

#elif PIXELBLENDING_QUALITY == PIXELBLENDING_QUALITY_HIGH
		#define NUM_CELLS 8
		float2 UVCells[NUM_CELLS];
		
		// 1/6 from the vertices results in an even color distribution with 9 samples
		UVCells[0] = UVTopLeftRotated + (UV - UVTopLeftRotated) / 6.f;			// Upper Left  
		UVCells[1] = UVTopRightRotated + (UV - UVTopRightRotated) / 6.f;		// Upper Right 
		UVCells[2] = UV + UV - UVCells[0];										// Bottom Right
		UVCells[3] = UV + UV - UVCells[1];										// Bottom Left
		UVCells[4] = UV + (UVCells[0] - UV + UVCells[1] - UV) / 2.f;			// Upper Center
		UVCells[5] = UV + (UVCells[1] - UV + UVCells[2] - UV) / 2.f;			// Right Center
		UVCells[6] = UV + (UVCells[2] - UV + UVCells[3] - UV) / 2.f;			// Bottom Center
		UVCells[7] = UV + (UVCells[3] - UV + UVCells[0] - UV) / 2.f;			// Left Center
		
		for(uint i = 0; i < NUM_CELLS; ++i)
		{
			OUT.Color += Texture2DSample(InputTexture, InputSampler, clamp(UVCells[i], 0, 1));
		}
		
		// The average of the center color + all the points on sides
		OUT.Color /= 9.f;
		
#elif PIXELBLENDING_QUALITY > PIXELBLENDING_QUALITY_HIGH
#error Invalid pixel blending quality level specified.
#endif

		OUT.Color *= Brightness;
	}
			
	return OUT;
}
