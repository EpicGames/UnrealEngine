// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DMXPixelMapping.ush Helper functions, buffers, includes
=============================================================================*/

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"

// VS Parameters
float4 DrawRectanglePosScaleBias;
float4 DrawRectangleInvTargetSizeAndTextureSize;
float4 DrawRectangleUVScaleBias;

// PS Textures Parameters
Texture2D InputTexture;
SamplerState InputSampler;

// PS Parameters
float Brightness;
float2 UVTopLeftRotated;
float2 UVTopRightRotated;

struct InputVS
{
	float4 Position : ATTRIBUTE0;
	float2 UV : ATTRIBUTE1;
};

struct OutputVS
{
	float4 Position : SV_Position;
	float4 UV : TEXCOORD0;
};

struct OutputPS
{
	float4 Color : SV_Target0;
};

#ifndef VERTEX_UV_STATIC_CALCULATION
#error VERTEX_UV_STATIC_CALCULATION is not defined.
#endif


/* -----------------------------------------------------------------
 * Used for calculating vertex positions and UVs when drawing with DrawRectangleForPixelMapping
 * -----------------------------------------------------------------
 */
void DrawRectangleForPixelMapping(in float4 InPosition, in float2 InTexCoord, out float4 OutPosition, out float2 OutTexCoord)
{
	OutPosition = InPosition;
	
	OutPosition.xy = -1.0f + 2.0f * (DrawRectanglePosScaleBias.zw + (InPosition.xy * DrawRectanglePosScaleBias.xy)) * DrawRectangleInvTargetSizeAndTextureSize.xy;
	OutPosition.xy *= float2(1, -1);

#if VERTEX_UV_STATIC_CALCULATION
	OutTexCoord.xy = DrawRectangleUVScaleBias.zw + (DrawRectangleUVScaleBias.xy / 2.f);
#else
	OutTexCoord.xy = (DrawRectangleUVScaleBias.zw + (InTexCoord.xy * DrawRectangleUVScaleBias.xy)) * DrawRectangleInvTargetSizeAndTextureSize.zw;
#endif
}