// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXPixelMappingBlueprintGraph : ModuleRules
{
	public DMXPixelMappingBlueprintGraph(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "DMXPxlMapBPGraph";
		
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"BlueprintGraph",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"KismetCompiler",
				"BlueprintGraph",
				
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"GraphEditor",
				"UnrealEd",
				"DMXPixelMappingRuntime",
			}
		);
	}
}
