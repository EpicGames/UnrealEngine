// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"


#include "DMXPixelMappingPinFactory.h"

class FDMXPixelMappingBlueprintGraphModule 
	: public IModuleInterface
{
public:
	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override
	{
		PixelMappingPinFactory = MakeShared<FDMXPixelMappingPinFactory>();
		FEdGraphUtilities::RegisterVisualPinFactory(PixelMappingPinFactory);
	}

	virtual void ShutdownModule() override
	{
		FEdGraphUtilities::UnregisterVisualPinFactory(PixelMappingPinFactory);
	}
	//~ End IModuleInterface implementation

private:
	TSharedPtr<FDMXPixelMappingPinFactory> PixelMappingPinFactory;
};

IMPLEMENT_MODULE(FDMXPixelMappingBlueprintGraphModule, DMXPixelMappingBlueprintGraph);
