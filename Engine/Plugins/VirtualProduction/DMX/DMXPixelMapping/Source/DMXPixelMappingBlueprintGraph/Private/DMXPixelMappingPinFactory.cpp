// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingPinFactory.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/SDMXPixelMappingComponentPin.h"
#include "K2Node_PixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"

#include "EdGraphSchema_K2.h"

#define DMX_PIXEL_MAPPING_ADD_COMPONENT_PIN(ComponentType)	\
if (Outer->IsA(UK2Node_PixelMapping##ComponentType::StaticClass()) && InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Name) \
{ \
	const UK2Node_PixelMapping##ComponentType* Component = CastChecked<UK2Node_PixelMapping##ComponentType>(Outer); \
	const UEdGraphPin* InPixelMappingPin = Component->GetInPixelMappingPin(); \
	const UEdGraphPin* ComponentPin = Component->GetIn##ComponentType##Pin(); \
	if (InPixelMappingPin != nullptr && ComponentPin != nullptr) \
	{ \
		if (InPixelMappingPin->DefaultObject != nullptr && InPixelMappingPin->LinkedTo.Num() == 0) \
		{ \
			if (class UDMXPixelMapping* DMXPixelMapping = Cast<UDMXPixelMapping>(InPixelMappingPin->DefaultObject)) \
			{ \
				return SNew(SDMXPixelMappingComponentPin<UDMXPixelMapping##ComponentType>, InPin, DMXPixelMapping); \
			} \
		} \
	} \
}

TSharedPtr<class SGraphPin> FDMXPixelMappingPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	UObject* Outer = InPin->GetOuter();

	DMX_PIXEL_MAPPING_ADD_COMPONENT_PIN(RendererComponent);

	return FGraphPanelPinFactory::CreatePin(InPin);
}
