// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

class SGraphPin;

/**
 * Custom Pixel Mapping pins factory
 */
class FDMXPixelMappingPinFactory 
	: public FGraphPanelPinFactory
{
	//~ Begin FGraphPanelPinFactory implementation
	virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* InPin) const override;
	//~ End FGraphPanelPinFactory implementation
};
