// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PixelMappingBaseComponent.h"
#include "DMXPixelMapping.h"
#include "Blueprint/DMXPixelMappingSubsystem.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "KismetCompiler.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "K2Node_CallFunction.h"

#define LOCTEXT_NAMESPACE "UK2Node_PixelMappingBaseComponent"

const FName UK2Node_PixelMappingBaseComponent::InPixelMappingPinName(TEXT("In Pixel Mapping"));

void UK2Node_PixelMappingBaseComponent::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Input pins
	UEdGraphPin* InPixelMappingPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Object, UDMXPixelMapping::StaticClass(), InPixelMappingPinName);
	K2Schema->ConstructBasicPinTooltip(*InPixelMappingPin, LOCTEXT("InPixelMappingPin", "Input Pixel Mapping"), InPixelMappingPin->PinToolTip);

	Super::AllocateDefaultPins();
}

void UK2Node_PixelMappingBaseComponent::RefreshGraph()
{
	UEdGraph* Graph = GetGraph();
	Graph->NotifyNodeChanged(this);
}

UEdGraphPin* UK2Node_PixelMappingBaseComponent::GetInPixelMappingPin() const
{
	UEdGraphPin* Pin = FindPinChecked(InPixelMappingPinName);
	check(Pin->Direction == EGPD_Input);

	return Pin;
}

FText UK2Node_PixelMappingBaseComponent::GetMenuCategory() const
{
	return FText::FromString("DMX");
}

void UK2Node_PixelMappingBaseComponent::PreloadRequiredAssets()
{
	if (UEdGraphPin* PixelMappingPin = GetPixelMappingPin())
	{
		if (UDMXPixelMapping* DMXPixelMapping = Cast<UDMXPixelMapping>(PixelMappingPin->DefaultObject))
		{
			// make sure to properly load the Pixel Mapping object
			DMXPixelMapping->PreloadWithChildren();
		}
	}
	return Super::PreloadRequiredAssets();
}

void UK2Node_PixelMappingBaseComponent::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	Super::ReallocatePinsDuringReconstruction(OldPins);

	if (UEdGraphPin* PixelMappingPin = GetPixelMappingPin(&OldPins))
	{
		if (UDMXPixelMapping* DMXPixelMapping = Cast<UDMXPixelMapping>(PixelMappingPin->DefaultObject))
		{
			// make sure to properly load the Pixel Mapping object
			DMXPixelMapping->PreloadWithChildren();
		}
	}
}

UEdGraphPin* UK2Node_PixelMappingBaseComponent::GetPixelMappingPin(const TArray<UEdGraphPin*>* InPinsToSearch /*= nullptr*/) const
{
	const TArray<UEdGraphPin*>* PinsToSearch = InPinsToSearch ? InPinsToSearch : &Pins;

	UEdGraphPin* Pin = nullptr;
	for (UEdGraphPin* TestPin : *PinsToSearch)
	{
		if (TestPin && TestPin->PinName == InPixelMappingPinName)
		{
			Pin = TestPin;
			break;
		}
	}
	check(Pin == nullptr || Pin->Direction == EGPD_Input);
	return Pin;
}

void UK2Node_PixelMappingBaseComponent::AddBlueprintAction(UClass* InClass, FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	if (ActionRegistrar.IsOpenForRegistration(InClass))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(InClass);
		check(NodeSpawner != nullptr);

		ActionRegistrar.AddBlueprintAction(InClass, NodeSpawner);
	}
}

void UK2Node_PixelMappingBaseComponent::ModifyBlueprint()
{
	if (UBlueprint* BP = GetBlueprint())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(BP);
	}
}

void UK2Node_PixelMappingBaseComponent::TryModifyBlueprintOnNameChanged(UDMXPixelMapping* InDMXPixelMapping, UEdGraphPin* InPin)
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
	UEdGraphPin* PixelMappingPin = GetPixelMappingPin();
	if (InDMXPixelMapping != nullptr && PixelMappingPin != nullptr && InDMXPixelMapping == PixelMappingPin->DefaultObject)
	{
		const bool TryRefresh = InPin != nullptr && !InPin->LinkedTo.Num();
		const FName CurrentName = InPin ? FName(*InPin->GetDefaultAsString()) : NAME_None;
		if (TryRefresh && !InDMXPixelMapping->FindComponent(CurrentName))
		{
			ModifyBlueprint();
		}
	}
}
}

void UK2Node_PixelMappingBaseComponent::TryRefreshGraphCheckInputPins(UEdGraphPin* TryPixelMappingPin, UEdGraphPin* TryComponentNamePin)
{
	if (TryPixelMappingPin && TryPixelMappingPin->PinName == InPixelMappingPinName)
	{
		if (TryComponentNamePin)
		{
			RefreshGraph();
		}
	}
}

void UK2Node_PixelMappingBaseComponent::ExecuteEarlyValidation(FCompilerResultsLog& MessageLog, UEdGraphPin* InComponentPin) const
{
	const UEdGraphPin* PixelMappingPin = GetPixelMappingPin();

	if(PixelMappingPin->LinkedTo.Num() == 1)
	{
		return;
	}
	
	UDMXPixelMapping* DMXPixelMapping = Cast<UDMXPixelMapping>(PixelMappingPin->DefaultObject);
	if (DMXPixelMapping == nullptr)
	{
		MessageLog.Error(*LOCTEXT("NoDMXPixelMapping", "No DMXPixelMapping in @@").ToString(), this);
		return;
	}

	if (!InComponentPin->LinkedTo.Num())
	{
		const FName CurrentName = FName(*InComponentPin->GetDefaultAsString());
		if (!DMXPixelMapping->FindComponent(CurrentName))
		{
			const FString Msg = FText::Format(
				LOCTEXT("WrongRendererComponent", "'{0}' name is not stored component in '{1}'. @@"),
				FText::FromString(CurrentName.ToString()),
				FText::FromString(GetFullNameSafe(DMXPixelMapping))
				).ToString();
			MessageLog.Error(*Msg, this);
			return;
		}
	}
}

void UK2Node_PixelMappingBaseComponent::ExecuteExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, const FName& SubsystemFuncitonName, UEdGraphPin* InComponentNamePin, UEdGraphPin* OutComponentPin)
{
	const UEdGraphSchema_K2* K2Schema = CompilerContext.GetSchema();

	// First node to execute. 
	FName GetPixelMappingSubsystemFunctionName = GET_FUNCTION_NAME_CHECKED(UDMXPixelMappingSubsystem, GetDMXPixelMappingSubsystem_Pure);
	const UFunction* GetPixelMappingSubsystemFunction = UDMXPixelMappingSubsystem::StaticClass()->FindFunctionByName(GetPixelMappingSubsystemFunctionName);
	check(nullptr != GetPixelMappingSubsystemFunction);
	UK2Node_CallFunction* PixelMappingSubsystemNode = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	PixelMappingSubsystemNode->SetFromFunction(GetPixelMappingSubsystemFunction);
	PixelMappingSubsystemNode->AllocateDefaultPins();

	UEdGraphPin* PixelMappingSubsystemResult = PixelMappingSubsystemNode->GetReturnValuePin();

	// Second node to execute. GetMatrixComponent
	const UFunction* GetComponentFunction = UDMXPixelMappingSubsystem::StaticClass()->FindFunctionByName(SubsystemFuncitonName);
	check(nullptr != GetComponentFunction);

	// Spawn call function node
	UK2Node_CallFunction* GetMatrixComponentCallFunction = CompilerContext.SpawnIntermediateNode<UK2Node_CallFunction>(this, SourceGraph);
	GetMatrixComponentCallFunction->FunctionReference.SetExternalMember(SubsystemFuncitonName, UK2Node_CallFunction::StaticClass());

	// Set function node pins
	GetMatrixComponentCallFunction->SetFromFunction(GetComponentFunction);
	GetMatrixComponentCallFunction->AllocateDefaultPins();

	// Hook up function node inputs

	UEdGraphPin* FunctionInDMXPixelMappingPin = GetMatrixComponentCallFunction->FindPinChecked(TEXT("InDMXPixelMapping"));
	UEdGraphPin* FunctionInComponentNamePin = GetMatrixComponentCallFunction->FindPinChecked(TEXT("InComponentName"));
	UEdGraphPin* FunctionOutMatrixComponentPin = GetMatrixComponentCallFunction->FindPinChecked(UEdGraphSchema_K2::PN_ReturnValue);
	UEdGraphPin* FunctionSelfPin = GetMatrixComponentCallFunction->FindPinChecked(UEdGraphSchema_K2::PN_Self);

	// Hook up input
	UEdGraphPin* InPixelMappingPin = GetInPixelMappingPin();
	K2Schema->TryCreateConnection(FunctionSelfPin, PixelMappingSubsystemResult);

	if(InPixelMappingPin->LinkedTo.Num())
	{
		CompilerContext.MovePinLinksToIntermediate(*InPixelMappingPin, *FunctionInDMXPixelMappingPin);
	}
	else
	{
		K2Schema->TrySetDefaultObject(*FunctionInDMXPixelMappingPin, InPixelMappingPin->DefaultObject);
	}
	
	if (InComponentNamePin->LinkedTo.Num())
	{
		CompilerContext.MovePinLinksToIntermediate(*InComponentNamePin, *FunctionInComponentNamePin);
	}
	else
	{
		K2Schema->TrySetDefaultValue(*FunctionInComponentNamePin, InComponentNamePin->DefaultValue);
	}

	// Hook up Matrix
	CompilerContext.MovePinLinksToIntermediate(*OutComponentPin, *FunctionOutMatrixComponentPin);
}

#undef LOCTEXT_NAMESPACE
