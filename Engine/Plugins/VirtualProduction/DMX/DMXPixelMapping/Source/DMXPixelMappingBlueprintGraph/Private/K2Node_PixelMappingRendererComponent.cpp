// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Blueprint/DMXPixelMappingSubsystem.h"

#include "EdGraphSchema_K2.h"

#define LOCTEXT_NAMESPACE "UK2Node_PixelMappingRendererComponent"

const FName UK2Node_PixelMappingRendererComponent::InRendererComponentPinName(TEXT("In Component"));
const FName UK2Node_PixelMappingRendererComponent::OutRendererComponentPinName(TEXT("Out Component"));

FText UK2Node_PixelMappingRendererComponent::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Get DMX Pixel Mapping Renderer Component");
}

void UK2Node_PixelMappingRendererComponent::AllocateDefaultPins()
{
	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	// Allocate Parent pins
	Super::AllocateDefaultPins();

	// Input pins
	UEdGraphPin* InRendererComponentPin = CreatePin(EGPD_Input, UEdGraphSchema_K2::PC_Name, InRendererComponentPinName);
	K2Schema->ConstructBasicPinTooltip(*InRendererComponentPin, LOCTEXT("InRendererComponentPin", "Input for Renderer Component"), InRendererComponentPin->PinToolTip);

	// Add output pin
	UEdGraphPin* OutputRendererComponentPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Object, UDMXPixelMappingRendererComponent::StaticClass(), OutRendererComponentPinName);
	K2Schema->ConstructBasicPinTooltip(*OutputRendererComponentPin, LOCTEXT("OutputRendererComponentPin", "Renderer Component"), OutputRendererComponentPin->PinToolTip);
}

void UK2Node_PixelMappingRendererComponent::PinDefaultValueChanged(UEdGraphPin* ChangedPin)
{
	Super::PinDefaultValueChanged(ChangedPin);

	TryRefreshGraphCheckInputPins(ChangedPin, GetOutRendererComponentPin());
}

void UK2Node_PixelMappingRendererComponent::ExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph)
{
	Super::ExpandNode(CompilerContext, SourceGraph);

	ExecuteExpandNode(CompilerContext, SourceGraph, GET_FUNCTION_NAME_CHECKED(UDMXPixelMappingSubsystem, GetRendererComponent), GetInRendererComponentPin(), GetOutRendererComponentPin());
}

void UK2Node_PixelMappingRendererComponent::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	AddBlueprintAction(GetClass(), ActionRegistrar);
}

UEdGraphPin* UK2Node_PixelMappingRendererComponent::GetInRendererComponentPin() const
{
	if (UEdGraphPin* Pin = FindPin(InRendererComponentPinName))
	{
		check(Pin->Direction == EGPD_Input);
		return Pin;
	}
	return nullptr;
}

UEdGraphPin* UK2Node_PixelMappingRendererComponent::GetOutRendererComponentPin() const
{
	if (UEdGraphPin* Pin = FindPin(OutRendererComponentPinName))
	{
		check(Pin->Direction == EGPD_Output);
		return Pin;
	}
	return nullptr;
}

void UK2Node_PixelMappingRendererComponent::EarlyValidation(FCompilerResultsLog& MessageLog) const
{
	Super::EarlyValidation(MessageLog);

	ExecuteEarlyValidation(MessageLog, GetInRendererComponentPin());
}

void UK2Node_PixelMappingRendererComponent::OnPixelMappingChanged(UDMXPixelMapping* InDMXPixelMapping)
{
	TryModifyBlueprintOnNameChanged(InDMXPixelMapping, GetInRendererComponentPin());
}

#undef LOCTEXT_NAMESPACE
