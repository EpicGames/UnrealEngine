// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphSchema.h"
#include "SGraphPin.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SComboBox.h"

#include "DMXPixelMapping.h" // IWYU pragma: keep

struct FComponentUserNameTable;

/** 
 * Cusotom widget for Pixel Mapping component pin.
 */
template<typename TComponentClass>
class SDMXPixelMappingComponentPin
	: public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingComponentPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj, UDMXPixelMapping* InDMXPixelMapping)
	{
		DMXPixelMappingWeakPtr = InDMXPixelMapping;

		NameTable = FComponentUserNameTable(InDMXPixelMapping);

		SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
	}

protected:

	/**
	 *	Function to create class specific widget.
	 *
	 *	@return Reference to the newly created widget object
	 */
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override
	{
		NameTable.GetUserNames(UserNameList);

		// Preserve previous selection if possible
		const FName SelectedComponentName = [&]()
		{
			if (GraphPinObj)
			{				
				FName PreviousSelection = FName(*GraphPinObj->GetDefaultAsString());

				if (NameTable.Contains(PreviousSelection))
				{
					return PreviousSelection;
				}
				else if (NameTable.Num() > 0)
				{
					return NameTable.First().Key;
				}
			}				
			return FName(NAME_None);
		}();

		SetNameToPin(SelectedComponentName);

		// Show the user  name, instead of the component's FName
		FString UserName = NameTable.GetUserName(SelectedComponentName);
		const TSharedPtr<FString>* SelectedUserNamePtr = UserNameList.FindByPredicate([UserName](const TSharedPtr<FString>& TestedUserName) {
			return TestedUserName->Equals(UserName);
			});
		const TSharedPtr<FString> SelectedUserName = SelectedUserNamePtr ? *SelectedUserNamePtr : nullptr;

		return SAssignNew(ComboBox, SComboBox<TSharedPtr<FString>>)
			.ContentPadding(FMargin(2.0f, 2.0f, 2.0f, 1.0f))
			.OptionsSource(&UserNameList)
			.InitiallySelectedItem(SelectedUserName)
			.OnGenerateWidget(this, &SDMXPixelMappingComponentPin::GenerateComboBoxEntry)
			.OnSelectionChanged(this, &SDMXPixelMappingComponentPin::ComboBoxSelectionChanged)
			.IsEnabled(this, &SGraphPin::IsEditingEnabled)
			.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
			[
				SNew(STextBlock)
				.Text(this, &SDMXPixelMappingComponentPin::GetSelectedUserName)
			];
	}

	/**
	 *	Function to set the newly selected index
	 *
	 * @param NameItem The newly selected item in the combo box
	 * @param SelectInfo Provides context on how the selection changed
	 */
	void ComboBoxSelectionChanged(TSharedPtr<FString> UserNameItem, ESelectInfo::Type SelectInfo)
	{
		if (GraphPinObj && UserNameItem.IsValid())
		{
			FString UserName = *UserNameItem.Get();
			FName ComponentName = NameTable.GetComponentName(UserName);

			SetNameToPin(ComponentName);
		}
	}
	
private:	
	TSharedRef<SWidget> GenerateComboBoxEntry(TSharedPtr<FString> UserName)
	{
		return
			SNew(STextBlock)
			.Text(FText::FromString(*UserName));
	}

	FText GetSelectedUserName() const
	{
		if (GraphPinObj)
		{
			FName Selection = FName(*GraphPinObj->GetDefaultAsString());
			return FText::FromString(NameTable.GetUserName(Selection));
		}

		return FText::GetEmpty();
	}

	/** Set name from Combo Box to input pin */
	void SetNameToPin(const FName& Name)
	{
		if (const UEdGraphSchema* Schema = (GraphPinObj ? GraphPinObj->GetSchema() : nullptr))
		{
			FString NameAsString = Name.ToString();
			if (GraphPinObj->GetDefaultAsString() != NameAsString)
			{
				const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangeNameListPinValue", "Change Name List Pin Value"));
				GraphPinObj->Modify();

				Schema->TrySetDefaultValue(*GraphPinObj, NameAsString);
			}
		}
	}

private:
	/** Weak pointer to Pixel Mapping Object */
	TWeakObjectPtr<UDMXPixelMapping> DMXPixelMappingWeakPtr;

	/** Reference to Combo Box object */
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ComboBox;

	TArray<TSharedPtr<FString>> UserNameList;

	/** Maps component FNames to the corresponding User Name displayed in pixel mapping designer */
	struct FComponentUserNameTable
	{
		FComponentUserNameTable()
		{}

		FComponentUserNameTable(UDMXPixelMapping* PixelMapping)
		{
			if (PixelMapping)
			{
				TArray<TComponentClass*> ComponentArr;
				PixelMapping->GetAllComponentsOfClass<TComponentClass>(ComponentArr);
				for (const TComponentClass* Component : ComponentArr)
				{
					if (Component)
					{
						// GetUserName will display what is shown everywhere else in the UI
						TTuple<FName, FString> Tuple = TTuple<FName, FString>(Component->GetFName(), Component->GetUserName());
						ComponentUserNameArr.Add(Tuple);
					}
				}
			}
		}

		bool Contains(const FName& ComponentName) const
		{
			int32 Index = ComponentUserNameArr.IndexOfByPredicate([&ComponentName](const TTuple<FName, FString>& ComponentUserKvp) {
				return ComponentUserKvp.Key == ComponentName;
				});
			return Index != INDEX_NONE;
		}

		void GetUserNames(TArray<TSharedPtr<FString>>& OutUserNames)
		{
			OutUserNames.Reset();
			for (const TTuple<FName, FString>& ComponentUserKvp : ComponentUserNameArr)
			{
				OutUserNames.Add(MakeShared<FString>(ComponentUserKvp.Value));
			}
		}

		FString GetUserName(const FName& ComponentName) const
		{
			int32 Index = ComponentUserNameArr.IndexOfByPredicate([&ComponentName](const TTuple<FName, FString>& ComponentUserKvp) {
				return ComponentUserKvp.Key == ComponentName;
				});

			if (Index != INDEX_NONE)
			{
				return ComponentUserNameArr[Index].Value;
			}

			return FString(TEXT("None"));
		}

		FName GetComponentName(const FString& UserName) const
		{
			for (const TTuple<FName, FString>& ComponentUserKvp : ComponentUserNameArr)
			{
				if (ComponentUserKvp.Value == UserName)
				{
					return ComponentUserKvp.Key;
				}
			}

			return NAME_None;
		}

		int32 Num() const
		{
			return ComponentUserNameArr.Num();
		}

		TTuple<FName, FString> First() const
		{
			check(ComponentUserNameArr.Num() > 0);
			return ComponentUserNameArr[0];
		}

	private:
		TArray<TTuple<FName, FString>> ComponentUserNameArr;
	};

	/** List of available component names */
	FComponentUserNameTable NameTable;
};
