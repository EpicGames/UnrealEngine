// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"

#include "K2Node_PixelMappingBaseComponent.generated.h"

class FKismetCompilerContext;
class UDMXPixelMapping;


/**
 * Base Pixel Mapping node. Never use directly
 */
UCLASS(abstract)
class DMXPIXELMAPPINGBLUEPRINTGRAPH_API UK2Node_PixelMappingBaseComponent
	: public UK2Node
{
	GENERATED_BODY()

public:
	//~ Begin UEdGraphNode Interface
	virtual bool IsNodePure() const override { return true; }
	virtual void AllocateDefaultPins() override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual FText GetMenuCategory() const override;
	virtual void PreloadRequiredAssets() override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	//~ End UK2Node Interface

	/** 
	 * Listener of Pixel Mapping object changes.
	 */
	virtual void OnPixelMappingChanged(UDMXPixelMapping* InDMXPixelMapping) {}

	/** Pixel Mapping Input Pin */
	UEdGraphPin* GetInPixelMappingPin() const;

protected:
	/** Safe method to get Pixel Mapping pin. It skips the checked pointer */
	UEdGraphPin* GetPixelMappingPin(const TArray<UEdGraphPin*>* InPinsToSearch = nullptr) const;

	/** Serves as an extensible way for new nodes */
	void AddBlueprintAction(UClass* InClass, FBlueprintActionDatabaseRegistrar& ActionRegistrar) const;

	/** Asking to change blueprint graph */
	void RefreshGraph();

	/** Mark Blueprint graph as Modified */
	void ModifyBlueprint();

	/** Modify blueprint if we have the component name change */
	void TryModifyBlueprintOnNameChanged(UDMXPixelMapping* InDMXPixelMapping, UEdGraphPin* InPin);

	/** Refresh blueprint graph if input pins is valid */
	void TryRefreshGraphCheckInputPins(UEdGraphPin* TryPixelMappingPin, UEdGraphPin* TryComponentNamePin);

	/** Execute valisation for the connected pins */
	void ExecuteEarlyValidation(FCompilerResultsLog& MessageLog, UEdGraphPin* InComponentPin) const;

	/**
	 * Add Execution for blueprint node
	 * That is the place where we set a compiler and spawn extra node for blueprint execution
	 *
	 * @param CompilerContext			Kismet blueprint compiler
	 * @param SourceGraph				Current blueprint graph pointer
	 * @param SubsystemFuncitonName		C++ subsustem function to execute
	 * @param InComponentNamePin		Input component FName pin pointer
	 * @param OutComponentPin			Out component UObject pin
	 */
	void ExecuteExpandNode(FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph, const FName& SubsystemFuncitonName, UEdGraphPin* InComponentNamePin, UEdGraphPin* OutComponentPin);

protected:
	/** Input Pixel Mapping pin name */
	static const FName InPixelMappingPinName;
};
