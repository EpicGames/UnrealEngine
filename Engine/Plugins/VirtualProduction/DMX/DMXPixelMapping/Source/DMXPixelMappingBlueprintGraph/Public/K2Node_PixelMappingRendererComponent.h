// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node_PixelMappingBaseComponent.h"

#include "K2Node_PixelMappingRendererComponent.generated.h"

/**
 * Node for getting Renderer Component from PixelMapping object and Renderer FName
 */
UCLASS()
class DMXPIXELMAPPINGBLUEPRINTGRAPH_API UK2Node_PixelMappingRendererComponent
	: public UK2Node_PixelMappingBaseComponent
{
	GENERATED_BODY()

public:

	//~ Begin UEdGraphNode Interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AllocateDefaultPins() override;
	virtual void PinDefaultValueChanged(UEdGraphPin* ChangedPin) override;
	//~ End UEdGraphNode Interface

	//~ Begin UK2Node Interface
	virtual bool IsNodePure() const override { return true; }
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void EarlyValidation(class FCompilerResultsLog& MessageLog) const override;
	//~ End UK2Node Interface

	//~ Begin UK2Node_PixelMappingBaseComponent Interface
	virtual void OnPixelMappingChanged(UDMXPixelMapping* InDMXPixelMapping) override;
	//~ End UK2Node_PixelMappingBaseComponent Interface

public:
	/** 
	 * Pointer to the input Renderer group pin.
	 * The pin holds the FName of the non Public UObject component.
	 * Since it not possible to save NonPublic UObject references outside uasset it should be used as FName
	 */
	UEdGraphPin* GetInRendererComponentPin() const;

	/** 
	 * Pointer to the output Renderer group pin.
	 * It dynamically returns a pointer to Renderer Component by input FName of the component.  
	 */
	UEdGraphPin* GetOutRendererComponentPin() const;

public:
	/** Input Renderer Component pin name. It holds a FName of the component. */
	static const FName InRendererComponentPinName;

	/** Output Renderer Component pin name. It holds a pointer to the component. */
	static const FName OutRendererComponentPinName;
};
