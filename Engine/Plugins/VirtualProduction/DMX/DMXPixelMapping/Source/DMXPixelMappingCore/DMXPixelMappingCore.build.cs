// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXPixelMappingCore : ModuleRules
{
	public DMXPixelMappingCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DMXProtocol",
				"DMXRuntime",
			}
		);
	}
}
