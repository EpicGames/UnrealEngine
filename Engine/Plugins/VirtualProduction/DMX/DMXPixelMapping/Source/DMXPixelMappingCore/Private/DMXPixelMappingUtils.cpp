// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingUtils.h"
#include "DMXPixelMappingTypes.h"

uint32 FDMXPixelMappingUtils::GetNumChannelsPerCell(EDMXCellFormat InCellFormat)
{
	switch (InCellFormat)
	{
	case EDMXCellFormat::PF_RG:
	case EDMXCellFormat::PF_RB:
	case EDMXCellFormat::PF_GB:
	case EDMXCellFormat::PF_GR:
	case EDMXCellFormat::PF_BR:
	case EDMXCellFormat::PF_BG:
		return 2;
	case EDMXCellFormat::PF_RGB:
	case EDMXCellFormat::PF_BRG:
	case EDMXCellFormat::PF_GRB:
	case EDMXCellFormat::PF_GBR:
		return 3;
	case EDMXCellFormat::PF_RGBA:
	case EDMXCellFormat::PF_GBRA:
	case EDMXCellFormat::PF_BRGA:
	case EDMXCellFormat::PF_GRBA:
		return 4;
	}

	return 1;
}

uint32 FDMXPixelMappingUtils::GetUniverseMaxChannels(EDMXCellFormat InCellFormat, uint32 InStartAddress)
{
	uint32 NumChannelsPerCell = FDMXPixelMappingUtils::GetNumChannelsPerCell(InCellFormat);

	return DMX_MAX_ADDRESS - ((DMX_MAX_ADDRESS - (InStartAddress - 1)) % NumChannelsPerCell);
}

bool FDMXPixelMappingUtils::CanFitCellIntoChannels(EDMXCellFormat InCellFormat, uint32 InStartAddress)
{
	return (InStartAddress + GetNumChannelsPerCell(InCellFormat) - 1) <= DMX_MAX_ADDRESS;
}
