// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "DMXPixelMappingTypes.generated.h"


UENUM(BlueprintType)
enum class EDMXPixelMappingRendererType : uint8
{
    Texture,
    Material,
    UMG
};

UENUM(BlueprintType)
enum class EDMXCellFormat : uint8
{
	PF_R UMETA(DisplayName = "R"),
	PF_G UMETA(DisplayName = "G"),
	PF_B UMETA(DisplayName = "B"),

	PF_RG UMETA(DisplayName = "RG"),
	PF_RB UMETA(DisplayName = "RB"),
	PF_GB UMETA(DisplayName = "GB"),
	PF_GR UMETA(DisplayName = "GR"),
	PF_BR UMETA(DisplayName = "BR"),
	PF_BG UMETA(DisplayName = "BG"),

	PF_RGB UMETA(DisplayName = "RGB"),
	PF_BRG UMETA(DisplayName = "BRG"),
	PF_GRB UMETA(DisplayName = "GRB"),
	PF_GBR  UMETA(DisplayName = "GBR"),

	PF_RGBA UMETA(DisplayName = "RGBA"),
	PF_GBRA UMETA(DisplayName = "GBRA"),
	PF_BRGA UMETA(DisplayName = "BRGA"),
	PF_GRBA UMETA(DisplayName = "GRBA"),
};

UENUM(BlueprintType)
enum class EDMXColorMode : uint8
{
	CM_RGB UMETA(DisplayName = "RGB"),
	CM_Monochrome UMETA(DisplayName = "Monochrome"),
};
