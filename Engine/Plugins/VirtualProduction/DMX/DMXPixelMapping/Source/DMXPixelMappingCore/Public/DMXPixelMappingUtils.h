// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#include "Library/DMXEntityFixtureType.h"

enum class EDMXCellFormat : uint8;

/**
 *	Core helper classes
 */
struct FDMXPixelMappingUtils
{
public:
	/**
	 * Sort a distribution array for texture
	 *
	 * @param InDistribution			Distribution schema
	 * @param InNumXPanels				Num columns
	 * @param InNumYPanels				Num rows
	 * @param InUnorderedList			Templated unsorted array input
	 * @param OutSortedList				Templated unsorted array output
	 */
	template<typename T>
	static void TextureDistributionSort(EDMXPixelMappingDistribution InDistribution, const int32& InNumXPanels, const int32& InNumYPanels, const TArray<T>& InUnorderedList, TArray<T>& OutSortedList)
	{
		int32 GridSize = InNumXPanels * InNumYPanels;

		if (InDistribution == EDMXPixelMappingDistribution::TopLeftToRight)
		{
			// Do nothing this is default
			OutSortedList = InUnorderedList;
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopLeftToBottom)
		{
			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					int32 Index = (YIndex * InNumXPanels) + XIndex;
					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopLeftToClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if (YIndex % 2 == 0)
					{
						OutSortedList.Add(InUnorderedList[YIndex * InNumXPanels + XIndex]);
					}
					else
					{
						OutSortedList.Add(InUnorderedList[((YIndex + 1) * InNumXPanels) - (XIndex + 1)]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopLeftToAntiClockwise)
		{
			int32 Index = 0;
			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					if (InNumXPanels % 2 == 0)
					{
						if (XIndex % 2 == 0)
						{
							Index = (YIndex * InNumXPanels) + XIndex;
							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex - 1)) + XIndex;
							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
					else
					{
						if (XIndex % 2 == 0)
						{
							Index = (YIndex * InNumXPanels) + XIndex;
							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex)) - (1 + XIndex);
							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToLeft)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					int32 Index = ((YIndex + 1) * InNumXPanels) - (XIndex + 1);
					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftToTop)
		{
			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					int32 Index = (InNumXPanels * (InNumYPanels - YIndex)) - (InNumXPanels - XIndex);
					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToAntiClockwise)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if ((YIndex % 2) == 0)
					{
						OutSortedList.Add(InUnorderedList[(InNumXPanels - XIndex - 1) + (YIndex * InNumXPanels)]);
					}
					else
					{
						OutSortedList.Add(InUnorderedList[XIndex + YIndex * InNumXPanels]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftToClockwise)
		{
			int32 Index = 0;

			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					if ((XIndex % 2) == 0)
					{
						Index = (InNumXPanels * (InNumYPanels - YIndex)) - (InNumXPanels - XIndex);

						OutSortedList.Add(InUnorderedList[Index]);
					}
					else
					{
						Index = ((1 + YIndex) * InNumXPanels) - (InNumXPanels - XIndex);
						OutSortedList.Add(InUnorderedList[Index]);
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftToRight)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					int32 Index = ((InNumYPanels - YIndex) * InNumXPanels) - (InNumXPanels - XIndex);

					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToBottom)
		{
			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					int32 Index = ((YIndex + 1) * InNumXPanels) - (XIndex + 1);
					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomLeftAntiClockwise)
		{
			int32 Index = 0;
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if (InNumYPanels % 2 == 0)
					{
						if (YIndex % 2 == 0)
						{
							Index = ((InNumYPanels - YIndex) * InNumXPanels) - (InNumXPanels - XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex)) - (1 + XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
					else
					{
						if (YIndex % 2 == 0)
						{
							Index = ((InNumYPanels - YIndex) * InNumXPanels) - (InNumXPanels - XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels - XIndex - 1) + (YIndex * InNumXPanels);

							OutSortedList.Add(InUnorderedList[Index]);
						}

					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::TopRightToClockwise)
		{
			int32 Index = 0;

			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					if (InNumXPanels % 2 == 0)
					{
						if (XIndex % 2 == 0)
						{
							Index = ((YIndex + 1) * InNumXPanels) - (XIndex + 1);
							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex)) - (XIndex + 1);
							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
					else
					{
						if (XIndex % 2 == 0)
						{
							Index = ((YIndex + 1) * InNumXPanels) - (XIndex + 1);
							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex)) - (InNumXPanels - XIndex);
							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToLeft)
		{
			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					int32 Index = (InNumXPanels * (InNumYPanels - YIndex)) - (1 + XIndex);

					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToTop)
		{
			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					int32 Index = (InNumXPanels * InNumYPanels) - ((YIndex * InNumXPanels) + 1 + XIndex);

					OutSortedList.Add(InUnorderedList[Index]);
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToClockwise)
		{
			int32 Index = 0;

			for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
			{
				for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
				{
					if (InNumXPanels % 2 == 0)
					{
						if (YIndex % 2 == 0)
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex)) - (1 + XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = ((InNumYPanels - YIndex) * InNumXPanels) - (InNumXPanels - XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
					else
					{
						if (YIndex % 2 == 0)
						{
							Index = ((InNumYPanels - YIndex) * InNumXPanels) - (InNumXPanels - XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (InNumXPanels * (InNumYPanels - YIndex)) - (1 + XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
				}
			}
		}
		else if (InDistribution == EDMXPixelMappingDistribution::BottomRightToAntiClockwise)
		{
			int32 Index = 0;

			for (int32 XIndex = 0; XIndex < InNumXPanels; ++XIndex)
			{
				for (int32 YIndex = 0; YIndex < InNumYPanels; ++YIndex)
				{
					if (InNumXPanels % 2 == 0)
					{
						if (XIndex % 2 == 0)
						{
							Index = (InNumXPanels * InNumYPanels) - ((YIndex * InNumXPanels) + 1 + XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = (YIndex * InNumXPanels) + (InNumXPanels - (XIndex + 1));

							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
					else
					{
						if (XIndex % 2 == 0)
						{
							Index = (InNumXPanels * InNumYPanels) - ((YIndex * InNumXPanels) + 1 + XIndex);

							OutSortedList.Add(InUnorderedList[Index]);
						}
						else
						{
							Index = ((YIndex + 1) * InNumXPanels) - (XIndex + 1);

							OutSortedList.Add(InUnorderedList[Index]);
						}
					}
				}
			}
		}
	}

	/**
	 * Calculate amount of the channels
	 *
	 * @param					InPixelFormat given pixel format. From 1 up to 4 colors per pixel
	 * @return Amount of the channels
	 */
	DMXPIXELMAPPINGCORE_API static uint32 GetNumChannelsPerCell(EDMXCellFormat InPixelFormat);

	/**
	 * Dynamically calculate amount of the channels based on Pixel Format schema
	 * @param					InPixelFormat given pixel format. From 1 up to 4 colors per pixel
	 * @param					InStartAddress start universe address
	 * @return Max channel value
	 */
	DMXPIXELMAPPINGCORE_API static uint32 GetUniverseMaxChannels(EDMXCellFormat InPixelFormat, uint32 InStartAddress);

	/**
	 * Checking is there enough channel space for giver offset
	 * @param					InPixelFormat given pixel format. From 1 up to 4 colors per pixel
	 * @param					InStartAddress start universe address
	 * @return True if it possible to fit at lease one pixel
	 */
	DMXPIXELMAPPINGCORE_API static bool CanFitCellIntoChannels(EDMXCellFormat InPixelFormat, uint32 InStartAddress);
};
