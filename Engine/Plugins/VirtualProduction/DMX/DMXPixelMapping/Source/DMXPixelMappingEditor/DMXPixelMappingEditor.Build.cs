// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DMXPixelMappingEditor : ModuleRules
{
	public DMXPixelMappingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		var EngineDir = Path.GetFullPath(Target.RelativeEnginePath);

		PrivateDependencyModuleNames.AddRange( new string[] {
			"AppFramework",
			"ApplicationCore",
			"AssetDefinition",
			"Core",
			"CoreUObject",
			"DMXEditor",
			"DMXPixelMappingCore",
			"DMXPixelMappingEditorWidgets",
			"DMXPixelMappingRuntime",
			"DMXPixelMappingRenderer",
			"DMXPixelMappingBlueprintGraph",
			"DMXRuntime",
			"EditorStyle",
			"Engine",
			"InputCore",
			"Projects",
			"PropertyEditor",
			"RenderCore",
			"RHI",
			"Slate",
			"SlateCore",
			"ToolMenus",
			"ToolWidgets",
			"UMG",
			"UnrealEd",
		});
	}
}
