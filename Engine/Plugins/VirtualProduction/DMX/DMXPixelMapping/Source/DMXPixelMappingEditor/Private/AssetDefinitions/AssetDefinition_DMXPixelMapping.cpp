// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_DMXPixelMapping.h"

#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorLog.h"
#include "Toolkits/DMXPixelMappingToolkit.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_DMXPixelMapping"

FText UAssetDefinition_DMXPixelMapping::GetAssetDisplayName() const
{
	return LOCTEXT("DMXPixelMapping_AssetName", "DMX Pixel Mapping");
}

TSoftClassPtr<UObject> UAssetDefinition_DMXPixelMapping::GetAssetClass() const
{
	return UDMXPixelMapping::StaticClass();
}

FLinearColor UAssetDefinition_DMXPixelMapping::GetAssetColor() const
{
	return FColor::Red;
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_DMXPixelMapping::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("DMXPixelMapping_AssetCategory", "Virtual Production")), LOCTEXT("DMXPixelMapping_CategorySection", "DMX"), ECategoryMenuType::Section)
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_DMXPixelMapping::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	EToolkitMode::Type Mode = OpenArgs.ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (UObject* Object : Objects)
	{
		if (UDMXPixelMapping* DMXPixelMapping = Cast<UDMXPixelMapping>(Object))
		{
			TSharedRef<FDMXPixelMappingToolkit> PixelMappingToolkit(MakeShared<FDMXPixelMappingToolkit>());
			PixelMappingToolkit->InitPixelMappingEditor(Mode, OpenArgs.ToolkitHost, DMXPixelMapping);
		}
		else
		{
			UE_LOGF(LogDMXPixelMappingEditor, Warning, "Wrong object class for pixel mapping editor %ls", *Object->GetClass()->GetFName().ToString());
		}
	}
	
	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
