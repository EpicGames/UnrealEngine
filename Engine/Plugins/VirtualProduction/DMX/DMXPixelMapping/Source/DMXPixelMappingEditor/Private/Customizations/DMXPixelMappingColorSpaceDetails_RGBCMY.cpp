// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingColorSpaceDetails_RGBCMY.h"

#include "ColorSpace/DMXPixelMappingColorSpace_RGBCMY.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingColorSpaceDetails_RGBCMY"

TSharedRef<IDetailCustomization> FDMXPixelMappingColorSpaceDetails_RGBCMY::MakeInstance()
{
	return MakeShared<FDMXPixelMappingColorSpaceDetails_RGBCMY>();
}

void FDMXPixelMappingColorSpaceDetails_RGBCMY::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Hide bCyan, bMagenta, bYellow properties, instead show them inline
	SendCyanHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, bSendCyan));
	SendMagentaHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, bSendMagenta));
	SendYellowHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, bSendYellow));
	SendCyanHandle->MarkHiddenByCustomization();
	SendMagentaHandle->MarkHiddenByCustomization();
	SendYellowHandle->MarkHiddenByCustomization();

	FDMXAttributeRowData RedAttributeRowData;
	RedAttributeRowData.AttributeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, RedAttribute));
	RedAttributeRowData.AttributeLabel = TAttribute<FText>::CreateLambda([this]() { return GetRedAttributeDisplayName(); });
	RedAttributeRowData.IsInvertColorHandle = SendCyanHandle;
	RedAttributeRowData.InvertColorLabel = LOCTEXT("InvertRedToCyanLabel", "Send Cyan");
	GenerateAttributeRow(DetailBuilder, RedAttributeRowData);

	FDMXAttributeRowData GreenAttributeRowData;
	GreenAttributeRowData.AttributeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, GreenAttribute));
	GreenAttributeRowData.AttributeLabel = TAttribute<FText>::CreateLambda([this]() { return GetGreenAttributeDisplayName(); });
	GreenAttributeRowData.IsInvertColorHandle = SendMagentaHandle;
	GreenAttributeRowData.InvertColorLabel = LOCTEXT("InvertGreenToMagentaLabel", "Send Magenta");
	GenerateAttributeRow(DetailBuilder, GreenAttributeRowData);

	FDMXAttributeRowData BlueAttributeRowData;
	BlueAttributeRowData.AttributeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, BlueAttribute));
	BlueAttributeRowData.AttributeLabel = TAttribute<FText>::CreateLambda([this]() { return GetBlueAttributeDisplayName(); });
	BlueAttributeRowData.IsInvertColorHandle = SendYellowHandle;
	BlueAttributeRowData.InvertColorLabel = LOCTEXT("InvertBlueToMagentaLabel", "Send Yellow");
	BlueAttributeRowData.bAppendSeparator = true;
	GenerateAttributeRow(DetailBuilder, BlueAttributeRowData);
}

void FDMXPixelMappingColorSpaceDetails_RGBCMY::GenerateAttributeRow(IDetailLayoutBuilder& DetailBuilder, const FDMXAttributeRowData& AttributeRowData)
{
	IDetailCategoryBuilder& ColorSpaceCategory = DetailBuilder.EditCategory("RGB");
	IDetailPropertyRow& Row = ColorSpaceCategory.AddProperty(AttributeRowData.AttributeHandle);

	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow DefaultWidgetRow;
	Row.GetDefaultWidgets(NameWidget, ValueWidget, DefaultWidgetRow);

	Row.CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(AttributeRowData.AttributeLabel)
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBox)
				.MinDesiredWidth(120.f)
				[
					ValueWidget.ToSharedRef()
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				AttributeRowData.IsInvertColorHandle->CreatePropertyValueWidget()
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(AttributeRowData.InvertColorLabel)
			]
		];
}

FText FDMXPixelMappingColorSpaceDetails_RGBCMY::GetRedAttributeDisplayName() const
{
	bool bIsCyan = false;
	SendCyanHandle->GetValue(bIsCyan);

	return bIsCyan ?
		LOCTEXT("RedAsCyanAttributeDisplayName", "Cyan Attribute") :
		LOCTEXT("RedAttributeDisplayName", "Red Attribute");
}

FText FDMXPixelMappingColorSpaceDetails_RGBCMY::GetGreenAttributeDisplayName() const
{
	bool bIsMagenta = false;
	SendMagentaHandle->GetValue(bIsMagenta);

	return bIsMagenta ?
		LOCTEXT("GreenAsMagentaAttributeDisplayName", "Magenta Attribute") :
		LOCTEXT("GreenAttributeDisplayName", "Green Attribute");
}

FText FDMXPixelMappingColorSpaceDetails_RGBCMY::GetBlueAttributeDisplayName() const
{
	bool bIsYellow = false;
	SendYellowHandle->GetValue(bIsYellow);

	return bIsYellow ?
		LOCTEXT("BlueAsYellowAttributeDisplayName", "Yellow Attribute") :
		LOCTEXT("BlueAttributeDisplayName", "Blue Attribute");
}

#undef LOCTEXT_NAMESPACE
