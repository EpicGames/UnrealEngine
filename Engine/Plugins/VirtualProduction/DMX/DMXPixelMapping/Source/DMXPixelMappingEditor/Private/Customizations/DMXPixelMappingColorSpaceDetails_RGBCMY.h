// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;
class IPropertyHandle;


/** Details customization for the RGBCMY Color Space */
class FDMXPixelMappingColorSpaceDetails_RGBCMY
	: public IDetailCustomization
{
public:
	/** Creates an instance of this customization */
	static TSharedRef<IDetailCustomization> MakeInstance();

	//~ Begin IDetailCustomization interface 
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization interface 

private:
	/** Holds required data to create an Attribute row */
	struct FDMXAttributeRowData
	{
		TSharedPtr<IPropertyHandle> AttributeHandle;
		TAttribute<FText> AttributeLabel;
		TSharedPtr<IPropertyHandle> IsInvertColorHandle;
		FText InvertColorLabel;
		bool bAppendSeparator = false;
	};

	/** Generates the row for one of the Attributes */
	void GenerateAttributeRow(IDetailLayoutBuilder& DetailBuilder, const FDMXAttributeRowData& AttributeRowData);

	/** Returns the Display Name of the Red Attribute */
	FText GetRedAttributeDisplayName() const;

	/** Returns the Display Name of the Red Attribute */
	FText GetGreenAttributeDisplayName() const;

	/** Returns the Display Name of the Red Attribute */
	FText GetBlueAttributeDisplayName() const;

	/** Property Handle for the bSendCyan Property */
	TSharedPtr<IPropertyHandle> SendCyanHandle;

	/** Property Handle for the bSendMagenta Property */
	TSharedPtr<IPropertyHandle> SendMagentaHandle;

	/** Property Handle for the bSendYellow Property */
	TSharedPtr<IPropertyHandle> SendYellowHandle;
};

