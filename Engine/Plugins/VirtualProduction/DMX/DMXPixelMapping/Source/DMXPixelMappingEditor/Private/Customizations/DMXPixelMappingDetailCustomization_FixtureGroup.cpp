// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingDetailCustomization_FixtureGroup.h"

#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingDetailCustomization_FixtureGroup"

void FDMXPixelMappingDetailCustomization_FixtureGroup::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	PropertyUtilities = InDetailLayout.GetPropertyUtilities();

	// Hide the Layout Script property (shown in its own panel, see SDMXPixelMappingLayoutView)
	InDetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupComponent, DMXLibrary));
	InDetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupComponent, LayoutScript));

	// Listen to component changes
	UDMXPixelMappingBaseComponent::GetOnComponentAdded().AddSP(this, &FDMXPixelMappingDetailCustomization_FixtureGroup::OnComponentAddedOrRemoved);
}

void FDMXPixelMappingDetailCustomization_FixtureGroup::OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	PropertyUtilities->RequestRefresh();
}

#undef LOCTEXT_NAMESPACE
