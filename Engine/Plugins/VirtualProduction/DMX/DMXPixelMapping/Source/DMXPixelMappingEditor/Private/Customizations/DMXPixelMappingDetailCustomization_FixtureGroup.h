// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FDMXPixelMappingToolkit;
struct FPropertyChangedEvent;
class IDetailLayoutBuilder;
class IPropertyUtilities;
class UDMXPixelMapping;
class UDMXPixelMappingBaseComponent;



class FDMXPixelMappingDetailCustomization_FixtureGroup
	: public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
	{
		return MakeShared<FDMXPixelMappingDetailCustomization_FixtureGroup>(InToolkitWeakPtr);
	}

	/** Constructor */
	FDMXPixelMappingDetailCustomization_FixtureGroup(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
		: ToolkitWeakPtr(InToolkitWeakPtr)
	{}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	/** Called when a component was added or removed */
	void OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXPixelMappingToolkit> ToolkitWeakPtr;

	TSharedPtr<IPropertyUtilities> PropertyUtilities;
};
