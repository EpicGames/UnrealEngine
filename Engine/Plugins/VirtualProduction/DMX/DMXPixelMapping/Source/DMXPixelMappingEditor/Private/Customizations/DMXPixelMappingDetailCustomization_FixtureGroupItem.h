// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FDMXPixelMappingToolkit;
class IDetailLayoutBuilder;
class IPropertyHandle;


class FDMXPixelMappingDetailCustomization_FixtureGroupItem
	: public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr);

	FDMXPixelMappingDetailCustomization_FixtureGroupItem(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr);
	
	//~ IDetailCustomization interface begin
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Returns true if the EditorColor property is editable */
	bool CanEditEditorColor() const;

	/** Handle for the bUsePatchColor property */
	TSharedPtr<IPropertyHandle> UsePatchColorHandle;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXPixelMappingToolkit> ToolkitWeakPtr;
};
