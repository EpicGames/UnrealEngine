// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingDetailCustomization_Matrix.h"

#include "Components/DMXPixelMappingMatrixComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"


FDMXPixelMappingDetailCustomization_Matrix::FDMXPixelMappingDetailCustomization_Matrix(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
	: ToolkitWeakPtr(InToolkitWeakPtr)
{}

TSharedRef<IDetailCustomization> FDMXPixelMappingDetailCustomization_Matrix::MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
{
	return MakeShared<FDMXPixelMappingDetailCustomization_Matrix>(InToolkitWeakPtr);
}

void FDMXPixelMappingDetailCustomization_Matrix::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	// Hide the Layout Script property
	InDetailLayout.HideProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, LayoutScript));

	// Disable the editor color property, if the component Uses the patch color
	IDetailCategoryBuilder& CategoryBuilder = InDetailLayout.EditCategory("Editor Settings");

	UsePatchColorHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputDMXComponent, bUsePatchColor), UDMXPixelMappingOutputDMXComponent::StaticClass());
	CategoryBuilder.AddProperty(UsePatchColorHandle);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// The EditorColor property should no longer be publicly accessed. However it's ok to access it here.
	const FName EditorColorMemberName = GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, EditorColor);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const TSharedRef<IPropertyHandle> EditorColorHandle = InDetailLayout.GetProperty(EditorColorMemberName, UDMXPixelMappingOutputComponent::StaticClass());
	CategoryBuilder.AddProperty(EditorColorHandle)
		.EditCondition(TAttribute<bool>::CreateSP(this, &FDMXPixelMappingDetailCustomization_Matrix::CanEditEditorColor), FOnBooleanValueChanged());

	// Sort categories
	InDetailLayout.SortCategories([](const TMap<FName, IDetailCategoryBuilder*>& CategoryMap)
		{
			int32 SortOrder = 0;

			IDetailCategoryBuilder* const* FixturePatchCategoryPtr = CategoryMap.Find("Fixture Patch");
			if (FixturePatchCategoryPtr)
			{
				(*FixturePatchCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* ColorSpaceCategoryPtr = CategoryMap.Find("Color Space");
			if (ColorSpaceCategoryPtr)
			{
				(*ColorSpaceCategoryPtr)->SetSortOrder(SortOrder++);
			}

			// Either 'RGB' 'XY' or 'XYZ' is displayed
			IDetailCategoryBuilder* const* RGBCategoryPtr = CategoryMap.Find("RGB");
			if (RGBCategoryPtr)
			{
				(*RGBCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* XYCategoryPtr = CategoryMap.Find("XY");
			if (XYCategoryPtr)
			{
				(*XYCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* XYZCategoryPtr = CategoryMap.Find("XYZ");
			if (XYZCategoryPtr)
			{
				(*XYZCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* IntensityCategoryPtr = CategoryMap.Find("Luminance");
			if (IntensityCategoryPtr)
			{
				(*IntensityCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* QualityCategoryPtr = CategoryMap.Find("Quality");
			if (QualityCategoryPtr)
			{
				(*QualityCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* ModulatorsCategoryPtr = CategoryMap.Find("Output Modulators");
			if (ModulatorsCategoryPtr)
			{
				(*ModulatorsCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* TransformCategoryPtr = CategoryMap.Find("Transform");
			if (TransformCategoryPtr)
			{
				(*TransformCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* MatrixCategoryPtr = CategoryMap.Find("Matrix");
			if (MatrixCategoryPtr)
			{
				(*MatrixCategoryPtr)->SetSortOrder(SortOrder++);
			}

			IDetailCategoryBuilder* const* EditorSettingsCategoryPtr = CategoryMap.Find("Editor Settings");
			if (EditorSettingsCategoryPtr)
			{
				(*EditorSettingsCategoryPtr)->SetSortOrder(SortOrder++);
			}
		});
}

bool FDMXPixelMappingDetailCustomization_Matrix::CanEditEditorColor() const
{
	bool bUsePatchColor;
	if (UsePatchColorHandle->GetValue(bUsePatchColor) == FPropertyAccess::Success)
	{
		return !bUsePatchColor;
	}

	return true;
}
