// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class FDMXPixelMappingToolkit;
class IDetailLayoutBuilder;
class IPropertyHandle;


class FDMXPixelMappingDetailCustomization_Matrix
	: public IDetailCustomization
{
public:
	FDMXPixelMappingDetailCustomization_Matrix(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr);

	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr);

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:	
	/** Returns true if the EditorColor property is editable */
	bool CanEditEditorColor() const;

	/** Handle for the bUsePatchColor property */
	TSharedPtr<IPropertyHandle> UsePatchColorHandle;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXPixelMappingToolkit> ToolkitWeakPtr;
};
