// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingDetailCustomization_OutputDMX.h"

#include "Components/DMXPixelMappingOutputDMXComponent.h"
#include "DetailsViewArgs.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "IDetailsView.h"
#include "Modulators/DMXModulator.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingDetailCustomization_OutputDMX"

void FDMXPixelMappingDetailCustomization_OutputDMX::CustomizeDetails(IDetailLayoutBuilder& InDetailLayout)
{
	PropertyUtilities = InDetailLayout.GetPropertyUtilities();

	CreateModulatorDetails(InDetailLayout);
}

void FDMXPixelMappingDetailCustomization_OutputDMX::CreateModulatorDetails(IDetailLayoutBuilder& InDetailLayout)
{
	IDetailCategoryBuilder& ModualtorsCategory = InDetailLayout.EditCategory("Output Modulators", LOCTEXT("DMXModulatorsCategory", "Output Modulators"), ECategoryPriority::Important);

	const TSharedPtr<IPropertyHandle> ModulatorClassesHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputDMXComponent, ModulatorClasses));
	ModulatorClassesHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXPixelMappingDetailCustomization_OutputDMX::ForceRefresh));
	ModulatorClassesHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXPixelMappingDetailCustomization_OutputDMX::ForceRefresh));

	ModualtorsCategory.AddProperty(ModulatorClassesHandle);

	const TSharedPtr<IPropertyHandle> ModulatorsHandle = InDetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputDMXComponent, Modulators));
	InDetailLayout.HideProperty(ModulatorsHandle);

	// Create detail views for the modulators
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	InDetailLayout.GetObjectsBeingCustomized(CustomizedObjects);
	if (CustomizedObjects.Num() > 0)
	{
		if (UDMXPixelMappingOutputDMXComponent* FirstOutputDMXComponent = Cast<UDMXPixelMappingOutputDMXComponent>(CustomizedObjects[0].Get()))
		{
			for (int32 IndexModulator = 0; IndexModulator < FirstOutputDMXComponent->Modulators.Num(); IndexModulator++)
			{
				TArray<UObject*> ModulatorsToEdit;
				if (CustomizedObjects.Num() > 1)
				{
					UClass* ModulatorClass = FirstOutputDMXComponent->Modulators[IndexModulator]->GetClass();

					for (const TWeakObjectPtr<UObject>& CustomizedObject : CustomizedObjects)
					{
						if (UDMXPixelMappingOutputDMXComponent* OutputDMXComponent = Cast<UDMXPixelMappingOutputDMXComponent>(CustomizedObject.Get()))
						{
							const bool bMultiEditableModulator =
								OutputDMXComponent->Modulators.IsValidIndex(IndexModulator) &&
								OutputDMXComponent->Modulators[IndexModulator] &&
								OutputDMXComponent->Modulators[IndexModulator]->GetClass() == ModulatorClass;

							if (bMultiEditableModulator)
							{
								ModulatorsToEdit.Add(OutputDMXComponent->Modulators[IndexModulator]);
							}
							else
							{
								// Don't allow multi edit if not all modulators are of same class
								ModulatorsToEdit.Reset();
							}
						}
					}
				}
				else if (UDMXModulator* ModulatorOfFirstGroupItem = FirstOutputDMXComponent->Modulators[IndexModulator])
				{
					ModulatorsToEdit.Add(ModulatorOfFirstGroupItem);
				}


				if (ModulatorsToEdit.Num() > 0)
				{
					FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

					FDetailsViewArgs DetailsViewArgs;
					DetailsViewArgs.bUpdatesFromSelection = false;
					DetailsViewArgs.bLockable = true;
					DetailsViewArgs.bAllowSearch = false;
					DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
					DetailsViewArgs.bHideSelectionTip = false;
					DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;

					TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
					DetailsView->SetObjects(ModulatorsToEdit);

					ModualtorsCategory.AddCustomRow(FText::GetEmpty())
						.WholeRowContent()
						[
							DetailsView
						];
				}
				else
				{
					ModualtorsCategory.AddCustomRow(FText::GetEmpty())
						.WholeRowContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ModulatorMultipleValues", "Multiple Values"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
						];

					break;
				}
			}
		}
	}
}

void FDMXPixelMappingDetailCustomization_OutputDMX::ForceRefresh()
{	
	// For some reason IPropertyUtilities::RequestRefresh doesn't recognize changed modulators
	PropertyUtilities->ForceRefresh();
}

#undef LOCTEXT_NAMESPACE
