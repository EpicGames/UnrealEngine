// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IPropertyUtilities;
class FDMXPixelMappingToolkit;
class IDetailLayoutBuilder;


class FDMXPixelMappingDetailCustomization_OutputDMX
	: public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
	{
		return MakeShared<FDMXPixelMappingDetailCustomization_OutputDMX>(InToolkitWeakPtr);
	}

	/** Constructor */
	FDMXPixelMappingDetailCustomization_OutputDMX(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
		: ToolkitWeakPtr(InToolkitWeakPtr)
	{}

	//~ IDetailCustomization interface begin
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ IPropertyTypeCustomization interface end

private:
	/** Creates Details for the Output Modulators */
	void CreateModulatorDetails(IDetailLayoutBuilder& InDetailLayout);
	
	/** Forces the details to refresh */
	void ForceRefresh();

	/** Property utilities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXPixelMappingToolkit> ToolkitWeakPtr;
};
