// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingDetailCustomization_Renderer.h"

#include "Blueprint/UserWidget.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DMXPixelMappingTypes.h"
#include "Engine/Texture.h"
#include "IPropertyUtilities.h"
#include "Materials/Material.h"
#include "SWarningOrErrorBox.h"
#include "TextureResource.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingDetailCustomization_Renderer"


FDMXPixelMappingDetailCustomization_Renderer::FDMXPixelMappingDetailCustomization_Renderer(const TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit)
	: WeakToolkit(InWeakToolkit)
{}

void FDMXPixelMappingDetailCustomization_Renderer::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	PropertyUtilities = DetailLayout.GetPropertyUtilities();

	InputTextureHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, InputTexture));
	InputMaterialHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, InputMaterial));
	InputWidgetHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, InputWidget));
	RendererTypeHandle = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, RendererType));


	// Hide Output Component properties
	const TSharedRef<IPropertyHandle> PositionXHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetPositionXPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(PositionXHandle);
	const TSharedRef<IPropertyHandle> PositionYHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetPositionXPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(PositionYHandle);

	const TSharedRef<IPropertyHandle> SizeXHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetSizeXPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(SizeXHandle);
	const TSharedRef<IPropertyHandle> SizeYHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetSizeYPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(SizeYHandle);

	const TSharedRef<IPropertyHandle> RotationHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetRotationPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(RotationHandle);

	const TSharedRef<IPropertyHandle> IsLockInDesignerHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetLockInDesignerPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(IsLockInDesignerHandle);

	const TSharedRef<IPropertyHandle> IsVisibleInDesignerHandle = DetailLayout.GetProperty(UDMXPixelMappingOutputComponent::GetVisibleInDesignerPropertyName(), UDMXPixelMappingOutputComponent::StaticClass());
	DetailLayout.HideProperty(IsVisibleInDesignerHandle);

	// Add properties
	IDetailCategoryBuilder& RenderSettingsCategory = DetailLayout.EditCategory("Render Settings", FText::GetEmpty(), ECategoryPriority::Important);
	RenderSettingsCategory.AddProperty(RendererTypeHandle);

	// Get editing UObject
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailLayout.GetObjectsBeingCustomized(CustomizedObjects);

	// Only show warnings if not multi-selecting
	CustomizedObjects.Remove(nullptr);
	if (CustomizedObjects.Num() == 1)
	{
		WeakRendererComponent = Cast<UDMXPixelMappingRendererComponent>(CustomizedObjects[0]);

		// Add Warning InputTexture Row
		AddInputTextureWarning(RenderSettingsCategory);

		// Add non UI Material Warning
		AddMaterialDomainWarning(RenderSettingsCategory);

		// Register properties
		RenderSettingsCategory.AddProperty(InputTextureHandle)
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]
				{
					return GetRendererType() == EDMXPixelMappingRendererType::Texture ? EVisibility::Visible : EVisibility::Hidden;
				})));

		RenderSettingsCategory.AddProperty(InputMaterialHandle)
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]
				{
					return GetRendererType() == EDMXPixelMappingRendererType::Material ? EVisibility::Visible : EVisibility::Hidden;
				})));

		RenderSettingsCategory.AddProperty(InputWidgetHandle)
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([this]
				{
					return GetRendererType() == EDMXPixelMappingRendererType::UMG ? EVisibility::Visible : EVisibility::Hidden;
				})));
	}
}

void FDMXPixelMappingDetailCustomization_Renderer::AddInputTextureWarning(IDetailCategoryBuilder& InCategory)
{
	const FSlateBrush* WarningIcon = FAppStyle::GetBrush("SettingsEditor.WarningIcon");

	InCategory.AddCustomRow(FText::GetEmpty())
		.Visibility(TAttribute<EVisibility>(this, &FDMXPixelMappingDetailCustomization_Renderer::GetInputTextureWarningVisibility))
		.WholeRowContent()
		[
			SNew(SWarningOrErrorBox)
			.MessageStyle(EMessageStyle::Warning)
			.Message(this, &FDMXPixelMappingDetailCustomization_Renderer::GetInputTextureWarningText)
		];
}

void FDMXPixelMappingDetailCustomization_Renderer::AddMaterialDomainWarning(IDetailCategoryBuilder& InCategory)
{
	InCategory.AddCustomRow(FText::GetEmpty())
		.Visibility(TAttribute<EVisibility>(this, &FDMXPixelMappingDetailCustomization_Renderer::GetMaterialDomainWarningVisibility))
		.WholeRowContent()
		[
			SNew(SWarningOrErrorBox)
			.MessageStyle(EMessageStyle::Warning)
			.Message(LOCTEXT("WarningNonUIMaterial", "Selected Material is not UI Material.\nChange Material Domain to User Interface.\nOr select another Material."))
		];
}


EVisibility FDMXPixelMappingDetailCustomization_Renderer::GetInputTextureWarningVisibility() const
{
	UDMXPixelMappingRendererComponent* RendererComponent = WeakRendererComponent.Get();
	if (!RendererComponent)
	{
		return EVisibility::Collapsed;
	}

	const EDMXPixelMappingRendererType RendererType = GetRendererType();
	if (RendererType == EDMXPixelMappingRendererType::Texture)
	{
		return RendererComponent->InputTexture ? EVisibility::Collapsed : EVisibility::Visible;
	}
	else if (RendererType == EDMXPixelMappingRendererType::Material)
	{
		return RendererComponent->InputMaterial ? EVisibility::Collapsed : EVisibility::Visible;
	}
	else if (RendererType == EDMXPixelMappingRendererType::UMG)
	{
		return RendererComponent->InputWidget ? EVisibility::Collapsed : EVisibility::Visible;
	}
	else
	{
		ensureMsgf(0, TEXT("Renderer type not implemented in FDMXPixelMappingDetailCustomization_Renderer"));
		return EVisibility::Collapsed;
	}
}

FText FDMXPixelMappingDetailCustomization_Renderer::GetInputTextureWarningText() const
{
	const EDMXPixelMappingRendererType RendererType = GetRendererType();
	if (RendererType == EDMXPixelMappingRendererType::Texture)
	{
		return LOCTEXT("WarningCategoryDisplayName.TextureNotSet", "Texture is not set.");
	}
	else if (RendererType == EDMXPixelMappingRendererType::Material)
	{
		return LOCTEXT("WarningCategoryDisplayName.MaterialNotSet", "Material is not set.");
	}
	else if (RendererType == EDMXPixelMappingRendererType::UMG)
	{
		return LOCTEXT("WarningCategoryDisplayName.UMGNotSet", "UMG is not set.");
	}
	else
	{
		ensureMsgf(0, TEXT("Renderer type not implemented in FDMXPixelMappingDetailCustomization_Renderer"));
		return FText::GetEmpty();
	}
}

EVisibility FDMXPixelMappingDetailCustomization_Renderer::GetMaterialDomainWarningVisibility() const
{
	UDMXPixelMappingRendererComponent* RendererComponent = WeakRendererComponent.Get();
	if (!RendererComponent || RendererComponent->RendererType != EDMXPixelMappingRendererType::Material)
	{
		return EVisibility::Collapsed;
	}

	UMaterial* Material = RendererComponent->InputMaterial ? RendererComponent->InputMaterial->GetMaterial() : nullptr;
	if (Material && !Material->IsUIMaterial())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EDMXPixelMappingRendererType FDMXPixelMappingDetailCustomization_Renderer::GetRendererType() const
{
	TArray<void*> RawDataArray;
	RendererTypeHandle->AccessRawData(RawDataArray);

	return RawDataArray.IsEmpty() ?
		EDMXPixelMappingRendererType::Texture :
		*static_cast<EDMXPixelMappingRendererType*>(RawDataArray[0]);
}

#undef LOCTEXT_NAMESPACE
