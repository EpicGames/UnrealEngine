// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

enum class EDMXPixelMappingRendererType : uint8;
struct EVisibility;
class FDMXPixelMappingToolkit;
class FText;
class IDetailCategoryBuilder;
class IPropertyHandle;
class IPropertyUtilities;
class UDMXPixelMappingRendererComponent;


class FDMXPixelMappingDetailCustomization_Renderer
	: public IDetailCustomization
{
public:
	FDMXPixelMappingDetailCustomization_Renderer(const TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit);

	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit)
	{
		return MakeShared<FDMXPixelMappingDetailCustomization_Renderer>(InWeakToolkit);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

private:
	/** Adds a warning if the input texture is not set */
	void AddInputTextureWarning(IDetailCategoryBuilder& InCategory);

	/** Add warning for material with domain != MD_UI */
	void AddMaterialDomainWarning(IDetailCategoryBuilder& InCategory);

	/** Returns the text displayed if the input texture is not set */
	FText GetInputTextureWarningText() const;

	/** Returns the visiblity of the input texture warning text */
	EVisibility GetInputTextureWarningVisibility() const;

	/** Returns the visibility of the input material domain */
	EVisibility GetMaterialDomainWarningVisibility() const;

	/** Returns the current renderer type */
	EDMXPixelMappingRendererType GetRendererType() const;

	TSharedPtr<IPropertyHandle> RendererTypeHandle;
	TSharedPtr<IPropertyHandle> InputTextureHandle;
	TSharedPtr<IPropertyHandle> InputMaterialHandle;
	TSharedPtr<IPropertyHandle> InputWidgetHandle;

	/** Renderer components being customized. Only valid if not multi editing */
	TWeakObjectPtr<UDMXPixelMappingRendererComponent> WeakRendererComponent;

	/** Property utilities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
