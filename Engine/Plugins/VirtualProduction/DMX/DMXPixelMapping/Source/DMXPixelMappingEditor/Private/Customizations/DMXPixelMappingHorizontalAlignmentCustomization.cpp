// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingHorizontalAlignmentCustomization.h"

#include "DetailWidgetRow.h"
#include "Widgets/Input/SSegmentedControl.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingHorizontalAlignmentCustomization"

TSharedRef<class IPropertyTypeCustomization> FDMXPixelMappingHorizontalAlignmentCustomization::MakeInstance()
{
	return MakeShared<FDMXPixelMappingHorizontalAlignmentCustomization>();
}

void FDMXPixelMappingHorizontalAlignmentCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(0)
	[
		SNew(SSegmentedControl<EHorizontalAlignment>)
		.Value(this, &FDMXPixelMappingHorizontalAlignmentCustomization::GetAlignment, PropertyHandle)
		.OnValueChanged(this, &FDMXPixelMappingHorizontalAlignmentCustomization::OnAlignmentChanged, PropertyHandle)
		+SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Left)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Left"))
			.ToolTip(LOCTEXT("HAlignLeft", "Left Align Horizontally"))
		+ SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Center)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Center"))
			.ToolTip(LOCTEXT("HAlignCenter", "Center Align Horizontally"))
		+ SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Right)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Right"))
			.ToolTip(LOCTEXT("HAlignRight", "Right Align Horizontally"))
		+ SSegmentedControl<EHorizontalAlignment>::Slot(EHorizontalAlignment::HAlign_Fill)
			.Icon(FAppStyle::GetBrush("HorizontalAlignment_Fill"))
			.ToolTip(LOCTEXT("HAlignFill", "Fill Horizontally"))
	];
}

void FDMXPixelMappingHorizontalAlignmentCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

EHorizontalAlignment FDMXPixelMappingHorizontalAlignmentCustomization::GetAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	uint8 Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return (EHorizontalAlignment)Value;
	}

	return HAlign_Fill;
}

void FDMXPixelMappingHorizontalAlignmentCustomization::OnAlignmentChanged(EHorizontalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue((uint8)NewAlignment);
}

#undef LOCTEXT_NAMESPACE
