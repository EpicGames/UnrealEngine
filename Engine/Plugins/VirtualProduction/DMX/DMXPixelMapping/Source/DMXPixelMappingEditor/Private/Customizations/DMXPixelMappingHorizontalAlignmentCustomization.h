// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

enum EHorizontalAlignment : int;


class FDMXPixelMappingHorizontalAlignmentCustomization
	: public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<class IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	//~ End IPropertyTypeCustomization interface

private:
	/** Returns the Aligment */
	EHorizontalAlignment GetAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const;

	/** Called when the Alignment changed */
	void OnAlignmentChanged(EHorizontalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle);
};
