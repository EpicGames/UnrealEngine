// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingLayoutViewModelDetails.h"

#include "Components/DMXPixelMappingOutputComponent.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "ViewModels/DMXPixelMappingLayoutViewModel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingLayoutViewModelDetails"

TSharedRef<IDetailCustomization> FDMXPixelMappingLayoutViewModelDetails::MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
{
	return MakeShared<FDMXPixelMappingLayoutViewModelDetails>(InToolkitWeakPtr);
}

void FDMXPixelMappingLayoutViewModelDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	PropertyUtilities = DetailLayout.GetPropertyUtilities();

	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	UDMXPixelMappingLayoutViewModel* ViewModel = GetLayoutViewModel();
	if (!Toolkit.IsValid() || !ViewModel)
	{
		return;
	}
	
	const TSharedPtr<IPropertyHandle> LayoutScriptClassHandle = DetailLayout.GetProperty(UDMXPixelMappingLayoutViewModel::GetLayoutScriptClassPropertyName());
	IDetailCategoryBuilder& LayoutScriptCategory = DetailLayout.EditCategory(LayoutScriptClassHandle->GetDefaultCategoryName());

	// Add a checkbox to apply layout scripts instantly
	const FText ApplyInstantlyTooltip = LOCTEXT("ApplyLayoutScriptWhenLoadedTooltip", "When checked, applies the Layout Script instantly when a new script is selected.");
	LayoutScriptCategory.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ApplyLayoutScriptWhenLoadedLabel", "Apply New Scripts Instantly"))
			.ToolTipText(ApplyInstantlyTooltip)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.ToolTipText(ApplyInstantlyTooltip)
			.IsChecked_Lambda([]()
				{
					return GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings.bApplyLayoutScriptWhenLoaded ?
						ECheckBoxState::Checked :
						ECheckBoxState::Unchecked;
				})
			.OnCheckStateChanged_Lambda([](ECheckBoxState NewCheckboxState)
			{
				UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>();
				EditorSettings->DesignerSettings.bApplyLayoutScriptWhenLoaded = NewCheckboxState == ECheckBoxState::Checked;
				EditorSettings->SaveConfig();
			})
		];

	// Add the auto apply property
	AutoApplyHandle = DetailLayout.GetProperty(UDMXPixelMappingLayoutViewModel::GetAutoApplyPropertyName());
	LayoutScriptCategory.AddProperty(AutoApplyHandle)
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FDMXPixelMappingLayoutViewModelDetails::GetAutoApplyPropertyVisiblity));

	// Add an apply button
	LayoutScriptCategory.AddCustomRow(FText::GetEmpty())
		.Visibility(TAttribute<EVisibility>::CreateSP(this, &FDMXPixelMappingLayoutViewModelDetails::GetApplyLayoutScriptButtonVisibility))
		.WholeRowContent()
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.ForegroundColor(FLinearColor::White)
			.ContentPadding(FMargin(5.f, 1.f))
			.OnClicked(this, &FDMXPixelMappingLayoutViewModelDetails::OnApplyLayoutScriptClicked)
			.Content()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("NoBorder"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ApplyLayoutScriptLabel", "Apply Script"))
				]
			]
		];

	// Show info about which components are being laid out, or why no layout can be applied
	LayoutScriptCategory
		.AddCustomRow(FText::GetEmpty())
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LayouotModeLabel", "Applied to"))
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		]
		.ValueContent()
		[
			SAssignNew(InfoTextBlock, STextBlock)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
		];
	UpdateInfoText();

	// Show the Layout Script Class property depending on the layout view model mode.
	const TAttribute<EVisibility> LayoutScriptClassVisibilityAttribute = TAttribute<EVisibility>::CreateLambda([this]()
		{
			const UDMXPixelMappingLayoutViewModel* const Model = GetLayoutViewModel();
			if (Model && Model->GetMode() != EDMXPixelMappingLayoutViewModelMode::LayoutNone)
			{
				return EVisibility::Visible;
			}
			return EVisibility::Collapsed;	
		});

	LayoutScriptCategory
		.AddProperty(LayoutScriptClassHandle)
		.Visibility(LayoutScriptClassVisibilityAttribute);

	// Update info on changes
	ViewModel->OnModelChanged.AddSP(this, &FDMXPixelMappingLayoutViewModelDetails::UpdateInfoText);
	Toolkit->GetOnSelectedComponentsChangedDelegate().AddSP(this, &FDMXPixelMappingLayoutViewModelDetails::UpdateInfoText);
}

EVisibility FDMXPixelMappingLayoutViewModelDetails::GetAutoApplyPropertyVisiblity() const
{
	if (UDMXPixelMappingLayoutViewModel* LayoutViewModel = GetLayoutViewModel())
	{
		return LayoutViewModel->CanApplyLayoutScript() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

EVisibility FDMXPixelMappingLayoutViewModelDetails::GetApplyLayoutScriptButtonVisibility() const
{
	bool bAutoApply;
	if (AutoApplyHandle->GetValue(bAutoApply) == FPropertyAccess::Success &&
		!bAutoApply)
	{
		if (UDMXPixelMappingLayoutViewModel* LayoutViewModel = GetLayoutViewModel())
		{
			return LayoutViewModel->CanApplyLayoutScript() ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	return EVisibility::Collapsed;
}

FReply FDMXPixelMappingLayoutViewModelDetails::OnApplyLayoutScriptClicked()
{
	if (UDMXPixelMappingLayoutViewModel* LayoutViewModel = GetLayoutViewModel())
	{
		const FScopedTransaction ApplyLayoutScriptTransaction(LOCTEXT("ApplyLayoutScriptTransaction", "Apply Layout Script"));
		LayoutViewModel->ForceApplyLayoutScript();
	}

	return FReply::Handled();
}

void FDMXPixelMappingLayoutViewModelDetails::UpdateInfoText()
{
	if (UDMXPixelMappingLayoutViewModel* LayoutViewModel = GetLayoutViewModel())
	{
		UDMXPixelMappingOutputComponent* ParentComponent = LayoutViewModel->GetParentComponent();

		const FText InfoText = [ParentComponent, this]()
			{		
				const FText ParentComponentNameText = ParentComponent ?
					FText::FromString(ParentComponent->GetUserName()) :
					LOCTEXT("InvalidParentName", "Invalid Parent");

				const FText ChildrenOfText = LOCTEXT("LayoutChildrenOfText", "Children of {0}");

				const UDMXPixelMappingLayoutViewModel* const Model = GetLayoutViewModel();
				if (!ParentComponent || !Model || Model->GetMode() == EDMXPixelMappingLayoutViewModelMode::LayoutNone)
				{
					return LOCTEXT("NoLayoutCanBeAppliedInfo", "Not applicable to the current selection");
				}
				else if (Model->GetMode() == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren)
				{
					return FText::Format(ChildrenOfText, ParentComponentNameText);
				}
				else if (Model->GetMode() == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren)
				{
					return FText::Format(ChildrenOfText, ParentComponentNameText);
				}
				else if (Model->GetMode() == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren)
				{
					return FText::Format(ChildrenOfText, ParentComponentNameText);
				}
				return FText::GetEmpty();
			}();

		InfoTextBlock->SetText(InfoText);
	}
}

UDMXPixelMappingLayoutViewModel* FDMXPixelMappingLayoutViewModelDetails::GetLayoutViewModel() const
{
	const TArray<TWeakObjectPtr<UObject>> SelectedObjects = PropertyUtilities->GetSelectedObjects();

	if (SelectedObjects.Num() == 1)
	{
		return  Cast<UDMXPixelMappingLayoutViewModel>(SelectedObjects[0]);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
