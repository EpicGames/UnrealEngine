// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"

struct EVisibility;
class FDMXPixelMappingToolkit;
class IPropertyHandle;
class IPropertyUtilities;
class STextBlock;
class UDMXPixelMappingLayoutViewModel;


/** Details customization for the Layout View Model */
class FDMXPixelMappingLayoutViewModelDetails
	: public IDetailCustomization
{
public:
	/** Creates an instance of this customization */
	static TSharedRef<IDetailCustomization> MakeInstance(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr);

	/** Constructor */
	FDMXPixelMappingLayoutViewModelDetails(const TWeakPtr<FDMXPixelMappingToolkit> InToolkitWeakPtr)
		: WeakToolkit(InToolkitWeakPtr)
	{}

	//~ Begin IDetailCustomization interface 
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	//~ End IDetailCustomization interface 

private:
	/** Returns the visibility for the bAutoApply property */
	EVisibility GetAutoApplyPropertyVisiblity() const;

	/** Returns the visibility for the 'Apply Layout Script' button */
	EVisibility GetApplyLayoutScriptButtonVisibility() const;

	/** Called when the apply layout script button was clicked */
	FReply OnApplyLayoutScriptClicked();

	/** Updates the info text */
	void UpdateInfoText();

	/** Retuns the object that is being customized */
	UDMXPixelMappingLayoutViewModel* GetLayoutViewModel() const;

	/** Weak reference to the DMX editor */
	TWeakPtr<FDMXPixelMappingToolkit> ToolkitWeakPtr;

	/** If true, shows the Layout Script Class property */
	bool bShowLayoutScriptClass = false;

	/** If true, shows an info why the Layout Script Class property is hidden */
	bool bShowLayoutScriptClassHiddenInfo = true;

	/** Text block displaying info about what's being arranged */
	TSharedPtr<STextBlock> InfoTextBlock;

	/** Property handle for the bAutoApply property */
	TSharedPtr<IPropertyHandle> AutoApplyHandle;

	/** Property utilities for this customization */
	TSharedPtr<IPropertyUtilities> PropertyUtilities;

	/** Weak reference to the Pixel Mapping Toolkit */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
