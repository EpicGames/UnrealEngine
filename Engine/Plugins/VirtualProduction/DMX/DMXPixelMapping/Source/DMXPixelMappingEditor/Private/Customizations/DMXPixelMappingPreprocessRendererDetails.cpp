// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingPreprocessRendererDetails.h"

#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMappingPreprocessRenderer.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialInterface.h"
#include "PropertyHandle.h"


namespace UE::DMXPixelMapping::Customizations
{
	TSharedRef<IDetailCustomization> FDMXPixelMappingPreprocessRendererDetails::MakeInstance()
	{
		return MakeShared<FDMXPixelMappingPreprocessRendererDetails>();
	}

	void FDMXPixelMappingPreprocessRendererDetails::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
	{
		PropertyUtilities = DetailLayout.GetPropertyUtilities();

		// Handle blur distance property enabled
		FilterMaterialHandle = DetailLayout.GetProperty(UDMXPixelMappingPreprocessRenderer::GetFilterMaterialMemberNameChecked());
		FilterMaterialHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXPixelMappingPreprocessRendererDetails::OnFilterMaterialChanged));
		FilterMaterialHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FDMXPixelMappingPreprocessRendererDetails::OnFilterMaterialChanged));

		if (!IsBlurDistanceMaterialParameterNameExisting())
		{
			const TSharedRef<IPropertyHandle> BlurDistanceHandle = DetailLayout.GetProperty(UDMXPixelMappingPreprocessRenderer::GetBlurDistanceMemberNameChecked());
			BlurDistanceHandle->MarkHiddenByCustomization();
		}
	}

	bool FDMXPixelMappingPreprocessRendererDetails::IsBlurDistanceMaterialParameterNameExisting() const
	{
		const TArray<TWeakObjectPtr<UObject>> EditedObjects = PropertyUtilities->GetSelectedObjects();

		for (TWeakObjectPtr<UObject> EditedObject : EditedObjects)
		{
			UDMXPixelMappingRendererComponent* RendererComponent = Cast<UDMXPixelMappingRendererComponent>(EditedObject);
			if (!RendererComponent)
			{
				continue;
			}

			UDMXPixelMappingPreprocessRenderer* PreprocessRenderer = RendererComponent->GetPreprocessRenderer();
			if (!ensureMsgf(PreprocessRenderer, TEXT("Missing default subobbject RenderInputTextureProxy in %s"), *RendererComponent->GetName()))
			{
				continue;
			}

			UObject* FilterMaterialObject = nullptr;
			if (FilterMaterialHandle->GetValue(FilterMaterialObject) != FPropertyAccess::Success)
			{
				continue;
			}

			UMaterialInterface* FilterMaterial = Cast<UMaterialInterface>(FilterMaterialObject);
			if (!FilterMaterial)
			{
				continue;
			}

			FMaterialParameterMetadata ParameterMetadata;
			return FilterMaterial->GetParameterValue(EMaterialParameterType::Scalar, FMemoryImageMaterialParameterInfo(PreprocessRenderer->GetBlurDistanceParameterName()), ParameterMetadata);
		}

		return false;
	}

	void FDMXPixelMappingPreprocessRendererDetails::OnFilterMaterialChanged()
	{
		PropertyUtilities->ForceRefresh();
	}
}
