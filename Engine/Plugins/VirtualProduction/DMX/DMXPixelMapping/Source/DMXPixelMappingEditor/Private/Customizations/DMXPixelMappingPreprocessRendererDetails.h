// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IPropertyHandle;
class IPropertyUtilities;


namespace UE::DMXPixelMapping::Customizations
{
	class FDMXPixelMappingPreprocessRendererDetails
		: public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance();

		/** IDetailCustomization interface */
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;

	private:
		/** Called when the filter material instance dynamic property changed */
		void OnFilterMaterialChanged();

		/** Tests if the blur distance parameter name exists on the filter material */
		bool IsBlurDistanceMaterialParameterNameExisting() const;

		/** Property handle for the FilterMaterial property */
		TSharedPtr<IPropertyHandle> FilterMaterialHandle;

		/** Property utilities for this customization */
		TSharedPtr<IPropertyUtilities> PropertyUtilities;
	};
}
