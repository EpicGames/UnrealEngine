// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/DMXPixelMappingVerticalAlignmentCustomization.h"

#include "DetailWidgetRow.h"
#include "Widgets/Input/SSegmentedControl.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingVerticalAlignmentCustomization"

TSharedRef<class IPropertyTypeCustomization> FDMXPixelMappingVerticalAlignmentCustomization::MakeInstance()
{
	return MakeShared<FDMXPixelMappingVerticalAlignmentCustomization>();
}

void FDMXPixelMappingVerticalAlignmentCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	const FMargin OuterPadding(2);
	const FMargin ContentPadding(2);

	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MaxDesiredWidth(0)
	[
		SNew(SSegmentedControl<EVerticalAlignment>)
		.Value(this, &FDMXPixelMappingVerticalAlignmentCustomization::GetAlignment, PropertyHandle)
		.OnValueChanged(this, &FDMXPixelMappingVerticalAlignmentCustomization::OnAlignmentChanged, PropertyHandle)
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Top)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Top"))
			.ToolTip(LOCTEXT("VAlignTop", "Top Align Vertically"))
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Center)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Center"))
			.ToolTip(LOCTEXT("VAlignCenter", "Center Align Vertically"))
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Bottom)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Bottom"))
			.ToolTip(LOCTEXT("VAlignBottom", "Bottom Align Vertically"))
		+ SSegmentedControl<EVerticalAlignment>::Slot(EVerticalAlignment::VAlign_Fill)
			.Icon(FAppStyle::GetBrush("VerticalAlignment_Fill"))
			.ToolTip(LOCTEXT("VAlignFill", "Fill Vertically"))
	];
}

void FDMXPixelMappingVerticalAlignmentCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

EVerticalAlignment FDMXPixelMappingVerticalAlignmentCustomization::GetAlignment(TSharedRef<IPropertyHandle> PropertyHandle) const
{
	uint8 Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Result::Success)
	{
		return (EVerticalAlignment)Value;
	}

	return VAlign_Fill;
}

void FDMXPixelMappingVerticalAlignmentCustomization::OnAlignmentChanged(EVerticalAlignment NewAlignment, TSharedRef<IPropertyHandle> PropertyHandle)
{
	PropertyHandle->SetValue((uint8)NewAlignment);
}

#undef LOCTEXT_NAMESPACE
