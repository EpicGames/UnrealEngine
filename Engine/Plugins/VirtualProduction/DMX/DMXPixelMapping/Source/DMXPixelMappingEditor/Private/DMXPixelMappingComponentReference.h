// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingBaseComponent.h"

#include "DMXPixelMappingComponentReference.generated.h"

class FDMXPixelMappingToolkit;

/**
 * The Component reference is a useful way to hold onto the selection in a way that allows for up to date access to the current preview object.
 * This is a safe way to communicate between different parts of the pixel mapping editor
 */
USTRUCT()
struct FDMXPixelMappingComponentReference
{
	GENERATED_BODY()

	friend FDMXPixelMappingToolkit;

public:
	FDMXPixelMappingComponentReference() = default;

	FDMXPixelMappingComponentReference(TSharedPtr<FDMXPixelMappingToolkit> InToolkit, UDMXPixelMappingBaseComponent* InComponent)
		: ToolkitWeakPtr(InToolkit)
		, Component(InComponent)
	{}

	/** Checks if widget reference is the same as another component reference, based on the template pointers. */
	bool operator==(const FDMXPixelMappingComponentReference& Other) const
	{
		if (IsValid() && Other.IsValid())
		{
			return GetComponent() == Other.GetComponent();
		}

		return false;
	}

	/** Checks if widget reference is the different from another component reference, based on the template pointers. */
	bool operator!=(const FDMXPixelMappingComponentReference& Other) const
	{
		return !operator==(Other);
	}

	bool IsValid() const { return Component.Get() != nullptr; }

	/** @returns stored component pointer */
	UDMXPixelMappingBaseComponent* GetComponent() const { return Component.Get(); }

	friend uint32 GetTypeHash(const FDMXPixelMappingComponentReference& ComponentRef)
	{
		return GetTypeHash(ComponentRef.GetComponent());
	}

private:
	TWeakPtr<FDMXPixelMappingToolkit> ToolkitWeakPtr;

	UPROPERTY(Transient)
	TWeakObjectPtr<UDMXPixelMappingBaseComponent> Component;
};
