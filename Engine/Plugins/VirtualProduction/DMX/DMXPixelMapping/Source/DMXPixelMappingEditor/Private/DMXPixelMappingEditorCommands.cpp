// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingEditorCommands.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingEditorCommands"

FDMXPixelMappingEditorCommands::FDMXPixelMappingEditorCommands()
	: TCommands<FDMXPixelMappingEditorCommands>
	(
		TEXT("DMXPixelMappingEditor"),
		NSLOCTEXT("Contexts", "DMXPixelMappingEditor", "DMX Pixel Mapping"),
		NAME_None,
		FAppStyle::GetAppStyleSetName()
	)
{}

void FDMXPixelMappingEditorCommands::RegisterCommands()
{
	UI_COMMAND(AddMapping, "Add Source", "Adds a new Source Texure, Material or User Widget to the Pixel Map asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayDMX, "Plays DMX", "Plays DMX", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PauseDMX, "Pause DMX", "Pauses playing DMX", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResumeDMX, "Resume DMX", "Resumes playing DMX after being paused", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopDMX, "Stop DMX", "Stops playing DMX.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TogglePlayPauseDMX, "Toggle Play/Pause DMX", "Toggles between playing and pausing DMX", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::SpaceBar));
	UI_COMMAND(TogglePlayStopDMX, "Toggle Play/Stop DMX", "Toggles between playing and stopping DMX", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));

	UI_COMMAND(EditorStopSendsZeroValues, "Stop Sends Zero Values", "When stop is clicked in the editor, zero values are sent to all patches in use", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(EditorStopSendsDefaultValues, "Stop Sends Default Values", "When stop is clicked in the editor, default values are sent to all patches in use", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(EditorStopKeepsLastValues, "Stop Keeps Last Values", "When stop is clicked in the editor, pixel mapping leaves the last sent values untouched", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(EnableResizeMode, "Resize Mode", "Resizes components when transform handles are being dragged", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(EnableRotateMode, "Rotate Mode", "Rotates components when transform handles are being dragged", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ToggleGridSnapping, "Toggle Grid Snapping", "Enables/disables grid snapping", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(SizeGroupToTexture, "Size Group to Texture", "Sizes the selected group to Texture.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FlipGroupHorizontally, "Flip Group Horizontally", "Flips children of selected group along its relative Y-Axis.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FlipGroupVertically, "Flip Group Verticaclly", "Flips children of selected group along its relative X-Axis.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ToggleScaleChildrenWithParent, "Scale Children with Parent", "Sets if children are scaled with their parent, when the parent is resized.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::Q));
	UI_COMMAND(ToggleAlwaysSelectGroup, "Always select Group", "Sets if the parent is selected, if a child is clicked.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::W));
	UI_COMMAND(ToggleShowMatrixCells, "Show Matrix Cells", "Sets if matrix cells are displayed. Can be turned off for better editor performance when pixel mapping large quantities of fixtures.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::A));
	UI_COMMAND(ToggleShowComponentNames, "Show Component Names", "Sets if the name of components are displayed. Can be turned off for better editor performance when pixel mapping large quantities of fixtures.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::S));
	UI_COMMAND(ToggleShowPatchInfo, "Show Patch Info", "Sets if information about the pach is displayed. Can be turned off for better editor performance when pixel mapping large quantities of fixtures.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::D));
	UI_COMMAND(ToggleShowCellIDs, "Show Cell IDs", "Sets if the cell IDs of matrix cells are displayed. Can be turned off for better editor performance when pixel mapping large quantities of fixtures.", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Alt, EKeys::F));
	UI_COMMAND(ToggleShowPivot, "Show Pivot", "Sets if the pivot is displayed for selected components.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE 
