// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Framework/Commands/Commands.h"

/**
 * Defines commands for the pixel mapping editor.
 */
class FDMXPixelMappingEditorCommands 
	: public TCommands<FDMXPixelMappingEditorCommands>
{
public:
	FDMXPixelMappingEditorCommands();

	virtual void RegisterCommands() override;

	// Toolbar related
	TSharedPtr<FUICommandInfo> AddMapping;
	TSharedPtr<FUICommandInfo> PlayDMX;
	TSharedPtr<FUICommandInfo> PauseDMX;
	TSharedPtr<FUICommandInfo> ResumeDMX;
	TSharedPtr<FUICommandInfo> StopDMX;
	TSharedPtr<FUICommandInfo> TogglePlayPauseDMX;
	TSharedPtr<FUICommandInfo> TogglePlayStopDMX;

	TSharedPtr<FUICommandInfo> EditorStopSendsDefaultValues;
	TSharedPtr<FUICommandInfo> EditorStopSendsZeroValues;
	TSharedPtr<FUICommandInfo> EditorStopKeepsLastValues;

	// Designer related
	TSharedPtr<FUICommandInfo> EnableResizeMode;
	TSharedPtr<FUICommandInfo> EnableRotateMode;

	TSharedPtr<FUICommandInfo> ToggleGridSnapping;

	TSharedPtr<FUICommandInfo> FlipGroupHorizontally;
	TSharedPtr<FUICommandInfo> FlipGroupVertically;
	TSharedPtr<FUICommandInfo> SizeGroupToTexture;
	
	TSharedPtr<FUICommandInfo> ToggleScaleChildrenWithParent;
	TSharedPtr<FUICommandInfo> ToggleAlwaysSelectGroup;
	TSharedPtr<FUICommandInfo> ToggleShowMatrixCells;
	TSharedPtr<FUICommandInfo> ToggleShowComponentNames;
	TSharedPtr<FUICommandInfo> ToggleShowPatchInfo;
	TSharedPtr<FUICommandInfo> ToggleShowCellIDs;
	TSharedPtr<FUICommandInfo> ToggleShowPivot;
};
