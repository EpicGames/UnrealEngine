// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingEditorModule.h"

#include "AssetToolsModule.h"
#include "DMXPixelMappingEditorStyle.h"
#include "DMXPixelMappingEditorCommands.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingThumbnailRendering.h"

#include "Misc/CoreDelegates.h"
#include "ThumbnailRendering/ThumbnailManager.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingEditorModule"

EAssetTypeCategories::Type FDMXPixelMappingEditorModule::DMXPixelMappingCategory;
const FName FDMXPixelMappingEditorModule::DMXPixelMappingEditorAppIdentifier(TEXT("PixelMappingApp"));

void FDMXPixelMappingEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShared<FExtensibilityManager>();
	ToolBarExtensibilityManager = MakeShared<FExtensibilityManager>();

	// Register Asset Tools for pixel mapping
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	DMXPixelMappingCategory = AssetTools.RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));

	FDMXPixelMappingEditorCommands::Register();

	// Access the Pixel Mapping editor style so it gets instantiated and registered early.
	// This will allow the engine to find related class icon and thumbnail.
	FDMXPixelMappingEditorStyle::Get();

	UThumbnailManager::Get().RegisterCustomRenderer(UDMXPixelMapping::StaticClass(), UDMXPixelMappingThumbnailRendering::StaticClass());
}

void FDMXPixelMappingEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		for (TSharedPtr<IAssetTypeActions> Action : CreatedAssetTypeActions)
		{
			AssetTools.UnregisterAssetTypeActions(Action.ToSharedRef());
		}
	}

	if (UObjectInitialized() && GIsEditor)
	{
		UThumbnailManager::Get().UnregisterCustomRenderer(UDMXPixelMapping::StaticClass());
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

const FDMXPixelMappingEditorCommands& FDMXPixelMappingEditorModule::GetCommands() const
{
	return FDMXPixelMappingEditorCommands::Get();
}

void FDMXPixelMappingEditorModule::RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action)
{
	AssetTools.RegisterAssetTypeActions(Action);
	CreatedAssetTypeActions.Add(Action);
}

IMPLEMENT_MODULE(FDMXPixelMappingEditorModule, DMXPixelMappingEditor)

#undef LOCTEXT_NAMESPACE
