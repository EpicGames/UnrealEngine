// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "AssetTypeCategories.h"

class FExtensibilityManager;
class IAssetTools;
class IAssetTypeActions;
class FDMXPixelMappingEditorCommands;

class FDMXPixelMappingEditorModule
	: public IModuleInterface
	, public IHasMenuExtensibility			// Extender for adds or removes extenders for menu
	, public IHasToolBarExtensibility		// Extender for adds or removes extenders for toolbar
{
public:
	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface implementation

	//~ Begin IHasMenuExtensibility implementation
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	//~ End IHasMenuExtensibility implementation

	//~ Begin IHasToolBarExtensibility implementation
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }
	//~ End IHasToolBarExtensibility implementation

	/** Get Editor toolkit commands */
	const FDMXPixelMappingEditorCommands& GetCommands() const;

	/** Asset category */
	static EAssetTypeCategories::Type GetAssetCategory() { return DMXPixelMappingCategory; }

private:
	void RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action);

public:
	static const FName DMXPixelMappingEditorAppIdentifier;

private:
	static EAssetTypeCategories::Type DMXPixelMappingCategory;

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;

	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;

	TArray<TSharedPtr<IAssetTypeActions>> CreatedAssetTypeActions;
};
