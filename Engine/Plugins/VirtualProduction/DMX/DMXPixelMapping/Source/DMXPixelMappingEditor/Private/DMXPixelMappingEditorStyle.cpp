// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingEditorStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"


FDMXPixelMappingEditorStyle::FDMXPixelMappingEditorStyle()
	: FSlateStyleSet("DMXPixelMappingEditorStyle")
{
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("DMXPixelMapping"));
	if (Plugin.IsValid())
	{
		SetContentRoot(FPaths::Combine(Plugin->GetBaseDir(), TEXT("Content/Slate")));
	}
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	// Class styles
	{
		Set("ClassIcon.DMXPixelMapping", new IMAGE_BRUSH_SVG("DMXPixelMapping_16", Icon16x16));
		Set("ClassThumbnail.DMXPixelMapping", new IMAGE_BRUSH_SVG("DMXPixelMapping_64", Icon64x64));
	}

	// Icons
	{
		Set("Icons.AddSource", new CORE_IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12));
	}

	// Component border style
	{
		FSlateBrush* ComponentBorderBrush = new FSlateBrush();
		ComponentBorderBrush->Margin = FMargin(2.f);
		ComponentBorderBrush->DrawAs = ESlateBrushDrawType::Border;
		ComponentBorderBrush->TintColor = FLinearColor::White;
		Set("DMXPixelMappingEditor.ComponentBorder", ComponentBorderBrush);
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FDMXPixelMappingEditorStyle::~FDMXPixelMappingEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

const ISlateStyle& FDMXPixelMappingEditorStyle::Get()
{
	static const FDMXPixelMappingEditorStyle Inst;
	return Inst;
}
