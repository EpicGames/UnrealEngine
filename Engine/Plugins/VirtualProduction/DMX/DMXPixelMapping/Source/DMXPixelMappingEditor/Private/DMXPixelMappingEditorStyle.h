// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"


class FDMXPixelMappingEditorStyle
	: public FSlateStyleSet
{
public:
	FDMXPixelMappingEditorStyle();

	virtual ~FDMXPixelMappingEditorStyle();

	/** @return The Slate style set for pixel mapping editor widgets */
	static const ISlateStyle& Get();
};
