// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingEditorUtils.h"

#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorLog.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DragDrop/DMXPixelMappingDragDropOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "Toolkits/DMXPixelMappingToolkit.h"


#define LOCTEXT_NAMESPACE "FDMXPixelMappingEditorUtils"

UDMXPixelMappingRendererComponent* FDMXPixelMappingEditorUtils::AddRenderer(UDMXPixelMapping* InPixelMapping)
{
	if (InPixelMapping == nullptr)
	{
		UE_LOGF(LogDMXPixelMappingEditor, Warning, "Cannot find PixelMapping in AddRenderer call. PixelMapping is nullptr.");

		return nullptr;
	}	
	
	if (InPixelMapping->RootComponent == nullptr)
	{
		UE_LOGF(LogDMXPixelMappingEditor, Warning, "Cannot find RootComponent in AddRenderer call. RootComponent is nullptr.");

		return nullptr;
	}

	// Get root componet
	UDMXPixelMappingBaseComponent* RootComponent = InPixelMapping->RootComponent;

	// Create renderer name
	UDMXPixelMappingBaseComponent* DefaultComponent = UDMXPixelMappingRendererComponent::StaticClass()->GetDefaultObject<UDMXPixelMappingRendererComponent>();
	FName UniqueName = MakeUniqueObjectName(RootComponent, UDMXPixelMappingRendererComponent::StaticClass(), FName(TEXT("OutputMapping")));

	// Create new renderer and add to Root
	UDMXPixelMappingRendererComponent* Component = NewObject<UDMXPixelMappingRendererComponent>(RootComponent, UDMXPixelMappingRendererComponent::StaticClass(), UniqueName, RF_Transactional);
	RootComponent->AddChild(Component);

	return Component;
}

bool FDMXPixelMappingEditorUtils::GetArrangedWidget(TSharedRef<SWidget> InWidget, FArrangedWidget& OutArrangedWidget)
{
	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(InWidget);
	if (!WidgetWindow.IsValid())
	{
		return false;
	}

	TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

	FWidgetPath WidgetPath;
	if (FSlateApplication::Get().GeneratePathToWidgetUnchecked(InWidget, WidgetPath))
	{
		OutArrangedWidget = WidgetPath.FindArrangedWidget(InWidget).Get(FArrangedWidget::GetNullWidget());
		return true;
	}

	return false;
}

UDMXPixelMappingBaseComponent* FDMXPixelMappingEditorUtils::GetTargetComponentFromDragDropEvent(const TWeakPtr<FDMXPixelMappingToolkit>& WeakToolkit, const FDragDropEvent& DragDropEvent)
{
	if (TSharedPtr<FDMXPixelMappingToolkit> ToolkitPtr = WeakToolkit.Pin())
	{
		TSharedPtr<FDMXPixelMappingDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FDMXPixelMappingDragDropOp>();
		if (DragDropOp.IsValid())
		{
			UDMXPixelMappingBaseComponent* Target = (DragDropOp->Parent.IsValid()) ? DragDropOp->Parent.Get() : ToolkitPtr->GetActiveRendererComponent();

			return Target;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
