// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Templates/SharedPointer.h"

class FArrangedWidget;
class FDragDropEvent;
class FDMXPixelMappingToolkit;
class SWidget;
class UDMXPixelMapping;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingRendererComponent;

/**
 * Shared Pixel Mapping editor functions
 */
class FDMXPixelMappingEditorUtils
{
public:
	/**
	 * Add renderer to Pixel Mapping object
	 *
	 * @param InDMXPixelMapping		Pixel Mapping object
	 *
	 * @return Render Component pointer
	 */
	static UDMXPixelMappingRendererComponent* AddRenderer(UDMXPixelMapping* InPixelMapping);

	/**
	 * Returns the arranged widget of a widget.
	 *
	 * @param InWidget				The widget to get the arranged widget from
	 * @param OutArrangedWidget		The arranged widget
	 * 
	 * @return						True if the arranged widget could be acquired
	 */
	static bool GetArrangedWidget(TSharedRef<SWidget> InWidget, FArrangedWidget& OutArrangedWidget);

	/**
	 * Returns the target component from a drag drop event
	 *
	 * @param WeakToolkit			The toolkit in use
	 * @param DragDropEvent			The DragDropEvent to consider.
	 * 
	 * @return						The component that is target of the drag drop op
	 */
	static UDMXPixelMappingBaseComponent* GetTargetComponentFromDragDropEvent(const TWeakPtr<FDMXPixelMappingToolkit>& WeakToolkit, const FDragDropEvent& DragDropEvent);
};
