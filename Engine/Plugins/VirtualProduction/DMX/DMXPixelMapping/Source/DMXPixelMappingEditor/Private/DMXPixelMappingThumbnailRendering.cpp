// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingThumbnailRendering.h"

#include "CanvasTypes.h"
#include "DMXPixelMapping.h"
#include "Engine/Texture.h"
#include "TextureResource.h"

bool UDMXPixelMappingThumbnailRendering::CanVisualizeAsset(UObject* Object)
{
	return GetThumbnailTextureFromObject(Object) != nullptr;
}

void UDMXPixelMappingThumbnailRendering::GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const
{
	UTexture* ObjectTexture = GetThumbnailTextureFromObject(Object);
	if (ObjectTexture && ObjectTexture->GetResource())
	{
		OutWidth = Zoom * ObjectTexture->GetResource()->GetSizeX();
		OutHeight = Zoom * ObjectTexture->GetResource()->GetSizeY();
	}
	else
	{
		OutWidth = 0;
		OutHeight = 0;
	}
}

void UDMXPixelMappingThumbnailRendering::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (UTexture* Texture = GetThumbnailTextureFromObject(Object))
	{
		Texture->WaitForPendingInitOrStreaming();
		Canvas->DrawTile(X, Y, Width, Height, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White, Texture->GetResource(), false);
	}
}

UTexture* UDMXPixelMappingThumbnailRendering::GetThumbnailTextureFromObject(UObject* Object) const
{
	UDMXPixelMapping* DMXPixelMapping = Cast<UDMXPixelMapping>(Object);
	return DMXPixelMapping ? DMXPixelMapping->ThumbnailImage : nullptr;
}
