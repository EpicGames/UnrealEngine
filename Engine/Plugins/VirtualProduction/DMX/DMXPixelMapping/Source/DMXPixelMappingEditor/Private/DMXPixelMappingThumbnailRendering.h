// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ThumbnailRendering/TextureThumbnailRenderer.h"

#include "DMXPixelMappingThumbnailRendering.generated.h"

class UTexture;

/**
 * Dynamically  rendering thumbnail for Pixel Mapping asset
 */
UCLASS()
class UDMXPixelMappingThumbnailRendering :
	public UTextureThumbnailRenderer
{
	GENERATED_BODY()

public:
	//~ Begin UThumbnailRenderer interface.
	virtual bool CanVisualizeAsset(UObject* Object) override;
	virtual void GetThumbnailSize(UObject* Object, float Zoom, uint32& OutWidth, uint32& OutHeight) const override;
	virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	UTexture* GetThumbnailTextureFromObject(UObject* Object) const;
	//~ End UThumbnailRenderer interface.
};
