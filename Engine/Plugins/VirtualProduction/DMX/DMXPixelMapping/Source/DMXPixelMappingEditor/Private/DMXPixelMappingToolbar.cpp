// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingToolbar.h"

#include "DMXEditorStyle.h"
#include "DMXPixelMappingEditorCommands.h"
#include "DMXPixelMappingEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/ToolBarStyle.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/SNullWidget.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FDMXPixelMappingToolbar"

FDMXPixelMappingToolbar::FDMXPixelMappingToolbar(TSharedPtr<FDMXPixelMappingToolkit> InToolkit)
	: WeakToolkit(InToolkit)
{}

void FDMXPixelMappingToolbar::ExtendToolbar()
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	FName ParentName;
	const FName MenuName = Toolkit->GetToolMenuToolbarName(ParentName);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);
	const FToolMenuInsert SectionInsertLocation("Asset", EToolMenuInsertType::After);

	// Render section
	{
		FToolMenuSection& RenderSection = ToolMenu->AddSection("Render");

		FToolMenuEntry AddMappingMenuEntry = FToolMenuEntry::InitToolBarButton(
			FDMXPixelMappingEditorCommands::Get().AddMapping,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXPixelMappingEditorStyle::Get().GetStyleSetName(), "Icons.AddSource"),
			FName(TEXT("Add Source")));

		RenderSection.AddEntry(AddMappingMenuEntry);
	}

	// Play section
	{
		FToolMenuSection& PlaySection = ToolMenu->AddSection("Play");

		PlaySection.AddSeparator(NAME_None);

		// Play
		FToolMenuEntry PlayMenuEntry = FToolMenuEntry::InitToolBarButton(
			FDMXPixelMappingEditorCommands::Get().PlayDMX, 
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.PlayDMX"));
		
		PlayMenuEntry.StyleNameOverride = FName("Toolbar.BackplateLeftPlay");
		PlaySection.AddEntry(PlayMenuEntry);

		// Pause
		FToolMenuEntry PauseMenuEntry = FToolMenuEntry::InitToolBarButton(
			FDMXPixelMappingEditorCommands::Get().PauseDMX,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.PauseDMX"));

		PauseMenuEntry.StyleNameOverride = FName("Toolbar.BackplateLeft");
		PlaySection.AddEntry(PauseMenuEntry);

		// Resume
		FToolMenuEntry ResumeMenuEntry = FToolMenuEntry::InitToolBarButton(
			FDMXPixelMappingEditorCommands::Get().ResumeDMX,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.ResumeDMX"));

		ResumeMenuEntry.StyleNameOverride = FName("Toolbar.BackplateLeftPlay");
		PlaySection.AddEntry(ResumeMenuEntry);

		// Stop
		FToolMenuEntry StopMenuEntry = FToolMenuEntry::InitToolBarButton(
			FDMXPixelMappingEditorCommands::Get().StopDMX,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FDMXEditorStyle::Get().GetStyleSetName(), "Icons.StopDMX"));

		StopMenuEntry.StyleNameOverride = FName("Toolbar.BackplateCenterStop");
		PlaySection.AddEntry(StopMenuEntry);

		// Playback Settings
		FToolMenuEntry PlaybackSettingsComboEntry = FToolMenuEntry::InitComboButton(
			"PlaybackSettings",
			FToolUIActionChoice(),
			FOnGetContent::CreateSP(this, &FDMXPixelMappingToolbar::GeneratePlaybackSettingsMenu, MenuName),
			LOCTEXT("PlaybackSettingsLabel", "DMX Playback Settings"), 
			LOCTEXT("PlaybackSettingsToolTip", "Change DMX Playback Settings"));

		PlaybackSettingsComboEntry.StyleNameOverride = FName("Toolbar.BackplateRightCombo");
		PlaySection.AddEntry(PlaybackSettingsComboEntry);

		FToolMenuEntry BackPlateRightSeparaterEntry = PlaySection.AddSeparator(NAME_None);
		BackPlateRightSeparaterEntry.StyleNameOverride = FName("Toolbar.BackplateRight");
	}
}

TSharedRef<SWidget> FDMXPixelMappingToolbar::GeneratePlaybackSettingsMenu(FName ParentMenuName) const
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, Toolkit->GetToolkitCommands());

	MenuBuilder.BeginSection("ResetDMXModeSection");
	{
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().EditorStopSendsDefaultValues);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().EditorStopSendsZeroValues);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().EditorStopKeepsLastValues);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
