// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FDMXPixelMappingToolkit;
class SWidget;


/** Extends the PixelMapping asset editor toolkit toolbar */
class FDMXPixelMappingToolbar :
	public TSharedFromThis<FDMXPixelMappingToolbar>
{
public:
	FDMXPixelMappingToolbar(TSharedPtr<FDMXPixelMappingToolkit> InToolkit);
	virtual ~FDMXPixelMappingToolbar() {}

	/** Extends the PixelMapping asset editor toolkit toolbar  */
	void ExtendToolbar();

private:
	/** Generates the playback settings submenu */
	TSharedRef<SWidget> GeneratePlaybackSettingsMenu(FName ParentMenuName) const;

	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
