// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDrop/DMXPixelMappingDragDropOp.h"

#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMapping.h"
#include "DragDrop/DMXPixelMappingGroupChildDragDropHelper.h"
#include "Editor.h"
#include "Toolkits/DMXPixelMappingToolkit.h"


#define LOCTEXT_NAMESPACE "FDMXPixelMappingDragDropOp"

FDMXPixelMappingDragDropOp::~FDMXPixelMappingDragDropOp()
{
	// End the transaction in any case
	GEditor->EndTransaction();
}

TSharedRef<FDMXPixelMappingDragDropOp> FDMXPixelMappingDragDropOp::New(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, const FVector2D& InGraphSpaceDragOffset, const TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>>& InTemplates, UDMXPixelMappingBaseComponent* InParent)
{
	TSharedRef<FDMXPixelMappingDragDropOp> Operation = MakeShared<FDMXPixelMappingDragDropOp>();
	
	Operation->WeakToolkit = InToolkit;
	Operation->Templates = InTemplates;
	Operation->bWasCreatedAsTemplate = true;
	Operation->Parent = InParent;
	Operation->GraphSpaceDragOffset = InGraphSpaceDragOffset;
		
	Operation->Construct();
	Operation->SetDecoratorVisibility(false);

	// Create a transaction for dragged templates
	Operation->TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("DragDropTemplateTransaction", "PixelMapping: Add {0}|plural(one=Component, other=Components)"), InTemplates.Num()));

	return Operation;
}

TSharedRef<FDMXPixelMappingDragDropOp> FDMXPixelMappingDragDropOp::New(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, const FVector2D& InGraphSpaceDragOffset, const TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>>& InDraggedComponents)
{
	TSharedRef<FDMXPixelMappingDragDropOp> Operation = MakeShared<FDMXPixelMappingDragDropOp>();

	Operation->WeakToolkit = InToolkit;
	Operation->bWasCreatedAsTemplate = false;
	Operation->SetDraggedComponents(InDraggedComponents);
	Operation->GraphSpaceDragOffset = InGraphSpaceDragOffset;
	Operation->GroupChildDragDropHelper = FDMXPixelMappingGroupChildDragDropHelper::Create(Operation); // After setting dragged components

	Operation->Construct();
	Operation->SetDecoratorVisibility(false);

	// Create a transaction for dragged components
	Operation->TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("DragDropComponentTransaction", "PixelMapping: Drag {0}|plural(one=Component, other=Components)"), InDraggedComponents.Num()));

	return Operation;
}

void FDMXPixelMappingDragDropOp::SetDraggedComponents(const TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>>& InDraggedComponents)
{
	DraggedComponents = InDraggedComponents;
	Templates.Reset();

	// Rebuild the group child drag drop helper
	GroupChildDragDropHelper = FDMXPixelMappingGroupChildDragDropHelper::Create(AsShared());
}

void FDMXPixelMappingDragDropOp::LayoutOutputComponents(const FVector2D& GraphSpacePosition)
{
	UDMXPixelMappingOutputComponent* FirstComponent = DraggedComponents.IsEmpty() ? nullptr : Cast<UDMXPixelMappingOutputComponent>(DraggedComponents[0]);
	if (!FirstComponent)
	{
		return;
	}

	// Compute the translation for the first component
	const FVector2D OldPositionRotated = FirstComponent->GetPositionRotated();
	const FVector2D DesiredPositionRotated = FVector2D(GraphSpacePosition - GraphSpaceDragOffset);
	const FVector2D NewPositionRotated = ComputeGridSnapPosition(DesiredPositionRotated);

	// Move all to the new position
	const FVector2D Translation = NewPositionRotated - OldPositionRotated;
	for (const TWeakObjectPtr<UDMXPixelMappingBaseComponent>& Component : DraggedComponents)
	{
		if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component.Get()))
		{
			OutputComponent->PreEditChange(nullptr);

			if (ensureMsgf(OutputComponent->GetClass() != UDMXPixelMappingMatrixComponent::StaticClass(),
				TEXT("Matrix components cannot be laid out with FDMXPixelMappingDragDropOp::LayoutOutputComponents. Please use FGroupChildDragDropHelper instead (see FDMXPixelMappingDragDropOp::GetGroupChildDragDropHelper().")))
			{
				OutputComponent->Modify();
				OutputComponent->SetPosition(OutputComponent->GetPosition() + Translation);
			}

			OutputComponent->PostEditChange();
		}
	}
}

FVector2D FDMXPixelMappingDragDropOp::ComputeGridSnapPosition(const FVector2D& DesiredPosition) const
{
	const UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	if (!PixelMapping || DraggedComponents.IsEmpty() || PixelMapping->SnapGridColumns == 0 || PixelMapping->SnapGridRows == 0)
	{
		return DesiredPosition;
	}

	UDMXPixelMappingOutputComponent* FirstComponent = Cast<UDMXPixelMappingOutputComponent>(DraggedComponents[0].Get());
	if (!FirstComponent)
	{
		return DesiredPosition;
	}

	UDMXPixelMappingRendererComponent* RendererOfFirstComponent = FirstComponent->GetRendererComponent();
	if (!RendererOfFirstComponent)
	{
		return DesiredPosition;
	}

	// Grid snap to pixels even when grid snapping is disabled, if the first component is axis aligned
	const bool bIsAxisAligned = FMath::IsNearlyZero(FMath::Abs(FMath::Fmod(FirstComponent->GetRotation(), 90.f)));
	if (!PixelMapping->bGridSnappingEnabled)
	{
		return DesiredPosition;
	}

	const FVector2D CellSize = [PixelMapping, RendererOfFirstComponent]()
		{
			const FVector2D TextureSize = RendererOfFirstComponent->GetSize();
			return TextureSize / FVector2D(PixelMapping->SnapGridColumns, PixelMapping->SnapGridRows);
		}();

	const int32 Column = FMath::RoundHalfToZero(DesiredPosition.X / CellSize.X);
	const int32 Row = FMath::RoundHalfToZero(DesiredPosition.Y / CellSize.Y);

	const FVector2D GridSnapPosition = FVector2D(Column, Row) * CellSize;

	return GridSnapPosition;
}

#undef LOCTEXT_NAMESPACE
