// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"

class FDMXPixelMappingComponentTemplate;
class FDMXPixelMappingGroupChildDragDropHelper;
class FDMXPixelMappingToolkit;
class UDMXPixelMappingBaseComponent;
struct FDMXPixelMappingComponentReference;

/**
 * This drag drop operation allows Component templates from the palate to be dragged and dropped into the designer
 * or the Component hierarchy in order to spawn new Components.
 */
class FDMXPixelMappingDragDropOp 
	: public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FDMXPixelMappingDragDropOp, FDecoratedDragDropOp)

	/** Destructor */
	virtual ~FDMXPixelMappingDragDropOp();

	/** Constructs the drag drop operation. */
	static TSharedRef<FDMXPixelMappingDragDropOp> New(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, const FVector2D& InGraphSpaceDragOffset, const TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>>& InTemplates, UDMXPixelMappingBaseComponent* InParent);
	
	/** Constructs the drag drop operation. Note, the component that initiated the drag drop op has to be the first element in the InDraggedComponents array. */
	static TSharedRef<FDMXPixelMappingDragDropOp> New(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, const FVector2D& InGraphSpaceDragOffset, const TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>>& InDraggedComponents);

	/** Sets dragged components. Clears the template, preventing from the template being used more than once. */
	void SetDraggedComponents(const TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>>& InDraggedComponents);

	/** 
	 * Lays out the draged output components at given locations. Requires the first in the selection to have specified the drag offset 
	 * Note, doesn't work with matrix components, use GroupChildHelper from this class instead for that.
	 */
	void LayoutOutputComponents(const FVector2D& GraphSpacePosition);

	/** Returns true if the drag drop op was created with the constructor that provides a template */
	bool WasCreatedAsTemplate() const { return bWasCreatedAsTemplate; }


	/** Can be used to store the parent of the dragged component(s) */
	TWeakObjectPtr<UDMXPixelMappingBaseComponent> Parent;

	/** The Drag offset, in graph space */
	FVector2D GraphSpaceDragOffset;

	/** Returns the Group Item dragdrop helper. Only valid if only group items are dragged */
	const TSharedPtr<FDMXPixelMappingGroupChildDragDropHelper>& GetGroupChildDragDropHelper() const { return GroupChildDragDropHelper; }

	/** Returns the dragged component references */
	const TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>>& GetDraggedComponents() const { return DraggedComponents; }

	/** Returns the dragged component references */
	const TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>>& GetTemplates() const { return Templates; }
	
	/** Utiltity that returns the grid snapped position from a desired position. */
	FVector2D ComputeGridSnapPosition(const FVector2D& DesiredPosition) const;

private:
	/** True if this was created with the New version was used that provides a template */
	bool bWasCreatedAsTemplate = false;

	/** Transaction index of the drag drop op */
	int32 TransactionIndex = INDEX_NONE;

	/** Components dragged with this opperation */
	TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>> DraggedComponents;

	/** The templates to create component instances */
	TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>> Templates;

	/** Group Item drag drop helper */
	TSharedPtr<FDMXPixelMappingGroupChildDragDropHelper> GroupChildDragDropHelper;

	/** The toolkit that constructed this drag drop op */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
