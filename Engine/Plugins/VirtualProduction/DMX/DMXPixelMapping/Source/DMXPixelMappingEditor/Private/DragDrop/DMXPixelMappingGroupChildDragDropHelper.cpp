// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragDrop/DMXPixelMappingGroupChildDragDropHelper.h"

#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "DragDrop/DMXPixelMappingDragDropOp.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"



TSharedPtr<FDMXPixelMappingGroupChildDragDropHelper> FDMXPixelMappingGroupChildDragDropHelper::Create(TSharedRef<FDragDropOperation> InPixelMappingDragDropOp)
{
	TSharedRef<FDMXPixelMappingGroupChildDragDropHelper> NewHelper = MakeShared<FDMXPixelMappingGroupChildDragDropHelper>();

	TSharedRef<FDMXPixelMappingDragDropOp> DragDropOp = StaticCastSharedRef<FDMXPixelMappingDragDropOp>(InPixelMappingDragDropOp);
	NewHelper->WeakPixelMappingDragDropOp = DragDropOp;

	// No dragged items
	if (DragDropOp->GetDraggedComponents().Num() == 0)
	{
		return nullptr;
	}

	// No parent group
	UDMXPixelMappingFixtureGroupComponent* ParentGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(DragDropOp->GetDraggedComponents()[0]->GetParent());
	if (!ParentGroupComponent)
	{
		return nullptr;
	}
	NewHelper->WeakParentGroupComponent = ParentGroupComponent;
	NewHelper->ParentPosition = ParentGroupComponent->GetPositionRotated();
	NewHelper->ParentSize = ParentGroupComponent->GetSize();

	for (const TWeakObjectPtr<UDMXPixelMappingBaseComponent>& BaseComponent : DragDropOp->GetDraggedComponents())
	{	
		if (UDMXPixelMappingMatrixCellComponent* MatrixCellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(BaseComponent.Get()))
		{		
			// For matrices, don't drag the childs along, they adopt their position from the parent
			continue;
		}

		UDMXPixelMappingOutputComponent* OutputComponent = [BaseComponent]() -> UDMXPixelMappingOutputComponent*
		{
			// Either a matrices or group items
			if ((BaseComponent.IsValid() && BaseComponent->GetClass() == UDMXPixelMappingFixtureGroupItemComponent::StaticClass()) ||
				(BaseComponent.IsValid() && BaseComponent->GetClass() == UDMXPixelMappingMatrixComponent::StaticClass()))
			{
				return Cast<UDMXPixelMappingOutputComponent>(BaseComponent);
			}

			return nullptr;
		}();


		// Invalid component or different parents
		if (!OutputComponent || OutputComponent->GetParent() != ParentGroupComponent)
		{
			return nullptr;
		}

		NewHelper->WeakChildComponents.Add(OutputComponent);
	}

	return NewHelper;
}

void FDMXPixelMappingGroupChildDragDropHelper::LayoutComponents(const FVector2D& Position, bool bAlignCells)
{
	if (bAlignCells)
	{
		LayoutAligned(Position);
	}
	else
	{
		LayoutUnaligned(Position);
	}
}

void FDMXPixelMappingGroupChildDragDropHelper::LayoutAligned(const FVector2D& GraphSpacePosition)
{
	TSharedPtr<FDMXPixelMappingDragDropOp> DragDropOp = WeakPixelMappingDragDropOp.Pin();
	UDMXPixelMappingFixtureGroupComponent* GroupComponent = WeakParentGroupComponent.Get();
	if (!DragDropOp.IsValid() || !GroupComponent)
	{
		return;
	}
	 
	FVector2D NextPosition = GraphSpacePosition - DragDropOp->GraphSpaceDragOffset;
	float RowHeight = 0.f;

	for (const TWeakObjectPtr<UDMXPixelMappingOutputComponent>& WeakChildComponent : WeakChildComponents)
	{
		if (UDMXPixelMappingOutputComponent* ChildComponent = WeakChildComponent.Get())
		{
			ChildComponent->PreEditChange(nullptr);

			constexpr bool bModifyChildrenRecursive = true;
			ChildComponent->ForEachChild([](UDMXPixelMappingBaseComponent* Component)
				{
					Component->Modify();
				}, bModifyChildrenRecursive);

			if (GroupComponent->IsOverPosition(NextPosition) && 
				GroupComponent->IsOverPosition(NextPosition + ChildComponent->GetSize()))
			{
				ChildComponent->SetPositionRotated(NextPosition);

				RowHeight = FMath::Max(ChildComponent->GetSize().Y, RowHeight);
				NextPosition = FVector2D(NextPosition.X + ChildComponent->GetSize().X, NextPosition.Y);
			}
			else
			{
				// Try on a new row
				FVector2D NewRowPosition = FVector2D(GraphSpacePosition.X, NextPosition.Y + RowHeight);

				const FVector2D NextPositionOnNewRow = FVector2D(NewRowPosition.X + ChildComponent->GetSize().X, NewRowPosition.Y);
				if (GroupComponent->IsOverPosition(NextPositionOnNewRow) &&
					GroupComponent->IsOverPosition(NextPositionOnNewRow + ChildComponent->GetSize()))
				{
					ChildComponent->SetPositionRotated(NewRowPosition);

					NextPosition = FVector2D(NewRowPosition.X + ChildComponent->GetSize().X, NewRowPosition.Y);
					RowHeight = ChildComponent->GetSize().Y;							
				}
				else
				{
					ChildComponent->SetPositionRotated(NextPosition);

					NextPosition = FVector2D(NextPosition.X + ChildComponent->GetSize().X, NextPosition.Y);
				}
			}
		}
	}
	const FVector2D GridSnapPosition = DragDropOp->ComputeGridSnapPosition(GroupComponent->GetPositionRotated());
	GroupComponent->SetPositionRotated(GridSnapPosition);

	for (const TWeakObjectPtr<UDMXPixelMappingOutputComponent>& WeakChildComponent : WeakChildComponents)
	{
		if (UDMXPixelMappingOutputComponent* ChildComponent = WeakChildComponent.Get())
		{
			ChildComponent->PostEditChange();
		}
	}
}

void FDMXPixelMappingGroupChildDragDropHelper::LayoutUnaligned(const FVector2D& GraphSpacePosition)
{
	TSharedPtr<FDMXPixelMappingDragDropOp> DragDropOp = WeakPixelMappingDragDropOp.Pin();
	if (!DragDropOp.IsValid() || WeakChildComponents.IsEmpty())
	{
		return;
	}

	UDMXPixelMappingOutputComponent* FirstComponent = WeakChildComponents[0].Get();
	if (!FirstComponent)
	{
		return;
	}

	// Find where the first component grid snaps
	const FVector2D Anchor = FirstComponent->GetPositionRotated();
	const FVector2D AnchorOffset = Anchor - FirstComponent->GetPositionRotated();

	const FVector2D DesiredPosition = GraphSpacePosition - AnchorOffset - DragDropOp->GraphSpaceDragOffset;
	const FVector2D GridSnapPosition = DragDropOp->ComputeGridSnapPosition(DesiredPosition);

	// Translate all
	const FVector2D Translation = GridSnapPosition - Anchor;
	for (const TWeakObjectPtr<UDMXPixelMappingOutputComponent>& WeakOutputComponent : WeakChildComponents)
	{
		if (UDMXPixelMappingOutputComponent* ChildComponent = WeakOutputComponent.Get())
		{
			ChildComponent->Modify();

			ChildComponent->SetPosition(ChildComponent->GetPosition() + Translation);
		}
	}
}
