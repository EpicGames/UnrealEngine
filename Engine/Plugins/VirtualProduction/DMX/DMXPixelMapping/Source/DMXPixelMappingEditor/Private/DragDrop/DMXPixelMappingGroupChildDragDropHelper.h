// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Math/Vector2D.h"
#include "UObject/WeakObjectPtr.h"

class UDMXPixelMappingFixtureGroupComponent;
class UDMXPixelMappingOutputComponent;

class FDMXPixelMappingDragDropOp;

class FDragDropOperation;


/** Helper class that manages to layout group items while being dragged. Can return what is over the parent */
class FDMXPixelMappingGroupChildDragDropHelper
	: public TSharedFromThis<FDMXPixelMappingGroupChildDragDropHelper>
{
public:
	/** Creates a new helper, returns the helper or nullptr if it cannot be created from the drag drop op */
	static TSharedPtr<FDMXPixelMappingGroupChildDragDropHelper> Create(TSharedRef<FDragDropOperation> InPixelMappingDragDropOp);

	/** Lays out the widgets */
	void LayoutComponents(const FVector2D& GraphSpacePosition, bool bAlignCells);

	/** Returns all group items */
	TArray<TWeakObjectPtr<UDMXPixelMappingOutputComponent>> GetChildComponents() const { return WeakChildComponents; }

private:
	/** Helper that lays out cells aligned */
	void LayoutAligned(const FVector2D& GraphSpacePosition);

	/** Helper that lays out cells unaligned */
	void LayoutUnaligned(const FVector2D& GraphSpacePosition);

	/** Local position of the parent */
	FVector2D ParentPosition;

	/** Size of the parent */
	FVector2D ParentSize;

	/** The parent group component */
	TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent> WeakParentGroupComponent;

	/** The child group item components */
	TArray<TWeakObjectPtr<UDMXPixelMappingOutputComponent>> WeakChildComponents;

	/** The drag drop op being handled */
	TWeakPtr<FDMXPixelMappingDragDropOp> WeakPixelMappingDragDropOp;
};
