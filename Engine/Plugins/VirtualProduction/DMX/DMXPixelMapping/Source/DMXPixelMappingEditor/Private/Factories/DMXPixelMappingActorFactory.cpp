// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXPixelMappingActorFactory.h"

#include "DMXPixelMapping.h"
#include "DMXPixelMappingActor.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Level.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMesh.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingActorFactory"

UDMXPixelMappingActorFactory::UDMXPixelMappingActorFactory()
{
	DisplayName = LOCTEXT("DMXPixelMappingActorFactoryDisplayName", "Pixel Mapping Actor");
	NewActorClass = ADMXPixelMappingActor::StaticClass();
}

bool UDMXPixelMappingActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (AssetData.IsValid())
	{
		const UClass* AssetClass = AssetData.GetClass();
		if (AssetClass && AssetClass == UDMXPixelMapping::StaticClass())
		{
			return true;
		}
	}

	return false;
}

AActor* UDMXPixelMappingActorFactory::SpawnActor(UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams)
{
	UDMXPixelMapping* PixelMapping = Cast<UDMXPixelMapping>(Asset);
	if (!PixelMapping)
	{
		return nullptr;
	}

	if (InSpawnParams.ObjectFlags & RF_Transient)
	{
		// This is a hack for drag and drop so that we don't spawn all the actors for the preview actor since it gets deleted right after.
		FActorSpawnParameters SpawnInfo(InSpawnParams);
		SpawnInfo.OverrideLevel = InLevel;
		AStaticMeshActor* DragActor = Cast<AStaticMeshActor>(InLevel->GetWorld()->SpawnActor(AStaticMeshActor::StaticClass(), &Transform, SpawnInfo));
		DragActor->GetStaticMeshComponent()->SetStaticMesh(Cast< UStaticMesh >(FSoftObjectPath(TEXT("StaticMesh'/Engine/EditorMeshes/EditorSphere.EditorSphere'")).TryLoad()));
		DragActor->SetActorScale3D(FVector(0.1f));

		return DragActor;
	}
	else 
	{
		if (!InLevel)
		{
			return nullptr;
		}

		UWorld* World = InLevel->GetWorld();
		if (!World)
		{
			return nullptr;
		}

		ADMXPixelMappingActor* PixelMappingActor = World->SpawnActor<ADMXPixelMappingActor>(ADMXPixelMappingActor::StaticClass());
		if (PixelMappingActor)
		{
			PixelMappingActor->SetPixelMapping(PixelMapping);

			return PixelMappingActor;
		}
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
