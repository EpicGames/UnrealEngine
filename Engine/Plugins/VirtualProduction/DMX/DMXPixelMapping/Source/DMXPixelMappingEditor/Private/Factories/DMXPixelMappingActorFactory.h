// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"
#include "UObject/ObjectMacros.h"

#include "DMXPixelMappingActorFactory.generated.h"

class AActor;
struct FAssetData;


/** Actor Factory for DMX Pixel Mapping Actor */
UCLASS()
class UDMXPixelMappingActorFactory
	: public UActorFactory
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXPixelMappingActorFactory();

	//~ Begin UActorFactory Interface
	virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	virtual AActor* SpawnActor(UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams) override;
	//~ End UActorFactory Interface
};
