// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/DMXPixelMappingFactoryNew.h"

#include "AssetDefinitionRegistry.h"
#include "DMXPixelMappingEditorModule.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorUtils.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingFactoryNew"

UDMXPixelMappingFactoryNew::UDMXPixelMappingFactoryNew()
{
	//~ Initialize parent class properties

	// This factory is responsible for manufacturing DMXPixelMapping assets.
	SupportedClass = UDMXPixelMapping::StaticClass();

	// This factory manufacture new objects from scratch.
	bCreateNew = true;

	// This factory will open the editor for each new object.
	bEditAfterNew = true;
}

UObject* UDMXPixelMappingFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	check(Class->IsChildOf(UDMXPixelMapping::StaticClass()));
	ensure(0 != (RF_Public & Flags));

	UDMXPixelMapping* DMXPixelMapping = NewObject<UDMXPixelMapping>(InParent, Class, Name, Flags);

	// Create all essentials UObjects
	DMXPixelMapping->CreateOrLoadObjects();

	// Add at least one renderer for a new Asset
	FDMXPixelMappingEditorUtils::AddRenderer(DMXPixelMapping);

	return DMXPixelMapping;
}

FText UDMXPixelMappingFactoryNew::GetDisplayName() const
{
	// Get the display name for this factory from the asset definition
	if (UClass* LocalSupportedClass = GetSupportedClass())
	{
		if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(LocalSupportedClass))
		{
			return AssetDefinition->GetAssetDisplayName();
		}
	}

	// Factories that have no supported class have no display name.
	return FText();
}

uint32 UDMXPixelMappingFactoryNew::GetMenuCategories() const
{
	return uint32(FDMXPixelMappingEditorModule::GetAssetCategory());
}

#undef LOCTEXT_NAMESPACE
