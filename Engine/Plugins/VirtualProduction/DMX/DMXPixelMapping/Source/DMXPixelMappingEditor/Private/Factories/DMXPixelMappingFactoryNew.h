// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "DMXPixelMappingFactoryNew.generated.h"

UCLASS(MinimalAPI)
class UDMXPixelMappingFactoryNew
	: public UFactory
{
	GENERATED_BODY()

public:
	UDMXPixelMappingFactoryNew();

private:
	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual FText GetDisplayName() const override;
	virtual uint32 GetMenuCategories() const override;
	//~ End UFactory Interface
};
