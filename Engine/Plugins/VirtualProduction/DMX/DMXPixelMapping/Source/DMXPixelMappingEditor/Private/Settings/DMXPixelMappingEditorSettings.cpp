// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingEditorSettings.h"


FSimpleMulticastDelegate UDMXPixelMappingEditorSettings::OnEditorSettingsChanged;
