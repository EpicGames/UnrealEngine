// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingEditorSettings.generated.h"

enum class EDMXPixelMappingResetDMXMode : uint8;
struct FPropertyChangedEvent;


USTRUCT()
struct FDMXPixelMappingHierarchySettings
{
	GENERATED_BODY()

	/** If true, shows the editor color column */
	UPROPERTY()
	bool bShowEditorColorColumn = true;

	/** If true, shows the fixture ID column */
	UPROPERTY()
	bool bShowFixtureIDColumn = true;

	/** If true, shows the patch column */
	UPROPERTY()
	bool bShowPatchColumn = true;

	/** The sort by column Id. Can be none or a valid column id */
	UPROPERTY()
	FName SortByColumnId;

	/** If true, the current sort mode is ascending */
	UPROPERTY()
	bool bSortAscending = true;
};

USTRUCT()
struct FDMXPixelMappingDesignerSettings
{
	GENERATED_BODY()

	/** If true, scales children when the parent component is resized */
	UPROPERTY()
	bool bScaleChildrenWithParent = true;

	/** If true, selects parent when a child is clicked */
	UPROPERTY()
	bool bAlwaysSelectGroup = false;

	/** If true, applies layout scripts instantly when they are loaded */
	UPROPERTY()
	bool bApplyLayoutScriptWhenLoaded = true;

	/** If true, a pivot is displayed for selected components */
	UPROPERTY()
	bool bShowPivot = true;

	/**  If true, shows a widget for each cell. It is recommended that this is turned off when pixel mapping large quantities of fixtures. */
	UPROPERTY()
	bool bShowMatrixCells = true;

	/** If true, shows the name of the Fixture Patch where applicable. It is recommended that this is turned off when pixel mapping large quantities of fixtures. */
	UPROPERTY()
	bool bShowComponentNames = true;

	/** If true, shows Fixture info bout the Fixture Patch where applicable. It is recommended that this is turned off when pixel mapping large quantities of fixtures. */
	UPROPERTY()
	bool bShowPatchInfo = false;

	/** If true, shows Cell IDs where applicable. It is recommended that this is turned off when pixel mapping large quantities of fixtures. */
	UPROPERTY()
	bool bShowCellIDs = false;
};


/** Layout options for the Pixel Mapping editor */
UCLASS(Config = DMXPixelMappingEditor, DefaultConfig)
class UDMXPixelMappingEditorSettings
	: public UObject
{
	GENERATED_BODY()

public:
	/** Settings for the hierarchy tab */
	UPROPERTY(Config)
	FDMXPixelMappingHierarchySettings HierarchySettings;
	
	/** Settings for the designer tab */
	UPROPERTY(Config)
	FDMXPixelMappingDesignerSettings DesignerSettings;

	/** Reset DMX mode to be used in editor */
	UE_DEPRECATED(5.6, "Each pixel mapping holds its own stop mode now. See UDMXPixelMapping::SetResetDMXMode")
	UPROPERTY(Config)
	EDMXPixelMappingResetDMXMode EditorResetDMXMode;

	/** Raised by the pixel mapping toolkit when settings changed */
	static FSimpleMulticastDelegate OnEditorSettingsChanged;
};
