// Copyright Epic Games, Inc. All Rights Reserved.

#include "Toolkits/DMXPixelMappingToolkit.h"

#include "Algo/Sort.h"
#include "Algo/Transform.h"
#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorCommands.h"
#include "DMXPixelMappingEditorModule.h"
#include "DMXPixelMappingEditorStyle.h"
#include "DMXPixelMappingEditorUtils.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"
#include "DMXPixelMappingToolbar.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HitProxies.h"
#include "K2Node_PixelMappingBaseComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Templates/DMXPixelMappingComponentTemplate.h"
#include "TextureResource.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"
#include "Views/SDMXPixelMappingDesignerView.h"
#include "Views/SDMXPixelMappingDetailsView.h"
#include "Views/SDMXPixelMappingDMXLibraryView.h"
#include "Views/SDMXPixelMappingHierarchyView.h"
#include "Views/SDMXPixelMappingLayoutView.h"
#include "Views/SDMXPixelMappingPreviewView.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingToolkit"

const FName FDMXPixelMappingToolkit::DMXLibraryViewTabID(TEXT("DMXPixelMappingEditor_DMXLibraryViewTabID"));
const FName FDMXPixelMappingToolkit::HierarchyViewTabID(TEXT("DMXPixelMappingEditor_HierarchyViewTabID"));
const FName FDMXPixelMappingToolkit::DesignerViewTabID(TEXT("DMXPixelMappingEditor_DesignerViewTabID"));
const FName FDMXPixelMappingToolkit::PreviewViewTabID(TEXT("DMXPixelMappingEditor_PreviewViewTabID"));
const FName FDMXPixelMappingToolkit::DetailsViewTabID(TEXT("DMXPixelMappingEditor_DetailsViewTabID"));
const FName FDMXPixelMappingToolkit::LayoutViewTabID(TEXT("DMXPixelMappingEditor_LayoutViewTabID"));

FDMXPixelMappingToolkit::FDMXPixelMappingToolkit()
	: AnalyticsProvider("PixelMappingEditor")
{
	EditorSettingsDump = TArray<uint8, TFixedAllocator<sizeof(UDMXPixelMappingEditorSettings)>>(reinterpret_cast<const uint8*>(GetDefault<UDMXPixelMappingEditorSettings>()), (int32)sizeof(UDMXPixelMappingEditorSettings));

	Selection = NewObject<UDMXPixelMappingToolkitSelection>(GetTransientPackage(), NAME_None, RF_Transactional);
}

FDMXPixelMappingToolkit::~FDMXPixelMappingToolkit()
{		
	// Explicitly stop playing DMX so the stop mode (send default or zero values) is correctly carried out.
	if (IsPlayingDMX())
	{
		StopPlayingDMX();
	}
}

void FDMXPixelMappingToolkit::InitPixelMappingEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UDMXPixelMapping* InDMXPixelMapping)
{
	check(InDMXPixelMapping);

	// Upgrade to use a per pixel mapping DMX reset mode
	if (InDMXPixelMapping->GetLinkerCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::PerPixelMappingResetDMXMode)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InDMXPixelMapping->SetResetDMXMode(GetDefault<UDMXPixelMappingEditorSettings>()->EditorResetDMXMode);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	InDMXPixelMapping->DestroyInvalidComponents();

	// Make sure we loaded all UObjects
	InDMXPixelMapping->CreateOrLoadObjects();

	// Bind to component changes
	UDMXPixelMappingBaseComponent::GetOnComponentAdded().AddSP(this, &FDMXPixelMappingToolkit::OnComponentAddedOrRemoved);
	UDMXPixelMappingBaseComponent::GetOnComponentRemoved().AddSP(this, &FDMXPixelMappingToolkit::OnComponentAddedOrRemoved);
	UDMXPixelMappingBaseComponent::GetOnComponentRenamed().AddSP(this, &FDMXPixelMappingToolkit::OnComponentRenamed);

	SetupCommands();

	CreateInternalViews();

	TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_PixelMapping_Layout_2.0")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(.25f)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(DMXLibraryViewTabID, ETabState::OpenedTab)
						->SetSizeCoefficient(.382f)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(HierarchyViewTabID, ETabState::OpenedTab)
						->SetSizeCoefficient(.618f)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(.5f)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(DesignerViewTabID, ETabState::OpenedTab)
						->SetSizeCoefficient(.75f)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(PreviewViewTabID, ETabState::OpenedTab)
						->SetSizeCoefficient(.25f)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(.25f)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(DetailsViewTabID, ETabState::OpenedTab)
						->SetSizeCoefficient(.618f)
					)
					->Split
					(
						FTabManager::NewStack()
						->AddTab(LayoutViewTabID, ETabState::OpenedTab)
						->SetSizeCoefficient(.382f)
					)
				)
			)
		);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FDMXPixelMappingEditorModule::DMXPixelMappingEditorAppIdentifier,
		StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InDMXPixelMapping);

	// Allow extenders to extend the toolbar, then regenerate menus and toolbars.
	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// Make an initial selection
	if (UDMXPixelMappingRootComponent* RootComponent = InDMXPixelMapping->GetRootComponent())
	{
		UDMXPixelMappingBaseComponent* const* FirstRendererComponentPtr = Algo::FindByPredicate(InDMXPixelMapping->GetRootComponent()->GetChildren(), [](UDMXPixelMappingBaseComponent* Component)
			{
				return Component && Component->GetClass() == UDMXPixelMappingRendererComponent::StaticClass();
			});
		if (FirstRendererComponentPtr)
		{
			UDMXPixelMappingBaseComponent* const* FirstFixtureGroupComponentPtr = Algo::FindByPredicate((*FirstRendererComponentPtr)->GetChildren(), [](UDMXPixelMappingBaseComponent* Component)
				{
					return Component && Component->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass();
				});

			if (UDMXPixelMappingBaseComponent* const* ComponentToSelectPtr = FirstFixtureGroupComponentPtr ?
				FirstFixtureGroupComponentPtr :
				FirstRendererComponentPtr)
			{
				const FDMXPixelMappingComponentReference ComponentReference(StaticCastSharedRef<FDMXPixelMappingToolkit>(AsShared()), *ComponentToSelectPtr);
				SelectComponents(TSet<FDMXPixelMappingComponentReference>({ ComponentReference }));
			}
		}
	}

	// Refresh the hierarchy view, so that it displays the pixelmapping of the now-initialized asset editor.
	HierarchyView->RequestRefresh();

	// Set the scale children with parent property on the pixel mapping object, so it is accessible the runtime module.
	const UDMXPixelMappingEditorSettings* EditorSettings = GetDefault<UDMXPixelMappingEditorSettings>();
	InDMXPixelMapping->bEditorScaleChildrenWithParent = EditorSettings->DesignerSettings.bScaleChildrenWithParent;

	// Listen to packages being saved
	UPackage::PreSavePackageWithContextEvent.AddSP(this, &FDMXPixelMappingToolkit::PreSavePackage);
}

void FDMXPixelMappingToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_TextureEditor", "DMX Pixel Mapping Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(DMXLibraryViewTabID, FOnSpawnTab::CreateSP(this, &FDMXPixelMappingToolkit::SpawnTab_DMXLibraryView))
		.SetDisplayName(LOCTEXT("Tab_DMXLibraryView", "DMX Library"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FDMXPixelMappingEditorStyle::Get().GetStyleSetName(), "ClassIcon.DMXPixelMapping"));

	InTabManager->RegisterTabSpawner(HierarchyViewTabID, FOnSpawnTab::CreateSP(this, &FDMXPixelMappingToolkit::SpawnTab_HierarchyView))
		.SetDisplayName(LOCTEXT("Tab_HierarchyView", "Hierarchy"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Kismet.Tabs.Components"));

	InTabManager->RegisterTabSpawner(DesignerViewTabID, FOnSpawnTab::CreateSP(this, &FDMXPixelMappingToolkit::SpawnTab_DesignerView))
		.SetDisplayName(LOCTEXT("Tab_DesignerView", "Designer"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"));

	InTabManager->RegisterTabSpawner(PreviewViewTabID, FOnSpawnTab::CreateSP(this, &FDMXPixelMappingToolkit::SpawnTab_PreviewView))
		.SetDisplayName(LOCTEXT("Tab_PreviewView", "Preview"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FDMXPixelMappingEditorStyle::Get().GetStyleSetName(), "Icons.Preview"));

	InTabManager->RegisterTabSpawner(DetailsViewTabID, FOnSpawnTab::CreateSP(this, &FDMXPixelMappingToolkit::SpawnTab_DetailsView))
		.SetDisplayName(LOCTEXT("Tab_DetailsView", "Details"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Details"));

	InTabManager->RegisterTabSpawner(LayoutViewTabID, FOnSpawnTab::CreateSP(this, &FDMXPixelMappingToolkit::SpawnTab_LayoutView))
		.SetDisplayName(LOCTEXT("Tab_LayoutView", "Layout"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Layout"));
}

void FDMXPixelMappingToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(HierarchyViewTabID);
	InTabManager->UnregisterTabSpawner(DesignerViewTabID);
	InTabManager->UnregisterTabSpawner(PreviewViewTabID);
	InTabManager->UnregisterTabSpawner(DetailsViewTabID);
	InTabManager->UnregisterTabSpawner(LayoutViewTabID);
}

FText FDMXPixelMappingToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "DMX Pixel Mapping");
}

FName FDMXPixelMappingToolkit::GetToolkitFName() const
{
	return FName("DMX Pixel Mapping");
}

FString FDMXPixelMappingToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "DMX Pixel Mapping ").ToString();
}

void FDMXPixelMappingToolkit::Tick(float DeltaTime)
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (!PixelMapping)
	{
		return;
	}

	UDMXPixelMappingRootComponent* RootComponent = PixelMapping->RootComponent;
	if (!ensure(RootComponent))
	{
		return;
	}

	// Render, send DMX if required
	RootComponent->Render();

	// Detect and broadcast editor setting changes
	if (FMemory::Memcmp(EditorSettingsDump.GetData(), GetDefault<UDMXPixelMappingEditorSettings>(), sizeof(UDMXPixelMappingEditorSettings)) != 0)
	{
		UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>();
		EditorSettings->OnEditorSettingsChanged.Broadcast();

		EditorSettingsDump = TArray<uint8, TFixedAllocator<sizeof(UDMXPixelMappingEditorSettings)>>(reinterpret_cast<const uint8*>(GetDefault<UDMXPixelMappingEditorSettings>()), (int32)sizeof(UDMXPixelMappingEditorSettings));
	
		// Set the scale children with parent property on the pixel mapping object, so it is accessible the runtime module.
		PixelMapping->bEditorScaleChildrenWithParent = EditorSettings->DesignerSettings.bScaleChildrenWithParent;
	}
}

TStatId FDMXPixelMappingToolkit::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FDMXPixelMappingToolkit, STATGROUP_Tickables);
}

UDMXPixelMapping* FDMXPixelMappingToolkit::GetDMXPixelMapping() const
{
	return HasEditingObject() ? Cast<UDMXPixelMapping>(GetEditingObject()) : nullptr;
}

FDMXPixelMappingComponentReference FDMXPixelMappingToolkit::GetReferenceFromComponent(UDMXPixelMappingBaseComponent* InComponent)
{
	return FDMXPixelMappingComponentReference(SharedThis(this), InComponent);
}

UDMXPixelMappingRendererComponent* FDMXPixelMappingToolkit::GetActiveRendererComponent() const
{
	checkf(Selection, TEXT("Unexpected invalid selection object in pixel mapping toolkit."));

	return Selection->ActiveRendererComponent.Get();
}

void FDMXPixelMappingToolkit::SetActiveRenderComponent(UDMXPixelMappingRendererComponent* InComponent)
{
	checkf(Selection, TEXT("Unexpected invalid selection object in pixel mapping toolkit."));

	Selection->Modify();
	Selection->ActiveRendererComponent = InComponent;
}

template <typename ComponentType>
TArray<ComponentType> FDMXPixelMappingToolkit::MakeComponentArray(const TSet<FDMXPixelMappingComponentReference>& Components) const
{
	TArray<ComponentType> Result;
	for (const FDMXPixelMappingComponentReference& Component : Components)
	{
		if (ComponentType* CastedComponent = Cast<ComponentType>(Component))
		{
			Result.Add(CastedComponent);
		}
	}
}

void FDMXPixelMappingToolkit::SelectComponents(const TSet<FDMXPixelMappingComponentReference>& InSelectedComponents)
{
	checkf(Selection, TEXT("Unexpected invalid selection object in pixel mapping toolkit."));

	// Update selection
	Selection->Modify();
	Selection->Components.Reset();
	ActiveOutputComponents.Reset();

	Selection->Components.Append(InSelectedComponents);

	for (const FDMXPixelMappingComponentReference& ComponentReference : Selection->Components)
	{
		if (UDMXPixelMappingRootComponent* RootComponent = Cast<UDMXPixelMappingRootComponent>(ComponentReference.GetComponent()))
		{
			continue;
		}
		else if (UDMXPixelMappingRendererComponent* RendererComponent = Cast<UDMXPixelMappingRendererComponent>(ComponentReference.GetComponent()))
		{
			SetActiveRenderComponent(RendererComponent);
		}
		else
		{
			if (UDMXPixelMappingRendererComponent* RendererComponentParent = ComponentReference.GetComponent() ? ComponentReference.GetComponent()->GetFirstParentByClass<UDMXPixelMappingRendererComponent>(ComponentReference.GetComponent()) : nullptr)
			{
				SetActiveRenderComponent(RendererComponentParent);
			}
		}

		if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(ComponentReference.GetComponent()))
		{
			ActiveOutputComponents.Add(OutputComponent);
		}
	}

	// Always order selected components topmost, but keep their relative z-ordering
	TArray<UDMXPixelMappingOutputComponent*> SelectedOutputComponents;
	Algo::TransformIf(Selection->Components, SelectedOutputComponents,
		[](const FDMXPixelMappingComponentReference& ComponentReference)
		{
			return ComponentReference.GetComponent() && ComponentReference.GetComponent()->IsA(UDMXPixelMappingOutputComponent::StaticClass());
		},
		[](const FDMXPixelMappingComponentReference& ComponentReference)
		{
			return CastChecked<UDMXPixelMappingOutputComponent>(ComponentReference.GetComponent());
		});
	Algo::SortBy(SelectedOutputComponents, &UDMXPixelMappingOutputComponent::GetZOrder);
	for (UDMXPixelMappingOutputComponent* SelectedComponent : SelectedOutputComponents)
	{
		SelectedComponent->ZOrderTopmost();
	}

	OnSelectedComponentsChangedDelegate.Broadcast();
}

bool FDMXPixelMappingToolkit::IsComponentSelected(UDMXPixelMappingBaseComponent* Component) const
{
	for (const FDMXPixelMappingComponentReference& ComponentReference : Selection->Components)
	{
		if (Component && Component == ComponentReference.GetComponent())
		{
			return true;
		}
	}
	return false;
}

void FDMXPixelMappingToolkit::AddRenderer()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	UDMXPixelMappingRootComponent* RootComponent = PixelMapping ? PixelMapping->GetRootComponent() : nullptr;
	if (RootComponent)
	{
		const FScopedTransaction AddMappingTransaction(LOCTEXT("AddMappingTransaction", "Add Mapping to Pixel Mapping"));

		RootComponent->PreEditChange(UDMXPixelMappingBaseComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingBaseComponent, Children)));
		UDMXPixelMappingRendererComponent* NewRendererComponent = FDMXPixelMappingEditorUtils::AddRenderer(PixelMapping);
		RootComponent->PostEditChange();

		SetActiveRenderComponent(NewRendererComponent);

		const FDMXPixelMappingComponentReference ComponentReference(StaticCastSharedRef<FDMXPixelMappingToolkit>(AsShared()), NewRendererComponent);
		SelectComponents({ ComponentReference } );
	}
}

void FDMXPixelMappingToolkit::PlayDMX()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (PixelMapping)
	{
		PixelMapping->StartSendingDMX();
	}
}

void FDMXPixelMappingToolkit::PauseDMX()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (PixelMapping)
	{
		PixelMapping->PauseSendingDMX();
	}
}

void FDMXPixelMappingToolkit::StopPlayingDMX()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (PixelMapping)
	{
		PixelMapping->StopSendingDMX();
	}
}

void FDMXPixelMappingToolkit::TogglePlayPauseDMX()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (PixelMapping && PixelMapping->IsSendingDMX())
	{
		PixelMapping->PauseSendingDMX();
	}
	else if (PixelMapping)
	{
		PixelMapping->StartSendingDMX();
	}
}

void FDMXPixelMappingToolkit::TogglePlayStopDMX()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (PixelMapping && PixelMapping->IsSendingDMX())
	{
		PixelMapping->StopSendingDMX();
	}
	else if (PixelMapping)
	{
		PixelMapping->StartSendingDMX();
	}
}

void FDMXPixelMappingToolkit::SetEditorResetDMXMode(EDMXPixelMappingResetDMXMode NewMode)
{
	if (UDMXPixelMapping* PixelMapping = GetDMXPixelMapping())
	{
		PixelMapping->SetResetDMXMode(NewMode);
	}
}

void FDMXPixelMappingToolkit::UpdateBlueprintNodes() const
{
	if (UDMXPixelMapping* PixelMapping = GetDMXPixelMapping())
	{
		for (TObjectIterator<UK2Node_PixelMappingBaseComponent> It(RF_Transient | RF_ClassDefaultObject, /** bIncludeDerivedClasses */ true, /** InternalExcludeFlags */ EInternalObjectFlags::Garbage); It; ++It)
		{
			It->OnPixelMappingChanged(PixelMapping);
		}
	}
}

void FDMXPixelMappingToolkit::SaveThumbnailImage()
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	UDMXPixelMappingRendererComponent* RendererComponent = GetActiveRendererComponent();
	if (!PixelMapping || !RendererComponent)
	{
		return;
	}

	// Fully load the input texture
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (InputTexture)
	{
		InputTexture->WaitForPendingInitOrStreaming();
	}

	// Don't set a thumbnail if no texture is available or no pixel mapping is setup
	using namespace UE::DMXPixelMapping::Rendering;
	const TArray<TSharedRef<FPixelMapRenderElement>> RenderElements = RendererComponent->GetPixelMapRenderElements();

	const bool bIsEmptyMapping = !InputTexture || !InputTexture->GetResource() || RenderElements.IsEmpty();
	if (bIsEmptyMapping)
	{
		PixelMapping->ThumbnailImage = nullptr;
		return;
	}

	// Paint a preview of the pixel mapping
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>();

	constexpr uint32 ThumbnailSize = 64;
	RenderTarget->InitAutoFormat(ThumbnailSize, ThumbnailSize);

	constexpr FHitProxyConsumer* HitProxyConsumer = nullptr;
	FCanvas Canvas(RenderTarget->GameThread_GetRenderTargetResource(), HitProxyConsumer, FGameTime(), GMaxRHIFeatureLevel);
	Canvas.Clear(FColor::Black);

	if (!InputTexture || !InputTexture->GetResource())
	{
		return;
	}

	for (const TSharedRef<FPixelMapRenderElement>& Element : RenderElements)
	{
		const FVector2D UV = Element->GetParameters().UV;
		const FVector2D UVSize = Element->GetParameters().UVSize;

		constexpr uint32 Margin = 12;
		constexpr uint32 ThumbnailSizeWithoutMargin = ThumbnailSize - Margin * 2.f;
		const FVector2D NormalizedMargin = FVector2D(Margin) / FVector2D(ThumbnailSize);

		const FVector2D Position = FVector2D(Margin) + UV * FIntPoint(ThumbnailSizeWithoutMargin, ThumbnailSizeWithoutMargin);
		const FVector2D Size = UVSize * FVector2D(ThumbnailSizeWithoutMargin, ThumbnailSizeWithoutMargin);

		FCanvasTileItem TileItem(Position, Size, Element->GetColor());
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_MAX;
		TileItem.PivotPoint = FVector2D(0.5, 0.5);
		TileItem.Rotation = FRotator(0.0, Element->GetParameters().Rotation, 0.0);
		Canvas.DrawItem(TileItem);
	}
	Canvas.Flush_GameThread();

	// Set the rendered thumbnail image 
	if (IsValid(RenderTarget))
	{
		PixelMapping->ThumbnailImage = NewObject<UTexture2D>(PixelMapping);
		RenderTarget->UpdateTexture(PixelMapping->ThumbnailImage);
	}
}

TArray<UDMXPixelMappingBaseComponent*> FDMXPixelMappingToolkit::CreateComponentsFromTemplates(UDMXPixelMappingRootComponent* RootComponent, UDMXPixelMappingBaseComponent* Target, const TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>>& Templates)
{
	TArray<UDMXPixelMappingBaseComponent*> NewComponents;
	if (Templates.Num() > 0)
	{
		TGuardValue Guard(bAddingComponents, true);

		if (ensureMsgf(RootComponent && Target, TEXT("Tried to create components from template but RootComponent or Target were invalid.")))
		{
			const float NumSteps = Templates.Num();
			FScopedSlowTask Task(NumSteps, LOCTEXT("CreateComponentsFromTemplatesSlowTask", "Creating Components..."));
			Task.MakeDialogDelayed(.5f);

			for (const TSharedPtr<FDMXPixelMappingComponentTemplate>& Template : Templates)
			{
				Task.EnterProgressFrame();

				if (UDMXPixelMappingBaseComponent* NewComponent = Template->CreateComponent<UDMXPixelMappingBaseComponent>(RootComponent))
				{
					NewComponents.Add(NewComponent);

					Target->Modify();
					NewComponent->Modify();

					Target->AddChild(NewComponent);

					// Find a reasonable size when components are added to a fixture group
					UDMXPixelMappingOutputComponent* NewOutputComponent = Cast<UDMXPixelMappingOutputComponent>(NewComponent);
					const UDMXPixelMappingOutputComponent* ParentOutputComponent = NewOutputComponent ? Cast<UDMXPixelMappingOutputComponent>(NewOutputComponent->GetParent()) : nullptr;
					const UDMXPixelMappingFixtureGroupComponent* GroupComponent = ParentOutputComponent ? Cast<UDMXPixelMappingFixtureGroupComponent>(ParentOutputComponent) : nullptr;
					const UDMXLibrary* DMXLibrary = GroupComponent ? GroupComponent->DMXLibrary.Get() : nullptr;
					if (NewOutputComponent && GroupComponent && DMXLibrary)
					{
						const TArray<UDMXEntityFixturePatch*> FixturePatches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
						const int32 Columns = FMath::RoundFromZero(FMath::Sqrt((float)FixturePatches.Num()));
						const int32 Rows = FMath::RoundFromZero((float)FixturePatches.Num() / Columns);
						const FVector2D Size = FVector2D(GroupComponent->GetSize().X / Columns, GroupComponent->GetSize().Y / Rows);

						NewOutputComponent->SetSize(Size);
					}

					// Output components need to adopt the initial rotation from their parent if possible
					if (NewOutputComponent && ParentOutputComponent)
					{
						NewOutputComponent->SetRotation(ParentOutputComponent->GetRotation());
					}
				}
			}
		}

		UpdateBlueprintNodes();
	}

	return NewComponents;
}

void FDMXPixelMappingToolkit::DeleteSelectedComponents()
{
	checkf(Selection, TEXT("Unexpected invalid selection object in pixel mapping toolkit."));

	if (Selection->Components.IsEmpty())
	{
		return;
	}

	TGuardValue Guard(bRemovingComponents, true);

	TSet<FDMXPixelMappingComponentReference> ParentComponentReferences;
	for (const FDMXPixelMappingComponentReference& SelectedComponentReference : Selection->Components)
	{
		UDMXPixelMappingBaseComponent* SelectedComponent = SelectedComponentReference.GetComponent();
		if (SelectedComponent)
		{
			constexpr bool bModifyChildrenRecursively = true;
			SelectedComponent->ForEachChild([](UDMXPixelMappingBaseComponent* ChildComponent)
				{
					ChildComponent->Modify();
				}, bModifyChildrenRecursively);

			UDMXPixelMappingBaseComponent* ParentComponent = SelectedComponent->GetParent();
			if (ParentComponent)
			{
				ParentComponent->Modify();
				SelectedComponent->Modify();

				ParentComponent->RemoveChild(SelectedComponent);

				const bool bParentComponentIsBeingRemoved = Algo::FindBy(Selection->Components, ParentComponent, &FDMXPixelMappingComponentReference::GetComponent) != nullptr;
				if (!bParentComponentIsBeingRemoved)
				{
					ParentComponentReferences.Add(FDMXPixelMappingComponentReference(StaticCastSharedRef<FDMXPixelMappingToolkit>(AsShared()), ParentComponent));
				}
			}
		}
	}
	
	// Select the Parent Components
	SelectComponents(ParentComponentReferences);

	UpdateBlueprintNodes();
}	

bool FDMXPixelMappingToolkit::CanPerformCommandsOnGroup() const
{
	const UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = GetFixtureGroupFromSelection(); 
	return FixtureGroupComponent != nullptr;
}

void FDMXPixelMappingToolkit::FlipGroup(EOrientation Orientation, bool bTransacted)
{
	if (!ensureMsgf(CanPerformCommandsOnGroup(), TEXT("Trying to flip cells without previously testing CanPerformCommandsOnGroup.")))
	{
		return;
	}

	UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = GetFixtureGroupFromSelection();
	if (!FixtureGroupComponent)
	{
		return;
	}

	TSharedPtr<FScopedTransaction> FilpCellsTransaction;
	if (bTransacted)
	{
		const FText TransactionText = FText::Format(LOCTEXT("FilpCellsTransaction", "Flip Group {0}"), Orientation == EOrientation::Orient_Horizontal ? 
			LOCTEXT("FlipHorizontalText", "Horizontally") :
			LOCTEXT("FlipVerticalText", "Vertically"));

		FilpCellsTransaction = MakeShared<FScopedTransaction>(TransactionText);
	}

	const double RestoreRotation = FixtureGroupComponent->GetRotation();
	FixtureGroupComponent->SetRotation(0.0);

	const FVector2D Center = FixtureGroupComponent->GetPosition() + FixtureGroupComponent->GetSize() / 2.f;

	constexpr bool bRecursive = false;
	FixtureGroupComponent->ForEachChildOfClass<UDMXPixelMappingOutputComponent>([&Center, Orientation](UDMXPixelMappingOutputComponent* Child)
		{
			const FVector2D ChildPivotOffset = Child->GetSize() / 2.f;
			const FVector2D ChildCenter = Child->GetPosition() + Child->GetSize() / 2.f;
			const FVector2D NewPositionBothAxes = Center + Center - ChildCenter - ChildPivotOffset;
			if (Orientation == EOrientation::Orient_Horizontal)
			{
				Child->SetPosition(FVector2D(NewPositionBothAxes.X, Child->GetPosition().Y));
			}
			else
			{
				Child->SetPosition(FVector2D(Child->GetPosition().X, NewPositionBothAxes.Y));
			}
		},
		bRecursive);

	FixtureGroupComponent->SetRotation(RestoreRotation);
}

void FDMXPixelMappingToolkit::SizeGroupToTexture(bool bTransacted)
{
	if (!ensureMsgf(CanPerformCommandsOnGroup(), TEXT("Trying to size selected component to texture without previously testing CanPerformCommandsOnGroup.")))
	{
		return;
	}

	UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = GetFixtureGroupFromSelection();
	if (!FixtureGroupComponent)
	{
		return;
	}

	const UDMXPixelMappingRendererComponent* RendererComponent = GetActiveRendererComponent();
	if (!RendererComponent)
	{
		return;
	}

	const FVector2D TextureSize = RendererComponent->GetSize();
	if (TextureSize == FVector2D::ZeroVector)
	{
		return;
	}

	TSharedPtr<FScopedTransaction> SizeGroupToTextureTransaction;
	if (bTransacted)
	{ 
		SizeGroupToTextureTransaction = MakeShared<FScopedTransaction>(LOCTEXT("SizeGroupToTextureTransaction", "Size Group to Texture"));
	}

	FixtureGroupComponent->Modify();
	FixtureGroupComponent->SetRotation(0.0);
	FixtureGroupComponent->SetPosition(FVector2D::ZeroVector);
	FixtureGroupComponent->SetSize(TextureSize);
}

void FDMXPixelMappingToolkit::SetTransformHandleMode(EDMXPixelMappingTransformHandleMode NewTransformHandleMode)
{
	TransformHandleMode = NewTransformHandleMode;
}

void FDMXPixelMappingToolkit::ToggleGridSnapping()
{
	if (UDMXPixelMapping* PixelMapping = GetDMXPixelMapping())
	{
		const FScopedTransaction AddMappingTransaction(LOCTEXT("ToggleGridSnappingTransaction", "Toggle Grid Snapping"));
		PixelMapping->PreEditChange(UDMXPixelMapping::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMapping, bGridSnappingEnabled)));
		PixelMapping->bGridSnappingEnabled = !PixelMapping->bGridSnappingEnabled;
		PixelMapping->PostEditChange();
	}
}

void FDMXPixelMappingToolkit::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Selection);
}

FString FDMXPixelMappingToolkit::GetReferencerName() const
{
	return TEXT("FDMXPixelMappingToolkit");
}

void FDMXPixelMappingToolkit::PostUndo(bool bSuccess)
{
	UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	UDMXPixelMappingRootComponent* RootComponent = PixelMapping ? PixelMapping->GetRootComponent() : nullptr;
	if (!RootComponent)
	{
		return;
	}

	constexpr bool bRecursive = false;
	RootComponent->ForEachChild([](UDMXPixelMappingBaseComponent* Component)
		{
			if (UDMXPixelMappingRendererComponent* RendererComponent = Cast<UDMXPixelMappingRendererComponent>(Component))
			{
				RendererComponent->UpdatePreprocessRenderer();
			}
		}, bRecursive);
}

void FDMXPixelMappingToolkit::PostRedo(bool bSuccess)
{
	// Same behaviour as PostUndo
	PostUndo(bSuccess);
}

void FDMXPixelMappingToolkit::OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	if (!bAddingComponents && !bRemovingComponents)
	{
		UpdateBlueprintNodes();
	}
}

void FDMXPixelMappingToolkit::OnComponentRenamed(UDMXPixelMappingBaseComponent* Component)
{
	UpdateBlueprintNodes();
}

TSharedRef<SDockTab> FDMXPixelMappingToolkit::SpawnTab_DMXLibraryView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DMXLibraryViewTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("DMXLibraryViewTabID", "DMXLibrary"))
		[
			DMXLibraryView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDMXPixelMappingToolkit::SpawnTab_HierarchyView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == HierarchyViewTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("HierarchyViewTabID", "Hierarchy"))
		[
			HierarchyView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDMXPixelMappingToolkit::SpawnTab_DesignerView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DesignerViewTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("DesignerViewTabID", "Designer"))
		[
			DesignerView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDMXPixelMappingToolkit::SpawnTab_PreviewView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PreviewViewTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("PreviewViewTabID", "Preview"))
		[
			PreviewView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDMXPixelMappingToolkit::SpawnTab_DetailsView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == DetailsViewTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("DetailsViewTabID", "Details"))
		[
			DetailsView.ToSharedRef()
		];

	return SpawnedTab;
}

TSharedRef<SDockTab> FDMXPixelMappingToolkit::SpawnTab_LayoutView(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == LayoutViewTabID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("LayoutViewTabID", "Layout"))
		[
			LayoutView.ToSharedRef()
		];

	return SpawnedTab;
}

void FDMXPixelMappingToolkit::CreateInternalViews()
{
	GetOrCreateDMXLibraryView();
	GetOrCreateHierarchyView();
	GetOrCreateDesignerView();
	GetOrCreatePreviewView();
	GetOrCreateDetailsView();
	GetOrCreateLayoutView();
}

void FDMXPixelMappingToolkit::PreSavePackage(class UPackage* Package, FObjectPreSaveContext Context)
{
	if (!Context.IsCooking())
	{
		UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
		if (PixelMapping && PixelMapping->GetPackage() == Package)
		{
			SaveThumbnailImage();
		}
	}
}

void FDMXPixelMappingToolkit::RenameComponent(const FName& CurrentObjectName, const FString& DesiredObjectName) const
{
	const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
	if (!PixelMapping)
	{
		return;
	}

	UDMXPixelMappingBaseComponent* ComponentToRename = PixelMapping->FindComponent(CurrentObjectName);
	if (!ensureMsgf(ComponentToRename, TEXT("Cannot find component '%s' to rename."), *CurrentObjectName.ToString()))
	{
		return;
	}

	const FName DesiredDisplayName = MakeObjectNameFromDisplayLabel(DesiredObjectName, ComponentToRename->GetFName());
	UDMXPixelMappingBaseComponent* ExistingComponent = PixelMapping->FindComponent(DesiredDisplayName);

	const FName UniqueName = ExistingComponent ?
		MakeUniqueObjectName(ComponentToRename->GetOuter(), ComponentToRename->GetClass(), DesiredDisplayName) :
		DesiredDisplayName;

	ComponentToRename->Modify();
	ComponentToRename->Rename(*UniqueName.ToString());
	UpdateBlueprintNodes();
}

const TSet<FDMXPixelMappingComponentReference>& FDMXPixelMappingToolkit::GetSelectedComponents() const
{
	checkf(Selection, TEXT("Unexpected invalid selection object in pixel mapping toolkit."));

	return Selection->Components;
}

TSharedRef<SDMXPixelMappingDMXLibraryView> FDMXPixelMappingToolkit::GetOrCreateDMXLibraryView()
{
	if (!DMXLibraryView.IsValid())
	{
		DMXLibraryView = SNew(SDMXPixelMappingDMXLibraryView, SharedThis(this));
	}

	return DMXLibraryView.ToSharedRef();
}

TSharedRef<SDMXPixelMappingHierarchyView> FDMXPixelMappingToolkit::GetOrCreateHierarchyView()
{
	if (!HierarchyView.IsValid())
	{
		HierarchyView = SNew(SDMXPixelMappingHierarchyView, SharedThis(this));
	}

	return HierarchyView.ToSharedRef();
}

TSharedRef<SDMXPixelMappingDesignerView> FDMXPixelMappingToolkit::GetOrCreateDesignerView()
{
	if (!DesignerView.IsValid())
	{
		DesignerView = SNew(SDMXPixelMappingDesignerView, SharedThis(this));
	}

	return DesignerView.ToSharedRef();
}

TSharedRef<SDMXPixelMappingPreviewView> FDMXPixelMappingToolkit::GetOrCreatePreviewView()
{
	if (!PreviewView.IsValid())
	{
		PreviewView = SNew(SDMXPixelMappingPreviewView, SharedThis(this));
	}

	return PreviewView.ToSharedRef();
}

TSharedRef<SDMXPixelMappingDetailsView> FDMXPixelMappingToolkit::GetOrCreateDetailsView()
{
	if (!DetailsView.IsValid())
	{
		DetailsView = SNew(SDMXPixelMappingDetailsView, SharedThis(this));
	}

	return DetailsView.ToSharedRef();
}

TSharedRef<SDMXPixelMappingLayoutView> FDMXPixelMappingToolkit::GetOrCreateLayoutView()
{
	if (!LayoutView.IsValid())
	{
		LayoutView = SNew(SDMXPixelMappingLayoutView, SharedThis(this));
	}

	return LayoutView.ToSharedRef();
}

bool FDMXPixelMappingToolkit::IsPlayingDMX() const
{
	const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();

	return PixelMapping && PixelMapping->IsSendingDMX();
}

#define UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, StructName, MemberName, Action) \
	GetToolkitCommands()->MapAction( \
		FDMXPixelMappingEditorCommands::Get().Action, \
		FExecuteAction::CreateLambda([] \
			{ \
				UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>(); \
				EditorSettings->StructName.MemberName = !EditorSettings->StructName.MemberName; \
				EditorSettings->SaveConfig(); \
			}), \
		FCanExecuteAction(), \
		FIsActionChecked::CreateLambda([]() \
			{  \
				const UDMXPixelMappingEditorSettings* EditorSettings = GetDefault<UDMXPixelMappingEditorSettings>(); \
				return EditorSettings->DesignerSettings.MemberName; \
			}) \
	); 

void FDMXPixelMappingToolkit::SetupCommands()
{
	// Create a command list for the designer view specifically
	DesignerCommandList = MakeShareable(new FUICommandList);
	DesignerCommandList->MapAction(FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::DeleteSelectedComponents)
	);

	// Init the command list for this toolkit 
	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().AddMapping,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::AddRenderer)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().PlayDMX,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::PlayDMX),
		FCanExecuteAction::CreateLambda([this]
			{
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();
				const bool bIsPaused = PixelMapping && PixelMapping->IsPaused();

				return !bIsSendingDMX && !bIsPaused;
			}),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this]
			{				
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();
				const bool bIsPaused = PixelMapping && PixelMapping->IsPaused();

				return !bIsSendingDMX && !bIsPaused;
			})
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().PauseDMX,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::PauseDMX),
		FCanExecuteAction::CreateLambda([this] 
			{ 
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();

				return bIsSendingDMX; 
			}),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this]
			{
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();

				return bIsSendingDMX;
			})
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().ResumeDMX,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::PlayDMX),
		FCanExecuteAction::CreateLambda([this]
			{
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();
				const bool bIsPaused = PixelMapping && PixelMapping->IsPaused();

				return !bIsSendingDMX && bIsPaused;
			}),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateLambda([this]
			{
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();
				const bool bIsPaused = PixelMapping && PixelMapping->IsPaused();

				return !bIsSendingDMX && bIsPaused;
			})
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().StopDMX,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::StopPlayingDMX),
		FCanExecuteAction::CreateLambda([this]
			{
				const UDMXPixelMapping* PixelMapping = GetDMXPixelMapping();
				const bool bIsSendingDMX = PixelMapping && PixelMapping->IsSendingDMX();
				const bool bIsPaused = PixelMapping && PixelMapping->IsPaused();

				return bIsSendingDMX || bIsPaused;
			})
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().TogglePlayPauseDMX,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::TogglePlayPauseDMX)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().TogglePlayStopDMX,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::TogglePlayStopDMX)
		);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().EditorStopSendsDefaultValues,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::SetEditorResetDMXMode, EDMXPixelMappingResetDMXMode::SendDefaultValues),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FDMXPixelMappingToolkit::GetEditorResetDMXModeCheckboxState, EDMXPixelMappingResetDMXMode::SendDefaultValues)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().EditorStopSendsZeroValues,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::SetEditorResetDMXMode, EDMXPixelMappingResetDMXMode::SendZeroValues),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FDMXPixelMappingToolkit::GetEditorResetDMXModeCheckboxState, EDMXPixelMappingResetDMXMode::SendZeroValues)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().EditorStopKeepsLastValues,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::SetEditorResetDMXMode, EDMXPixelMappingResetDMXMode::DoNotSendValues),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FDMXPixelMappingToolkit::GetEditorResetDMXModeCheckboxState, EDMXPixelMappingResetDMXMode::DoNotSendValues)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().EnableResizeMode,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::SetTransformHandleMode, EDMXPixelMappingTransformHandleMode::Resize),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FDMXPixelMappingToolkit::GetTransformHandleModeCheckboxState, EDMXPixelMappingTransformHandleMode::Resize)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().EnableRotateMode,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::SetTransformHandleMode, EDMXPixelMappingTransformHandleMode::Rotate),
		FCanExecuteAction(),
		FGetActionCheckState::CreateSP(this, &FDMXPixelMappingToolkit::GetTransformHandleModeCheckboxState, EDMXPixelMappingTransformHandleMode::Rotate)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().ToggleGridSnapping,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::ToggleGridSnapping)
	);

	// Designer related
	constexpr bool bTransact = true;
	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().FlipGroupHorizontally,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::FlipGroup, EOrientation::Orient_Horizontal, bTransact),
		FCanExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::CanPerformCommandsOnGroup),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FDMXPixelMappingToolkit::CanPerformCommandsOnGroup)
	);
	
	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().FlipGroupVertically,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::FlipGroup, EOrientation::Orient_Vertical, bTransact),
		FCanExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::CanPerformCommandsOnGroup),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FDMXPixelMappingToolkit::CanPerformCommandsOnGroup)
	);

	GetToolkitCommands()->MapAction(
		FDMXPixelMappingEditorCommands::Get().SizeGroupToTexture,
		FExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::SizeGroupToTexture, bTransact),
		FCanExecuteAction::CreateSP(this, &FDMXPixelMappingToolkit::CanPerformCommandsOnGroup),
		FIsActionChecked(),
		FIsActionButtonVisible::CreateSP(this, &FDMXPixelMappingToolkit::CanPerformCommandsOnGroup)
	);

	UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>(); 
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bScaleChildrenWithParent, ToggleScaleChildrenWithParent);
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bAlwaysSelectGroup, ToggleAlwaysSelectGroup);
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bShowMatrixCells, ToggleShowMatrixCells);
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bShowComponentNames, ToggleShowComponentNames);
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bShowPatchInfo, ToggleShowPatchInfo);
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bShowCellIDs, ToggleShowCellIDs);
	UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND(EditorSettings, DesignerSettings, bShowPivot, ToggleShowPivot);
}
#undef UE_DMX_MAP_EDITOR_SETTING_TO_TOGGLE_COMMAND

void FDMXPixelMappingToolkit::ExtendToolbar()
{
	Toolbar = MakeShared<FDMXPixelMappingToolbar>(SharedThis(this));

	Toolbar->ExtendToolbar();

	// Let other part of the plugin extend DMX Pixel Maping Editor toolbar
	FDMXPixelMappingEditorModule& DMXPixelMappingEditorModule = FModuleManager::LoadModuleChecked<FDMXPixelMappingEditorModule>("DMXPixelMappingEditor");
	AddMenuExtender(DMXPixelMappingEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
	AddToolbarExtender(DMXPixelMappingEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

ECheckBoxState FDMXPixelMappingToolkit::GetEditorResetDMXModeCheckboxState(EDMXPixelMappingResetDMXMode CompareMode) const
{
	if (UDMXPixelMapping* PixelMapping = GetDMXPixelMapping())
	{
		return PixelMapping->GetResetDMXMode() == CompareMode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

ECheckBoxState FDMXPixelMappingToolkit::GetTransformHandleModeCheckboxState(EDMXPixelMappingTransformHandleMode CompareTransformHandleMode) const
{
	return CompareTransformHandleMode == TransformHandleMode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

UDMXPixelMappingFixtureGroupComponent* FDMXPixelMappingToolkit::GetFixtureGroupFromSelection() const
{
	checkf(Selection, TEXT("Unexpected invalid selection object in pixel mapping toolkit."));

	TArray<UDMXPixelMappingFixtureGroupComponent*> FixtureGroupComponents;
	for (const FDMXPixelMappingComponentReference& ComponentReference : Selection->Components)
	{
		UDMXPixelMappingBaseComponent* Component = ComponentReference.GetComponent();
		if (!Component)
		{
			continue;
		}

		UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = nullptr;
		if (Component->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass())
		{
			FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Component);
		}
		if ((Component->GetClass() == UDMXPixelMappingFixtureGroupItemComponent::StaticClass() ||
			Component->GetClass() == UDMXPixelMappingMatrixComponent::StaticClass()) &&
			Component->GetParent())
		{
			FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Component->GetParent());
		}
		else if (Component->GetClass() == UDMXPixelMappingMatrixCellComponent::StaticClass() &&
			Component->GetParent() &&
			Component->GetParent()->GetParent())
		{
			FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Component->GetParent()->GetParent());
		}

		if (FixtureGroupComponent)
		{
			FixtureGroupComponents.AddUnique(FixtureGroupComponent);
		}
	}

	// Return the group only if exactly one is contained in selection
	if (FixtureGroupComponents.Num() == 1)
	{
		return FixtureGroupComponents[0];
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
