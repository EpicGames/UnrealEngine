// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Analytics/DMXEditorToolAnalyticsProvider.h"
#include "DMXPixelMappingComponentReference.h"
#include "EditorUndoClient.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "TickableEditorObject.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/GCObject.h"
#include "Widgets/Views/SHeaderRow.h"

#include "DMXPixelMappingToolkit.generated.h"

enum class ECheckBoxState : uint8;
enum class EDMXPixelMappingResetDMXMode : uint8;
class FDMXPixelMappingComponentTemplate;
class FDMXPixelMappingToolbar;
class FSpawnTabArgs;
class FTabManager;
class SDockableTab;
class SDMXPixelMappingHierarchyView;
class SDMXPixelMappingDesignerView;
class SDMXPixelMappingDetailsView;
class SDMXPixelMappingDMXLibraryView;
class SDMXPixelMappingLayoutView;
class SDMXPixelMappingPreviewView;
class UDMXPixelMapping;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingFixtureGroupComponent;
class UDMXPixelMappingMatrixComponent;
class UDMXPixelMappingOutputComponent;
class UDMXPixelMappingRendererComponent;

namespace UE::DMX
{
	enum class EDMXPixelMappingTransformHandleMode : uint8
	{
		Resize,
		Rotate
	};
}

/** Selection in the pixel mapping toollkit. Using the UObject system for undo/redo. */
UCLASS(Transient)
class UDMXPixelMappingToolkitSelection
	: public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TSet<FDMXPixelMappingComponentReference> Components;

	UPROPERTY()
	TWeakObjectPtr<UDMXPixelMappingRendererComponent> ActiveRendererComponent;
};

/**
 * Implements an Editor toolkit for Pixel Mapping.
 */
class FDMXPixelMappingToolkit
	: public FAssetEditorToolkit
	, public FTickableEditorObject
	, public FSelfRegisteringEditorUndoClient
	, public FGCObject
{
	using EDMXPixelMappingTransformHandleMode = UE::DMX::EDMXPixelMappingTransformHandleMode;

public:
	DECLARE_MULTICAST_DELEGATE(FOnSelectedComponentsChangedDelegate)
	FOnSelectedComponentsChangedDelegate& GetOnSelectedComponentsChangedDelegate() { return OnSelectedComponentsChangedDelegate; }

public:
	FDMXPixelMappingToolkit();
	~FDMXPixelMappingToolkit();

	/**
	 * Initilaizes the pixel mapping editor.
	 *
	 * @param Mode				The Toolkit Mode.
	 * @param					THe Toolkit Host.
	 * @param UDMXPixelMapping	The Pixel Mapping object that is being edited.
	 */
	void InitPixelMappingEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, UDMXPixelMapping* InDMXPixelMapping);

public:
	//~ Begin FAssetEditorToolkit Interface
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	//~ End FAssetEditorToolkit Interface

	//~ Begin IToolkit Interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override { return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f); }
	virtual FString GetWorldCentricTabPrefix() const override;
	//~ End IToolkit Interface

	//~ Begin FTickableEditorObject interface
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return true; }
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject Interface

	UDMXPixelMapping* GetDMXPixelMapping() const;

	FDMXPixelMappingComponentReference GetReferenceFromComponent(UDMXPixelMappingBaseComponent* InComponent);

	UDMXPixelMappingRendererComponent* GetActiveRendererComponent() const;

	const TSharedPtr<FUICommandList>& GetDesignerCommandList() const { return DesignerCommandList; }

	const TSet<FDMXPixelMappingComponentReference>& GetSelectedComponents() const;

	/** Gets or creates the DMX Library View for this Pixel Mapping instance */
	TSharedRef<SDMXPixelMappingDMXLibraryView> GetOrCreateDMXLibraryView();

	/** Gets or creates the Hierarchy View for this Pixel Mapping instance */
	TSharedRef<SDMXPixelMappingHierarchyView> GetOrCreateHierarchyView();

	/** Gets or creates the Designer View for this Pixel Mapping instance */
	TSharedRef<SDMXPixelMappingDesignerView> GetOrCreateDesignerView();

	/** Gets or creates the Preview View for this Pixel Mapping instance */
	TSharedRef<SDMXPixelMappingPreviewView> GetOrCreatePreviewView();

	/** Gets or creates the Details View for this Pixel Mapping instance */
	TSharedRef<SDMXPixelMappingDetailsView> GetOrCreateDetailsView();

	/** Gets or creates the Layout View for this Pixel Mapping instance */
	TSharedRef<SDMXPixelMappingLayoutView> GetOrCreateLayoutView();

	/** Returns true if the pixel mapping asset is currently playing DMX */
	bool IsPlayingDMX() const;

	void SetActiveRenderComponent(UDMXPixelMappingRendererComponent* InComponent);

	/** Creates an array of components given specifed component references */
	template <typename ComponentType>
	TArray<ComponentType> MakeComponentArray(const TSet<FDMXPixelMappingComponentReference>& Components) const;

	void SelectComponents(const TSet<FDMXPixelMappingComponentReference>& Components);

	/** Returns true if the component is selected */
	bool IsComponentSelected(UDMXPixelMappingBaseComponent* Component) const;

	/** Adds a renderer to the pixel mapping. Sets it as active renderer if a new renderer was created */
	void AddRenderer();

	UE_DEPRECATED(5.3, "Removed without replacement. Use UObject::Rename instead")
	void RenameComponent(const FName& PreviousObjectName, const FString& DesiredObjectName) const;

	/** 
	 * Creates components from the template. Returns the new components.
	 * If many component were created, the first component is the topmost parent.
	 */
	TArray<UDMXPixelMappingBaseComponent*> CreateComponentsFromTemplates(UDMXPixelMappingRootComponent* RootComponent, UDMXPixelMappingBaseComponent* Target, const TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>>& Templates);

	/** Deletes the selected Components */
	void DeleteSelectedComponents();

	/** Returns true if the current selection yields a fixture group on which commands can be performed on, e.g. SizeGroupToTexture */
	bool CanPerformCommandsOnGroup() const;

	/** Flips children of a group either horizontally or vertically */
	void FlipGroup(EOrientation Orientation, bool bTransacted);

	/** Sizes the component to the render target of the pixelmapping asset */
	void SizeGroupToTexture(bool bTransacted);

	/** Toggles between grid snapping enabled and disabled */
	void ToggleGridSnapping();

	/** Sets how transform handles operate */
	void SetTransformHandleMode(EDMXPixelMappingTransformHandleMode NewTransformHandleMode);

	/** Returns the current transform handle mode */
	EDMXPixelMappingTransformHandleMode GetTransformHandleMode() const { return TransformHandleMode; }

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	//~ End FGCObject interface

private:
	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

	/** Called when a component was added to the pixel mapping */
	void OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** Called when a component object was renamed */
	void OnComponentRenamed(UDMXPixelMappingBaseComponent* Component);

	/** Starts to play DMX on editor tick */
	void PlayDMX();

	/** Pauses playing DMX. Current DMX values will still be sent at a lower rate. */
	void PauseDMX();

	/** Stops playing DMX */
	void StopPlayingDMX();

	/** Toggles between playing and pausing DMX */
	void TogglePlayPauseDMX();

	/** Toggles between playing and stopping DMX */
	void TogglePlayStopDMX();

	/** Sets the reset DMX mode used by the editor */
	void SetEditorResetDMXMode(EDMXPixelMappingResetDMXMode NewMode);

	/** Updates blueprint nodes */
	void UpdateBlueprintNodes() const;

	/** Saves a thumbnail image for the pixel mapping asset */
	void SaveThumbnailImage();

	/** Spawns the DMX Library View */
	TSharedRef<SDockTab> SpawnTab_DMXLibraryView(const FSpawnTabArgs& Args);

	/** Spawns the Hierarchy View */
	TSharedRef<SDockTab> SpawnTab_HierarchyView(const FSpawnTabArgs& Args);

	/** Spawns the Designer View */
	TSharedRef<SDockTab> SpawnTab_DesignerView(const FSpawnTabArgs& Args);

	/** Spawns the Preview View */
	TSharedRef<SDockTab> SpawnTab_PreviewView(const FSpawnTabArgs& Args);

	/** Spawns the Details View */
	TSharedRef<SDockTab> SpawnTab_DetailsView(const FSpawnTabArgs& Args);

	/** Spawns the Layout View */
	TSharedRef<SDockTab> SpawnTab_LayoutView(const FSpawnTabArgs& Args);

	void SetupCommands();

	void ExtendToolbar();

	void CreateInternalViews();

	/** Called before any package was saved */
	void PreSavePackage(class UPackage* Package, FObjectPreSaveContext Context);

	/** Returns the check box state for the compared reset DMX mode */
	ECheckBoxState GetEditorResetDMXModeCheckboxState(EDMXPixelMappingResetDMXMode CompareMode) const;

	/** Returns the checkbox state for a transform handle mode, checked if the mode equals the current mode. */
	ECheckBoxState GetTransformHandleModeCheckboxState(EDMXPixelMappingTransformHandleMode CompareTransformHandleMode) const;

	/** 
	 * Returns the fixture group from a selection. Returns the parent group if the primary selection is a child of a group.
	 * Returns the primary fixture group in selection or nullptr if no fixture group can be deduced from selection. 
	 */
	UDMXPixelMappingFixtureGroupComponent* GetFixtureGroupFromSelection() const;

	/** List of open tool panels; used to ensure only one exists at any one time */
	TMap<FName, TWeakPtr<SDockableTab>> SpawnedToolPanels;

	/** The DMX Library View instance */
	TSharedPtr<SDMXPixelMappingDMXLibraryView> DMXLibraryView;

	/** The Hierarchy View instance */
	TSharedPtr<SDMXPixelMappingHierarchyView> HierarchyView;

	/** The Designer View instance */
	TSharedPtr<SDMXPixelMappingDesignerView> DesignerView;

	/** The Preview View instance */
	TSharedPtr<SDMXPixelMappingPreviewView> PreviewView;

	/** The Details View instance */
	TSharedPtr<SDMXPixelMappingDetailsView> DetailsView;

	/** The Layout View instance */
	TSharedPtr<SDMXPixelMappingLayoutView> LayoutView;

	FOnSelectedComponentsChangedDelegate OnSelectedComponentsChangedDelegate;

	/** The selection object */
	TObjectPtr<UDMXPixelMappingToolkitSelection> Selection;

	TSharedPtr<FDMXPixelMappingToolbar> Toolbar;

	TArray<TWeakObjectPtr<UDMXPixelMappingOutputComponent>> ActiveOutputComponents;

	/** Command list for handling widget actions in the PixelMapping Toolkit */
	TSharedPtr<FUICommandList> DesignerCommandList;

	/** True while adding components (to avoid needlessly updating blueprint nodes on each component added via our own methods) */
	bool bAddingComponents = false;

	/** True while removing components (to avoid needlessly updating blueprint nodes on each component removed via our own methods) */
	bool bRemovingComponents = false;

	/** The current transform handle mode */
	EDMXPixelMappingTransformHandleMode TransformHandleMode = EDMXPixelMappingTransformHandleMode::Resize;

public:
	/** Name of the DMX Library View Tab */
	static const FName DMXLibraryViewTabID;

	/** Name of the Hierarchy View Tab */
	static const FName HierarchyViewTabID;

	/** Name of the Designer View Tab */
	static const FName DesignerViewTabID;

	/** Name of the Preview View Tab */
	static const FName PreviewViewTabID;

	/** Name of the Details View Tab */
	static const FName DetailsViewTabID;

	/** Name of the Details View Tab */
	static const FName LayoutViewTabID;

	/** The analytics provider for this tool */
	UE::DMX::FDMXEditorToolAnalyticsProvider AnalyticsProvider;

	/** Dumped editor settings for fast comparison */
	TArray<uint8, TFixedAllocator<sizeof(UDMXPixelMappingEditorSettings)>> EditorSettingsDump;
};
