// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingDMXLibraryViewModel.h"

#include "Algo/MaxElement.h"
#include "Algo/Transform.h"
#include "Components/DMXPixelMappingBaseComponent.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorLog.h"
#include "Editor.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "Templates/DMXPixelMappingComponentTemplate.h"
#include "Toolkits/DMXPixelMappingToolkit.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingDMXLibraryViewModel"

void UDMXPixelMappingDMXLibraryViewModel::CreateAndSetNewFixtureGroup(TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit)
{
	if (!InWeakToolkit.IsValid())
	{
		return;
	}
	WeakToolkit = InWeakToolkit;
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

	UDMXPixelMappingRootComponent* RootComponent = GetPixelMappingRootComponent();
	UDMXPixelMappingRendererComponent* ActiveRendererComponent = Toolkit->GetActiveRendererComponent();
	if (!ensureMsgf(RootComponent && ActiveRendererComponent, TEXT("Cannot add a new fixture group to pixel mapping, root component is invalid or there is no active renderer component")))
	{
		return;
	}

	const FScopedTransaction AddNewFixtureGroupTransaction(LOCTEXT("AddNewFixtureGroupTransaction", "Add Fixture Group to Pixel Mapping"));
	RootComponent->Modify();

	const TArray<UDMXPixelMappingFixtureGroupComponent*> OtherFixtureGroupComponents = GetFixtureGroupComponentsOfSameLibrary();

	const TSharedRef<FDMXPixelMappingComponentTemplate> Template = MakeShared<FDMXPixelMappingComponentTemplate>(UDMXPixelMappingFixtureGroupComponent::StaticClass());
	const TArray<UDMXPixelMappingBaseComponent*> NewComponents = WeakToolkit.Pin()->CreateComponentsFromTemplates(RootComponent, ActiveRendererComponent, TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>>{ Template });
	if (!ensureMsgf(NewComponents.Num() == 1 && NewComponents[0]->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass(), TEXT("Cannot find newly added Fixture Group Component.")))
	{
		return;
	}

	// Create and size the new fixture group
	UDMXPixelMappingFixtureGroupComponent* NewFixtureGroupComponent = CastChecked<UDMXPixelMappingFixtureGroupComponent>(NewComponents[0]);
	SelectFixtureGroupComponent(NewFixtureGroupComponent);

	if (OtherFixtureGroupComponents.IsEmpty())
	{
		// If there's no group, add one that scales the texture of the active renderer component
		if (Toolkit->CanPerformCommandsOnGroup())
		{
			constexpr bool bTransacted = false;
			Toolkit->SizeGroupToTexture(bTransacted);
		}
	}
	else
	{
		// If there's already a group, offset over the top left of the existing group
		const UDMXPixelMappingFixtureGroupComponent* const* MostOffsetOtherPtr = Algo::MaxElementBy(OtherFixtureGroupComponents,
			[](const UDMXPixelMappingFixtureGroupComponent* Other)
			{
				return Other->GetPosition().Length();
			});
		checkf(MostOffsetOtherPtr, TEXT("No result from array that was tested to not be empty."));

		FVector2D NewSize = ActiveRendererComponent->GetSize();
		FVector2D NewPosition = (*MostOffsetOtherPtr)->GetPosition() + FVector2D(FMath::Max(1.f, NewSize.X / 16), FMath::Max(1.f, NewSize.Y / 16));

		NewFixtureGroupComponent->SetPosition(NewPosition);
		NewFixtureGroupComponent->SetSize(NewSize);
		NewFixtureGroupComponent->ZOrderTopmost();
	}

	UpdateFixtureGroupFromSelection(InWeakToolkit);
}

void UDMXPixelMappingDMXLibraryViewModel::UpdateFixtureGroupFromSelection(TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit)
{
	if (!InWeakToolkit.IsValid())
	{
		return;
	}
	WeakToolkit = InWeakToolkit;
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

	// Gather fixture groups present in the current selection
	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = Toolkit->GetSelectedComponents();
	TArray<UDMXPixelMappingFixtureGroupComponent*> FixtureGroupComponents;
	for (const FDMXPixelMappingComponentReference& ComponentReference : SelectedComponents)
	{
		for (UDMXPixelMappingBaseComponent* Component = ComponentReference.GetComponent(); Component != nullptr; Component = Component->GetParent())
		{
			if (Component->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass())
			{
				FixtureGroupComponents.AddUnique(CastChecked<UDMXPixelMappingFixtureGroupComponent>(Component));
			}
		}
	}

	// Update members, only allow a single selection. 
	if (FixtureGroupComponents.IsEmpty() || FixtureGroupComponents.Num() > 1)
	{
		DMXLibrary = nullptr;
		WeakFixtureGroupComponent = nullptr;
		bMoreThanOneFixtureGroupSelected = !FixtureGroupComponents.IsEmpty();
	}
	else
	{
		DMXLibrary = FixtureGroupComponents[0]->DMXLibrary;
		WeakFixtureGroupComponent = FixtureGroupComponents[0];
		bMoreThanOneFixtureGroupSelected = false;
	}
}

void UDMXPixelMappingDMXLibraryViewModel::SetNewComponentsUsePatchColor(bool bUsePatchColor)
{
	UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	if (PixelMapping)
	{
		PixelMapping->bNewComponentsUsePatchColor = bUsePatchColor;
	}
}

bool UDMXPixelMappingDMXLibraryViewModel::ShouldNewComponentsUsePatchColor() const
{
	const UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	return PixelMapping ? PixelMapping->bNewComponentsUsePatchColor : false;
}

void UDMXPixelMappingDMXLibraryViewModel::AddFixturePatchesEnsured(const TArray<UDMXEntityFixturePatch*>& FixturePatches)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (FixturePatches.IsEmpty() || !Toolkit.IsValid())
	{
		return;
	}

	// Ensure first patch is of the same library as the fixture group usees
	UDMXLibrary* CommonDMXLibrary = FixturePatches[0] ? FixturePatches[0]->GetParentLibrary() : nullptr;
	if (!ensureMsgf(WeakFixtureGroupComponent.IsValid() && CommonDMXLibrary && CommonDMXLibrary == WeakFixtureGroupComponent->DMXLibrary, TEXT("Cannot add Fixture Patches to a Fixture Group that doesn't use the Library of the patches")))
	{
		return;
	}
	UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = WeakFixtureGroupComponent.Get();

	// Ensure all patches are of same library
	const UDMXEntityFixturePatch* const * FixturePatchOfDifferentLibraryPtr = Algo::FindByPredicate(FixturePatches, [CommonDMXLibrary](const UDMXEntityFixturePatch* FixturePatch)
		{
			return FixturePatch && FixturePatch->GetParentLibrary() != CommonDMXLibrary;
		});
	if (!ensureMsgf(!FixturePatchOfDifferentLibraryPtr, TEXT("Cannot add Fixture Patches to Pixel Mapping. Patches don't share a common library")))
	{
		const FString PreviousDMXLibraryName = CommonDMXLibrary->GetName();
		const FString FixturePatchOfOtherLibraryName = (*FixturePatchOfDifferentLibraryPtr) ? (*FixturePatchOfDifferentLibraryPtr)->GetName() : TEXT("Invalid Fixture Patch");
		const FString OtherLibraryName = (*FixturePatchOfDifferentLibraryPtr) && (*FixturePatchOfDifferentLibraryPtr)->GetParentLibrary() ? (*FixturePatchOfDifferentLibraryPtr)->GetParentLibrary()->GetName() : TEXT("Invalid DMX Library");

		UE_LOGF(LogDMXPixelMappingEditor, Warning, "Expected DMX Library '%ls', but Fixture Patch '%ls' uses DMX Library '%ls'", *PreviousDMXLibraryName, *FixturePatchOfOtherLibraryName, *OtherLibraryName);
		return;
	}

	UDMXPixelMappingRootComponent* RootComponent = GetPixelMappingRootComponent();
	if (!ensureMsgf(RootComponent, TEXT("Unexpected Pixel Mappinng without root component. Cannot add Fixture Patches to Pixel Mapping.")))
	{
		return;
	}

	const float NumSteps = FixturePatches.Num();
	FScopedSlowTask Task(NumSteps, LOCTEXT("AddFixturePatchesSlowTask", "Updating Pixel Mapping Editor..."));
	Task.MakeDialogDelayed(.5f);

	TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>> Templates;
	for (UDMXEntityFixturePatch* FixturePatch : FixturePatches)
	{
		Task.EnterProgressFrame();

		if (!FixturePatch)
		{
			continue;
		}

		UDMXEntityFixtureType* FixtureType = FixturePatch->GetFixtureType();
		const FDMXFixtureMode* ActiveModePtr = FixturePatch->GetActiveMode();
		if (FixtureType && ActiveModePtr)
		{
			const FDMXEntityFixturePatchRef FixturePatchRef(FixturePatch);
			if (ActiveModePtr->bFixtureMatrixEnabled)
			{
				const TSharedRef<FDMXPixelMappingComponentTemplate> FixturePatchMatrixTemplate = MakeShared<FDMXPixelMappingComponentTemplate>(UDMXPixelMappingMatrixComponent::StaticClass(), FixturePatchRef);
				Templates.Add(FixturePatchMatrixTemplate);
			}
			else
			{
				const TSharedRef<FDMXPixelMappingComponentTemplate> FixturePatchItemTemplate = MakeShared<FDMXPixelMappingComponentTemplate>(UDMXPixelMappingFixtureGroupItemComponent::StaticClass(), FixturePatchRef);
				Templates.Add(FixturePatchItemTemplate);
			}
		}
	}

	const TArray<UDMXPixelMappingBaseComponent*> NewComponents = Toolkit->CreateComponentsFromTemplates(RootComponent, FixtureGroupComponent, Templates);

	// Layout
	const TArray<UDMXEntityFixturePatch*> FixturePatchesInLibrary = CommonDMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	bool bLayoutEvenOverParent = FixturePatchesInLibrary.Num() > 1 && FixturePatchesInLibrary.Num() == FixturePatches.Num();
	if (bLayoutEvenOverParent)
	{
		LayoutEvenOverParent(NewComponents);
	}
	else
	{
		LayoutAfterLastPatch(NewComponents);
	}

	// Select new components
	TSet<FDMXPixelMappingComponentReference> ComponentReferencesToSelect;
	Algo::Transform(NewComponents, ComponentReferencesToSelect,
		[Toolkit](UDMXPixelMappingBaseComponent* Component)
		{
			return FDMXPixelMappingComponentReference(Toolkit, Component);
		});
	Toolkit->SelectComponents(ComponentReferencesToSelect);
}

void UDMXPixelMappingDMXLibraryViewModel::SaveFixturePatchListDescriptor(const FDMXReadOnlyFixturePatchListDescriptor& NewDescriptor)
{
	FixturePatchListDescriptor = NewDescriptor;
	SaveConfig();	
}

void UDMXPixelMappingDMXLibraryViewModel::PostUndo(bool bSuccess)
{
	UpdateDMXLibraryFromComponent();
}

void UDMXPixelMappingDMXLibraryViewModel::PostRedo(bool bSuccess)
{
	UpdateDMXLibraryFromComponent();
}

void UDMXPixelMappingDMXLibraryViewModel::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	TGuardValue<bool> ChangingPropertiesGuard(bChangingProperties, true);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingDMXLibraryViewModel, DMXLibrary))
	{
		if (WeakFixtureGroupComponent.IsValid())
		{
			WeakFixtureGroupComponent->PreEditChange(UDMXPixelMappingFixtureGroupComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupComponent, DMXLibrary)));
			WeakFixtureGroupComponent->DMXLibrary = DMXLibrary;
			WeakFixtureGroupComponent->PostEditChange();

			RemoveInvalidPatches();

			OnDMXLibraryChanged.Broadcast();
		}
	}
}

void UDMXPixelMappingDMXLibraryViewModel::UpdateDMXLibraryFromComponent()
{
	if (bChangingProperties)
	{
		return;
	}

	UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = WeakFixtureGroupComponent.Get();
	if (FixtureGroupComponent)
	{
		Modify();
		DMXLibrary = FixtureGroupComponent->DMXLibrary;
		RemoveInvalidPatches();
	}
}

void UDMXPixelMappingDMXLibraryViewModel::RemoveInvalidPatches()
{
	if (UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = WeakFixtureGroupComponent.Get())
	{
		TArray<UDMXPixelMappingBaseComponent*> CachedChildren(WeakFixtureGroupComponent->Children);
		for (UDMXPixelMappingBaseComponent* ChildComponent : CachedChildren)
		{
			if (UDMXPixelMappingFixtureGroupItemComponent* ChildGroupItem = Cast<UDMXPixelMappingFixtureGroupItemComponent>(ChildComponent))
			{
				if (ChildGroupItem->FixturePatchRef.GetFixturePatch() &&
					ChildGroupItem->FixturePatchRef.GetFixturePatch()->GetParentLibrary() != FixtureGroupComponent->DMXLibrary)
				{
					FixtureGroupComponent->RemoveChild(ChildGroupItem);
				}
			}
			else if (UDMXPixelMappingMatrixComponent* ChildMatrix = Cast<UDMXPixelMappingMatrixComponent>(ChildComponent))
			{
				if (ChildMatrix->FixturePatchRef.GetFixturePatch() &&
					ChildMatrix->FixturePatchRef.GetFixturePatch()->GetParentLibrary() != FixtureGroupComponent->DMXLibrary)
				{
					FixtureGroupComponent->RemoveChild(ChildMatrix);
				}
			}
		};
	}
}

void UDMXPixelMappingDMXLibraryViewModel::LayoutEvenOverParent(const TArray<UDMXPixelMappingBaseComponent*> Components)
{
	UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = WeakFixtureGroupComponent.Get();
	if (!ensureMsgf(FixtureGroupComponent, TEXT("Cannot layout fixture group items over fixture group component. Fixture group component is invalid")))
	{
		return;
	}

	const double RestoreRotation = FixtureGroupComponent->GetRotation();
	FixtureGroupComponent->SetRotation(0.0);

	const int32 Columns = FMath::RoundFromZero(FMath::Sqrt((float)Components.Num()));
	const int32 Rows = FMath::RoundFromZero((float)Components.Num() / Columns);
	const FVector2D Size = FVector2D(FixtureGroupComponent->GetSize().X / Columns, FixtureGroupComponent->GetSize().Y / Rows);

	const FVector2D ParentPosition = FixtureGroupComponent->GetPosition();
	int32 Column = -1;
	int32 Row = 0;
	for (UDMXPixelMappingBaseComponent* Component : Components)
	{
		if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component))
		{
			Column++;
			if (Column >= Columns)
			{
				Column = 0;
				Row++;
			}
			const FVector2D Position = ParentPosition + FVector2D(Column * Size.X, Row * Size.Y);
			OutputComponent->SetPosition(Position);
			OutputComponent->SetSize(Size);
		}
	}

	FixtureGroupComponent->SetRotation(RestoreRotation);
}

void UDMXPixelMappingDMXLibraryViewModel::LayoutAfterLastPatch(const TArray<UDMXPixelMappingBaseComponent*> Components)
{
	if (Components.IsEmpty() || !DMXLibrary)
	{
		return;
	}
	const TArray<UDMXEntityFixturePatch*> Patches = DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>();
	if (Patches.IsEmpty())
	{
		return;
	}

	UDMXPixelMappingOutputComponent* FirstComponentToLayout = Cast<UDMXPixelMappingOutputComponent>(Components[0]);

	UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = WeakFixtureGroupComponent.Get();
	if (!ensureMsgf(FixtureGroupComponent, TEXT("Cannot layout fixture group items over fixture group component. Fixture group component is invalid")))
	{
		return;
	}

	const double RestoreRotation = FixtureGroupComponent->GetRotation();
	FixtureGroupComponent->SetRotation(0.0);

	// Find other components
	TArray<UDMXPixelMappingBaseComponent*> OtherComponentsInGroup;
	for (UDMXPixelMappingBaseComponent* Child : FixtureGroupComponent->GetChildren())
	{
		if (!Components.Contains(Child))
		{
			OtherComponentsInGroup.Add(Child);
		}
	}

	// Find a starting position
	FVector2D NextPosition = FixtureGroupComponent->GetPosition();
	for (UDMXPixelMappingBaseComponent* OtherComponent : OtherComponentsInGroup)
	{
		if (UDMXPixelMappingOutputComponent* OtherOutputComponent = Cast<UDMXPixelMappingOutputComponent>(OtherComponent))
		{
			NextPosition.X = FMath::Max(NextPosition.X, OtherOutputComponent->GetPosition().X + OtherOutputComponent->GetSize().X);
			if (NextPosition.X + FirstComponentToLayout->GetSize().X > FixtureGroupComponent->GetPosition().X + FixtureGroupComponent->GetSize().X)
			{
				NextPosition.X = FixtureGroupComponent->GetPosition().X;
				NextPosition.Y += FirstComponentToLayout->GetSize().Y;
			}
		}
	}

	// Layout
	float RowHeight = 0.f;
	for (UDMXPixelMappingBaseComponent* Component : Components)
	{
		if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component))
		{
			NextPosition = NextPosition - FLT_EPSILON * 2.f;
			OutputComponent->SetPosition(NextPosition);

			if (OutputComponent->IsOverParent())
			{
				RowHeight = FMath::Max(OutputComponent->GetSize().Y, RowHeight);
				NextPosition = FVector2D(NextPosition.X + OutputComponent->GetSize().X, NextPosition.Y);
			}
			else
			{
				// Try on a new row
				FVector2D NewRowPosition = FVector2D(FixtureGroupComponent->GetPosition().X, NextPosition.Y + RowHeight);
				const FVector2D NextPositionOnNewRow = FVector2D(NewRowPosition.X + OutputComponent->GetSize().X, NewRowPosition.Y);

				OutputComponent->SetPosition(NextPositionOnNewRow);

				if (OutputComponent->IsOverParent())
				{
					NextPosition = FVector2D(NewRowPosition.X + OutputComponent->GetSize().X, NewRowPosition.Y);
					RowHeight = OutputComponent->GetSize().Y;
				}
				else
				{
					// Append as the component cannot be fit into the group
					OutputComponent->SetPosition(NextPosition);

					NextPosition = FVector2D(NextPosition.X + OutputComponent->GetSize().X, NextPosition.Y);
				}
			}
		}
	}

	FixtureGroupComponent->SetRotation(RestoreRotation);
}

void UDMXPixelMappingDMXLibraryViewModel::SelectFixtureGroupComponent(UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent)
{
	if (const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin())
	{
		const FDMXPixelMappingComponentReference ComponentRefToSelect(Toolkit, FixtureGroupComponent);
		const TSet<FDMXPixelMappingComponentReference> NewSelection{ ComponentRefToSelect };
		Toolkit->SelectComponents(NewSelection);
	}
}

UDMXPixelMappingRootComponent* UDMXPixelMappingDMXLibraryViewModel::GetPixelMappingRootComponent() const
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.IsValid() ? WeakToolkit.Pin() : nullptr;
	UDMXPixelMapping* PixelMapping = Toolkit.IsValid() ? Toolkit->GetDMXPixelMapping() : nullptr;

	return PixelMapping ? PixelMapping->GetRootComponent() : nullptr;
}

TArray<UDMXPixelMappingFixtureGroupComponent*> UDMXPixelMappingDMXLibraryViewModel::GetFixtureGroupComponentsOfSameLibrary() const
{
	TArray<UDMXPixelMappingFixtureGroupComponent*> FixtureGroupComponents;
	UDMXPixelMappingRendererComponent* RendererComponent = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetActiveRendererComponent() : nullptr;
	if (RendererComponent)
	{
		for (UDMXPixelMappingBaseComponent* Child : RendererComponent->GetChildren())
		{
			if (UDMXPixelMappingFixtureGroupComponent* GroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Child))
			{
				FixtureGroupComponents.Add(GroupComponent);
			}
		}
	}

	return FixtureGroupComponents;
}

#undef LOCTEXT_NAMESPACE
