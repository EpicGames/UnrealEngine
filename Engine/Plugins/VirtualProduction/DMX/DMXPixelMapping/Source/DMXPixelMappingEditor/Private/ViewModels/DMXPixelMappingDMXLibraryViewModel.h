// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXEntityReference.h"
#include "Widgets/SDMXReadOnlyFixturePatchList.h"

#include "EditorUndoClient.h"
#include "Delegates/Delegate.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

#include "DMXPixelMappingDMXLibraryViewModel.generated.h"

class FDMXPixelMappingToolkit;
class UDMXEntityFixturePatch;
class UDMXLibrary;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingFixtureGroupComponent;
class UDMXPixelMappingRootComponent;


/** Model for the DMX Library View */
UCLASS(Config = DMXEditor)
class UDMXPixelMappingDMXLibraryViewModel
	: public UObject
	, public FEditorUndoClient
{
	GENERATED_BODY()

public:
	/** Adds a fixture group to pixel mapping. Sets it for this model */
	void CreateAndSetNewFixtureGroup(TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit);

	/** Sets the fixture group for this model from what's currently selected. Note only single selection is supported as of 5.3 */
	void UpdateFixtureGroupFromSelection(TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit);

	/** Sets if new components should use the patch color */
	void SetNewComponentsUsePatchColor(bool bUsePatchColor);

	/** Returns true if new components should use the patch color */
	bool ShouldNewComponentsUsePatchColor() const;

	/** Adds specified fixture patches to the pixel mapping. Ensures all patches are of the same library as the one of the current group. */
	void AddFixturePatchesEnsured(const TArray<UDMXEntityFixturePatch*>& FixturePatches);

	/** Returns the DMX Library, or nullptr if none or many are selected */
	UDMXLibrary* GetDMXLibrary() const { return DMXLibrary;  }

	/** Returns the Fixture Group, or nullptr if none or many are selected */
	UDMXPixelMappingFixtureGroupComponent* GetFixtureGroupComponent() const { return WeakFixtureGroupComponent.Get(); }

	/** Returns true if more than one fixture group is sected. */
	bool IsMoreThanOneFixtureGroupSelected() const { return bMoreThanOneFixtureGroupSelected; }

	/** Returns the default fixture patch list descriptor */
	const FDMXReadOnlyFixturePatchListDescriptor& GetFixturePatchListDescriptor() const { return FixturePatchListDescriptor; }

	/** Saves the fixture patch list descriptor */
	void SaveFixturePatchListDescriptor(const FDMXReadOnlyFixturePatchListDescriptor& NewDescriptor);

	/** Delegate broadcast when the DMX library changed */
	FSimpleMulticastDelegate OnDMXLibraryChanged;

protected:
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	// Begin UObject interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	// End UObject interface

private:
	/** Updates the DMX Library from the component */
	void UpdateDMXLibraryFromComponent();

	/** Removes invalid patches from pixel mapping */
	void RemoveInvalidPatches();

	/** Selects the fixture group component */
	void SelectFixtureGroupComponent(UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent);

	/** Layouts components evenly over the parent group */
	void LayoutEvenOverParent(const TArray<UDMXPixelMappingBaseComponent*> Components);

	/** Layouts components after the last patch in the parent group */
	void LayoutAfterLastPatch(const TArray<UDMXPixelMappingBaseComponent*> Components);

	/** Returns the root component of the pixel mapping */
	UDMXPixelMappingRootComponent* GetPixelMappingRootComponent() const;

	/** Gets all fixture groups that use the same library as the current fixture group in use */
	TArray<UDMXPixelMappingFixtureGroupComponent*> GetFixtureGroupComponentsOfSameLibrary() const;

	/** The DMX library of this view */
	UPROPERTY(EditAnywhere, Transient, Category = "DMXLibrary", Meta = (AllowPrivateAccess = true))
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** The fixture group component that this model uses */
	UPROPERTY(Transient)
	TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent> WeakFixtureGroupComponent;

	/** Default fixture patch list descriptor */
	UPROPERTY(Config)
	FDMXReadOnlyFixturePatchListDescriptor FixturePatchListDescriptor;

	/** True if more than one fixture group is selected */
	bool bMoreThanOneFixtureGroupSelected = false;

	/** True while changing properties */
	bool bChangingProperties = false;

	/** The toolkit of the editor that uses this model */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
