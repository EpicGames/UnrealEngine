// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/DMXPixelMappingHierarchyItem.h"

#include "Algo/Count.h"
#include "Algo/Reverse.h"
#include "Algo/Sort.h"
#include "Components/DMXPixelMappingOutputComponent.h"
#include "Components/DMXPixelMappingOutputDMXComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DMXPixelMapping.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Toolkits/DMXPixelMappingToolkit.h"


TSharedRef<FDMXPixelMappingHierarchyItem> FDMXPixelMappingHierarchyItem::CreateNew(TWeakPtr<FDMXPixelMappingToolkit> InToolkit)
{
	const TSharedRef<FDMXPixelMappingHierarchyItem> NewObject = MakeShareable(new FDMXPixelMappingHierarchyItem(InToolkit));
	NewObject->Initialize();

	return NewObject;
}

TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> FDMXPixelMappingHierarchyItem::GetItemAndChildrenRecursive()
{
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> Result;
	Result.Add(AsShared());

	for (const TSharedPtr<FDMXPixelMappingHierarchyItem>& Item : Children)
	{
		Result.Append(Item->GetItemAndChildrenRecursive());
	}

	return Result;
}

bool FDMXPixelMappingHierarchyItem::HasEditorColor() const
{
	return WeakComponent.Get() ? WeakComponent->IsA(UDMXPixelMappingOutputDMXComponent::StaticClass()) : false;
}

FLinearColor FDMXPixelMappingHierarchyItem::GetEditorColor() const
{
	if (UDMXPixelMappingOutputDMXComponent* OutputComponent = Cast<UDMXPixelMappingOutputDMXComponent>(WeakComponent.Get()))
	{
		return OutputComponent->GetEditorColor();
	}
	return FLinearColor::Red;
}

FText FDMXPixelMappingHierarchyItem::GetComponentNameText() const
{
	return WeakComponent.IsValid() ?
		FText::FromString(WeakComponent->GetUserName()) :
		FText::GetEmpty();
}

FText FDMXPixelMappingHierarchyItem::GetFixtureIDText() const
{
	const UDMXPixelMappingOutputDMXComponent* OutputDMXComponent = Cast<UDMXPixelMappingOutputDMXComponent>(WeakComponent.Get());
	const UDMXEntityFixturePatch* FixturePatch = OutputDMXComponent ? OutputDMXComponent->FixturePatchRef.GetFixturePatch() : nullptr;

	return FixturePatch ?
		FText::FromString(FString::FromInt(FixturePatch->GetFixtureID())) :
		FText::GetEmpty();
}

FText FDMXPixelMappingHierarchyItem::GetPatchText() const
{
	if (UDMXPixelMappingOutputDMXComponent* OutputDMXComponent = Cast<UDMXPixelMappingOutputDMXComponent>(WeakComponent.Get()))
	{
		if (UDMXEntityFixturePatch* FixturePatch = OutputDMXComponent->FixturePatchRef.GetFixturePatch())
		{
			const FString PatchString = FString::FromInt(FixturePatch->GetUniverseID()) + TEXT(".") + FString::FromInt(FixturePatch->GetStartingChannel());
			return FText::FromString(PatchString);
		}
	}
	return FText::GetEmpty();
}

int64 FDMXPixelMappingHierarchyItem::GetAbsoluteChannel() const
{
	if (UDMXPixelMappingOutputDMXComponent* OutputDMXComponent = Cast<UDMXPixelMappingOutputDMXComponent>(WeakComponent.Get()))
	{
		if (UDMXEntityFixturePatch* FixturePatch = OutputDMXComponent->FixturePatchRef.GetFixturePatch())
		{
			return FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE + FixturePatch->GetStartingChannel();
		}
	}
	return -1;
}

bool FDMXPixelMappingHierarchyItem::IsExpanded() const
{
	if (UDMXPixelMappingBaseComponent* BaseComponent = Cast<UDMXPixelMappingBaseComponent>(WeakComponent.Get()))
	{
		return BaseComponent->bExpanded;
	}

	return false;
}

void FDMXPixelMappingHierarchyItem::SetIsExpanded(bool bExpanded)
{
	UDMXPixelMappingBaseComponent* BaseComponent = Cast<UDMXPixelMappingBaseComponent>(WeakComponent.Get());
	if (BaseComponent && BaseComponent->bExpanded != bExpanded)
	{
		BaseComponent->Modify();
		BaseComponent->bExpanded = bExpanded;
	}
}

void FDMXPixelMappingHierarchyItem::ReverseChildren()
{
	Algo::Reverse(Children);
}

FDMXPixelMappingHierarchyItem::FDMXPixelMappingHierarchyItem(TWeakPtr<FDMXPixelMappingToolkit> InToolkit)
	: WeakToolkit(InToolkit)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	UDMXPixelMapping* DMXPixelMapping = Toolkit->GetDMXPixelMapping();
	if (DMXPixelMapping && DMXPixelMapping->RootComponent)
	{
		WeakComponent = DMXPixelMapping->RootComponent;
	}
}

FDMXPixelMappingHierarchyItem::FDMXPixelMappingHierarchyItem(TWeakPtr<FDMXPixelMappingToolkit> InToolkit, UDMXPixelMappingBaseComponent* InComponent)
	: WeakComponent(InComponent)
	, WeakToolkit(InToolkit)
{}

void FDMXPixelMappingHierarchyItem::Initialize()
{
	BuildChildren();
}

void FDMXPixelMappingHierarchyItem::BuildChildren(UDMXPixelMappingBaseComponent* ParentPixelMappingComponent)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	UDMXPixelMappingBaseComponent* Component = WeakComponent.Get();
	if (!Toolkit.IsValid() || !Component)
	{
		return;
	}

	for (UDMXPixelMappingBaseComponent* ChildComponent : Component->GetChildren())
	{
		if (ChildComponent)
		{
			const TSharedRef<FDMXPixelMappingHierarchyItem> ChildItem = MakeShareable(new FDMXPixelMappingHierarchyItem(Toolkit, ChildComponent));
			ChildItem->Initialize();

			Children.Add(ChildItem);
		}
	}
}
