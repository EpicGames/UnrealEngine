// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Sort.h"
#include "DMXPixelMappingComponentReference.h"

struct FSlateColor;
class UDMXEntityFixturePatch;
class UDMXPixelMappingBaseComponent;


/** 
 * Model for the items in the hierarchy tree view.
 * Children populate by construction and update on changes.
 * Use OnModelChanged to listen to changes.
 */
class FDMXPixelMappingHierarchyItem final
	: public TSharedFromThis<FDMXPixelMappingHierarchyItem>
{
public:
	/** Creates a new instance */
	static TSharedRef<FDMXPixelMappingHierarchyItem> CreateNew(TWeakPtr<FDMXPixelMappingToolkit> InToolkit);

	/** Returns this item and all its children */
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> GetItemAndChildrenRecursive();

	/** Returns true if this row has an editor color */
	bool HasEditorColor() const;

	/** Returns the editor color. Optional return value is only set if the component can display an editor color */
	FLinearColor GetEditorColor() const;

	/** Returns the component name */
	FText GetComponentNameText() const;

	/** Returns the fixture ID text */
	FText GetFixtureIDText() const;

	/** Returns the patch text */
	FText GetPatchText() const;

	/** Gets the absolute channel of the patch of the component. Returns -1 if this item doesn't have a patch */
	int64 GetAbsoluteChannel() const;

	/** Returns true if this item is expanded */
	bool IsExpanded() const;

	/** Sets if the item is expanded */
	void SetIsExpanded(bool bExpanded);

	/** Sorts the children of this item by Projection. */
	template <typename ProjectionType>
	void StableSortChildren(ProjectionType Projection)
	{
		Algo::StableSortBy(Children, Projection);
	}

	/** Reverses the children of this item */
	void ReverseChildren();

	/** Returns the component of this model, or nullptr if the component is no longer valid */
	UDMXPixelMappingBaseComponent* GetComponent() const { return WeakComponent.Get(); }

	/** Returns the child models of this model (the child components of the component of this model) */
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> GetChildren() const { return Children; }

	/** Delegate any object can raise, causes the table row that draws this items to enter rename mode */
	FSimpleMulticastDelegate RequestEnterRenameMode;

private:
	/** Constructs this item from the root component of the toolkit */
	FDMXPixelMappingHierarchyItem(TWeakPtr<FDMXPixelMappingToolkit> InToolkit);

	/** Constructst this item from the specified component in the component hierarchy */
	FDMXPixelMappingHierarchyItem(TWeakPtr<FDMXPixelMappingToolkit> InToolkit, UDMXPixelMappingBaseComponent* InComponentReference);

	/** Common initializer for constructors */
	void Initialize();

	/** Builds the Children member. If ParentPixelMappingComponent uses the root component as parent */
	void BuildChildren(UDMXPixelMappingBaseComponent* ParentPixelMappingComponent = nullptr);

	/** Called when a fixture patch changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

	/** Updates the optional fixture ID */
	void UpdateFixtureID();

	/** The child items in the component hierarchy */
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> Children;

	/** Reference to the component this model handles */
	TWeakObjectPtr<UDMXPixelMappingBaseComponent> WeakComponent;

	/** Toolkit this model resides in */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
