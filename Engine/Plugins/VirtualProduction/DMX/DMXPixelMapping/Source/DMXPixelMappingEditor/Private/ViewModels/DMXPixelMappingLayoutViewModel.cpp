// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/DMXPixelMappingLayoutViewModel.h"

#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMappingEditorLog.h"
#include "LayoutScripts/DMXPixelMappingLayoutScript.h"
#include "Misc/ScopedSlowTask.h"
#include "ScopedTransaction.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingLayoutViewModel"

void UDMXPixelMappingLayoutViewModel::PreEditChange(FProperty* PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (PropertyAboutToChange && PropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingLayoutViewModel, LayoutScriptClass))
	{
		PreEditChangeLayoutScriptClass = LayoutScriptClass;
	}
}

void UDMXPixelMappingLayoutViewModel::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingLayoutViewModel, LayoutScriptClass))
	{
		OnLayoutScriptClassChanged();
	}
}

void UDMXPixelMappingLayoutViewModel::SetToolkit(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;

	RefreshComponents();
	RefreshLayoutScriptClass();

	const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;
	if (DesignerSettings.bApplyLayoutScriptWhenLoaded)
	{
		FScopedTransaction ApplyLayoutScriptTransactionDirect(LOCTEXT("ApplyLayoutScriptTransactionDirect", "Apply Layout Script"));
		ForceApplyLayoutScript();
	}

	InToolkit->GetOnSelectedComponentsChangedDelegate().AddUObject(this, &UDMXPixelMappingLayoutViewModel::OnSelectedComponentsChanged);
}

EDMXPixelMappingLayoutViewModelMode UDMXPixelMappingLayoutViewModel::GetMode() const
{
	// If a Fixture Group is selected, layout its children.
	// If a Matrix is selected, layout its children.
	// If more than one Fixture Group is selected, layout those.
	if (FixtureGroupComponents.Num() == 1)
	{
		return EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren;
	}
	else if (MatrixComponents.Num() == 1 && FixtureGroupComponents.IsEmpty())
	{
		return EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren;
	}
	else if (RendererComponent.IsValid() && FixtureGroupComponents.IsEmpty() && MatrixComponents.IsEmpty())
	{
		return EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren;
	}

	return EDMXPixelMappingLayoutViewModelMode::LayoutNone;
}

TArray<UObject*> UDMXPixelMappingLayoutViewModel::GetLayoutScriptsObjectsSlow() const
{
	TArray<UObject*> Result;

	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren)
	{
		if (RendererComponent.IsValid())
		{
			Result.Add(RendererComponent->LayoutScript);
		}
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren)
	{
		for (TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent> FixtureGroup : FixtureGroupComponents)
		{
			if (FixtureGroup.IsValid() && FixtureGroup->LayoutScript)
			{
				Result.Add(FixtureGroup->LayoutScript);
			}
		}
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren)
	{
		for (TWeakObjectPtr<UDMXPixelMappingMatrixComponent> Matrix : MatrixComponents)
		{
			if (Matrix.IsValid() && Matrix->LayoutScript)
			{
				Result.Add(Matrix->LayoutScript);
			}
		}
	}

	return Result;
}

bool UDMXPixelMappingLayoutViewModel::CanApplyLayoutScript() const
{
	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	return 
		LayoutScriptClass.IsValid() &&
		LayoutMode != EDMXPixelMappingLayoutViewModelMode::LayoutNone;
}

void UDMXPixelMappingLayoutViewModel::RequestApplyLayoutScript()
{
	if (!ApplyLayoutScriptTimerHandle.IsValid())
	{
		ApplyLayoutScriptTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateUObject(this, &UDMXPixelMappingLayoutViewModel::ForceApplyLayoutScript));
	}
}

void UDMXPixelMappingLayoutViewModel::ForceApplyLayoutScript()
{
	ApplyLayoutScriptTimerHandle.Invalidate();

	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren)
	{
		LayoutRendererComponentChildren();
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren)
	{
		LayoutFixtureGroupComponentChildren();
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren)
	{
		LayoutMatrixComponentChildren();
	}
	else
	{
		ensureAlwaysMsgf(LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutNone, TEXT("Unhandled Layout Type, layout cannot be applied"));
	}
}

UDMXPixelMappingOutputComponent* UDMXPixelMappingLayoutViewModel::GetParentComponent() const
{
	// Some ensures here to make sure we rely on the same conditions as what we assume the specific mode means (should correspond to GetMode);
	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren)
	{
		if (!ensureMsgf(RendererComponent.IsValid(), TEXT("GetMode no longer matches assumed conditions.")))
		{
			return nullptr;
		}

		return RendererComponent.Get();
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren)
	{
		if (!ensureMsgf(FixtureGroupComponents.Num() == 1, TEXT("GetMode no longer matches assumed conditions.")))
		{
			return nullptr;
		}

		return FixtureGroupComponents[0].Get();
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren)
	{
		if (!ensureMsgf(MatrixComponents.Num() == 1 && FixtureGroupComponents.IsEmpty(), TEXT("GetMode no longer matches assumed conditions.")))
		{
			return nullptr;
		}

		return MatrixComponents[0].Get();
	}

	return nullptr;
}

void UDMXPixelMappingLayoutViewModel::PostUndo(bool bSuccess)
{
	RefreshComponents();
	RefreshLayoutScriptClass();

	OnModelChanged.Broadcast();
}

void UDMXPixelMappingLayoutViewModel::PostRedo(bool bSuccess)
{
	RefreshComponents();
	RefreshLayoutScriptClass();

	OnModelChanged.Broadcast();
}

void UDMXPixelMappingLayoutViewModel::LayoutRendererComponentChildren()
{
	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (!ensureMsgf(LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren, TEXT("Trying to layout children of Renderer Components, but the current selection does not support this.")))
	{
		return;
	}

	if (!RendererComponent.IsValid())
	{
		return;
	}

	InitializeLayoutScript(RendererComponent.Get(), RendererComponent->LayoutScript);

	TArray<FDMXPixelMappingLayoutToken> InLayoutTokens;
	for (UDMXPixelMappingBaseComponent* ChildComponentBase : RendererComponent->GetChildren())
	{
		if (UDMXPixelMappingOutputComponent* ChildComponent = Cast<UDMXPixelMappingOutputComponent>(ChildComponentBase))
		{
			InLayoutTokens.Add(FDMXPixelMappingLayoutToken(ChildComponent));
		}
	}

	TArray<FDMXPixelMappingLayoutToken> OutLayoutTokens;
	RendererComponent->LayoutScript->Layout(InLayoutTokens, OutLayoutTokens);

	RendererComponent->PreEditChange(UDMXPixelMappingRendererComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, LayoutScript)));
	ApplyLayoutTokens(OutLayoutTokens);
	RendererComponent->PostEditChange();
}

void UDMXPixelMappingLayoutViewModel::LayoutFixtureGroupComponentChildren()
{
	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (!ensureMsgf(!FixtureGroupComponents.IsEmpty() && LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren, TEXT("Trying to layout children of Fixture Group Components, but the current selection does not support this.")))
	{
		return;
	}

	for (TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent> WeakFixtureGroupComponent : FixtureGroupComponents)
	{
		if (UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = WeakFixtureGroupComponent.Get())
		{
			if (!ensureMsgf(FixtureGroupComponent->LayoutScript, TEXT("Missing layout script when trying to apply layout script to Fixture Group Component.")))
			{
				continue;
			}

			// Tokens expect unrotated space
			const double RestoreRotation = FixtureGroupComponent->GetRotation();
			FixtureGroupComponent->SetRotation(0.0);

			InitializeLayoutScript(FixtureGroupComponent, FixtureGroupComponent->LayoutScript);

			TArray<FDMXPixelMappingLayoutToken> InLayoutTokens;
			for (UDMXPixelMappingBaseComponent* ChildComponent : FixtureGroupComponent->GetChildren())
			{
				if (UDMXPixelMappingOutputComponent* ChildOutputComponent = Cast<UDMXPixelMappingOutputComponent>(ChildComponent))
				{
					InLayoutTokens.Add(FDMXPixelMappingLayoutToken(ChildOutputComponent));
				}
			}

			TArray<FDMXPixelMappingLayoutToken> OutLayoutTokens;
			FixtureGroupComponent->LayoutScript->Layout(InLayoutTokens, OutLayoutTokens);

			FixtureGroupComponent->PreEditChange(UDMXPixelMappingFixtureGroupComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupComponent, LayoutScript)));
			ApplyLayoutTokens(OutLayoutTokens);
			FixtureGroupComponent->PostEditChange();

			FixtureGroupComponent->SetRotation(RestoreRotation);
		}
	}
}

void UDMXPixelMappingLayoutViewModel::LayoutMatrixComponentChildren()
{
	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (!ensureMsgf(!MatrixComponents.IsEmpty() && LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren, TEXT("Trying to layout children of Fixture Group Components, but the current selection does not support this.")))
	{
		return;
	}

	for (TWeakObjectPtr<UDMXPixelMappingMatrixComponent> WeakMatrixComponent : MatrixComponents)
	{
		if (UDMXPixelMappingMatrixComponent* MatrixComponent = WeakMatrixComponent.Get())
		{
			if (!ensureMsgf(MatrixComponent->LayoutScript, TEXT("Missing layout script when trying to apply layout script to Matrix Component.")))
			{
				continue;
			}

			// Tokens expect unrotated space
			const double RestoreRotation = MatrixComponent->GetRotation();
			MatrixComponent->SetRotation(0.0);

			InitializeLayoutScript(MatrixComponent, MatrixComponent->LayoutScript);

			TArray<FDMXPixelMappingLayoutToken> InLayoutTokens;
			for (UDMXPixelMappingBaseComponent* ChildComponent : MatrixComponent->GetChildren())
			{
				if (UDMXPixelMappingOutputComponent* ChildOutputComponent = Cast<UDMXPixelMappingOutputComponent>(ChildComponent))
				{
					InLayoutTokens.Add(FDMXPixelMappingLayoutToken(ChildOutputComponent));
				}
			}

			TArray<FDMXPixelMappingLayoutToken> OutLayoutTokens;
			MatrixComponent->LayoutScript->Layout(InLayoutTokens, OutLayoutTokens);

			MatrixComponent->PreEditChange(UDMXPixelMappingMatrixComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, LayoutScript)));
			ApplyLayoutTokens(OutLayoutTokens);
			MatrixComponent->PostEditChange();

			MatrixComponent->SetRotation(RestoreRotation);
		}
	}
}

void UDMXPixelMappingLayoutViewModel::ApplyLayoutTokens(const TArray<FDMXPixelMappingLayoutToken>& LayoutTokens) const
{
	for (const FDMXPixelMappingLayoutToken& LayoutToken : LayoutTokens)
	{
		if (UDMXPixelMappingOutputComponent* OutputComponent = LayoutToken.Component.Get())
		{
			FVector2D Size(LayoutToken.SizeX, LayoutToken.SizeY);
			if (LayoutToken.SizeX <= 0.f || LayoutToken.SizeY <= 0.f)
			{
				UE_LOGF(LogDMXPixelMappingEditor, Warning, "Trying to apply layout script. But layout defines a component size <= 0. This is not supported. Setting a small size instead.");
				Size.X = FMath::Max(UE_SMALL_NUMBER, LayoutToken.SizeX);
				Size.Y = FMath::Max(UE_SMALL_NUMBER, LayoutToken.SizeY);
			}

			OutputComponent->Modify();

			OutputComponent->SetPosition(FVector2D(LayoutToken.PositionX, LayoutToken.PositionY));
			OutputComponent->SetSize(Size);

			// Parent rotation was set to 0 before initialized. Hence the relative rotation is the current rotation.
			OutputComponent->SetRotation(LayoutToken.RelativeRotation);
		}
	}
}

void UDMXPixelMappingLayoutViewModel::OnLayoutScriptClassChanged()
{
	if (PreEditChangeLayoutScriptClass != LayoutScriptClass)
	{
		InstantiateLayoutScripts();

		// Give others time to update before applying the scripts
		OnModelChanged.Broadcast();
	}

	const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;
	if (DesignerSettings.bApplyLayoutScriptWhenLoaded && LayoutScriptClass)
	{
		ForceApplyLayoutScript();
	}
}

void UDMXPixelMappingLayoutViewModel::OnSelectedComponentsChanged()
{
	RefreshComponents();
	RefreshLayoutScriptClass();
	InstantiateLayoutScripts();

	OnModelChanged.Broadcast();
}

void UDMXPixelMappingLayoutViewModel::InstantiateLayoutScripts()
{
	if (!RendererComponent.IsValid())
	{
		return;
	}

	UClass* StrongLayoutScriptClass = LayoutScriptClass.Get();
	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren)
	{
		if (RendererComponent.IsValid() && 
		   (!RendererComponent->LayoutScript ||	RendererComponent->LayoutScript->GetClass() != StrongLayoutScriptClass))
		{
			if (StrongLayoutScriptClass)
			{
				const FName UniqueName = MakeUniqueObjectName(RendererComponent.Get(), UDMXPixelMappingLayoutScript::StaticClass());
				RendererComponent->LayoutScript = NewObject<UDMXPixelMappingLayoutScript>(RendererComponent.Get(), StrongLayoutScriptClass, UniqueName, RF_Transactional | RF_Public);
				InitializeLayoutScript(RendererComponent.Get(), RendererComponent->LayoutScript);
			}
			else
			{
				RendererComponent->LayoutScript = nullptr;
			}
		}
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren)
	{
		const float NumSteps = FixtureGroupComponents.Num();
		FScopedSlowTask Task(NumSteps, LOCTEXT("InstantiateLayoutScripts", "Updating Pixel Mapping Editor..."));
		Task.MakeDialogDelayed(.5f);

		for (TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent> FixtureGroupComponent : FixtureGroupComponents)
		{
			Task.EnterProgressFrame();

			if (FixtureGroupComponent.IsValid() && 
				(!FixtureGroupComponent->LayoutScript || FixtureGroupComponent->LayoutScript->GetClass() != StrongLayoutScriptClass))
			{
				if (StrongLayoutScriptClass)
				{
					const FName UniqueName = MakeUniqueObjectName(FixtureGroupComponent.Get(), UDMXPixelMappingLayoutScript::StaticClass());
					FixtureGroupComponent->LayoutScript = NewObject<UDMXPixelMappingLayoutScript>(FixtureGroupComponent.Get(), StrongLayoutScriptClass, UniqueName, RF_Transactional | RF_Public);
					InitializeLayoutScript(FixtureGroupComponent.Get(), FixtureGroupComponent->LayoutScript);
				}
				else
				{
					FixtureGroupComponent->LayoutScript = nullptr;
				}
			}
		}
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren)
	{
		const float NumSteps = MatrixComponents.Num();
		FScopedSlowTask Task(NumSteps, LOCTEXT("InstantiateLayoutScripts", "Updating Pixel Mapping Editor..."));
		Task.MakeDialogDelayed(.5f);

		for (TWeakObjectPtr<UDMXPixelMappingMatrixComponent> MatrixComponent : MatrixComponents)
		{
			Task.EnterProgressFrame();

			if (MatrixComponent.IsValid() &&
			   (!MatrixComponent->LayoutScript || MatrixComponent->LayoutScript->GetClass() != StrongLayoutScriptClass))
			{
				if (StrongLayoutScriptClass)
				{
					const FName UniqueName = MakeUniqueObjectName(MatrixComponent.Get(), UDMXPixelMappingLayoutScript::StaticClass());
					MatrixComponent->LayoutScript = NewObject<UDMXPixelMappingLayoutScript>(MatrixComponent.Get(), StrongLayoutScriptClass, UniqueName, RF_Transactional | RF_Public);
					InitializeLayoutScript(MatrixComponent.Get(), MatrixComponent->LayoutScript);
				}
				else
				{
					MatrixComponent->LayoutScript = nullptr;
				}
			}
		}
	}
	else
	{
		ensureMsgf(LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutNone, TEXT("Unhandled Layout Type, layout cannot be applied"));
	}
}

void UDMXPixelMappingLayoutViewModel::InitializeLayoutScript(UDMXPixelMappingOutputComponent* OutputComponent, UDMXPixelMappingLayoutScript* LayoutScript)
{
	if (!ensureMsgf(OutputComponent && LayoutScript && OutputComponent->GetRendererComponent(), TEXT("Invalid component, layout script or renderer component when trying to initialize layut script. layout cannot be applied")))
	{
		return;
	}

	LayoutScript->SetNumTokens(OutputComponent->GetChildren().Num());
	LayoutScript->SetParentComponentPosition(OutputComponent->GetPosition());
	LayoutScript->SetParentComponentSize(OutputComponent->GetSize());
	LayoutScript->SetParentComponentRotation(OutputComponent->GetRotation());
	LayoutScript->SetTextureSize(OutputComponent->GetRendererComponent()->GetSize());
}

void UDMXPixelMappingLayoutViewModel::RefreshComponents()
{
	// Reset
	RendererComponent = nullptr;
	FixtureGroupComponents.Reset();
	MatrixComponents.Reset();

	// Find relevant components
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	RendererComponent = Toolkit->GetActiveRendererComponent();
	if (!RendererComponent.IsValid())
	{
		return;
	}

	const TSet<FDMXPixelMappingComponentReference> ComponentsToLayout = Toolkit->GetSelectedComponents();

	for (const FDMXPixelMappingComponentReference& ComponentReference : ComponentsToLayout)
	{
		if (UDMXPixelMappingBaseComponent* Component = ComponentReference.GetComponent())
		{
			if (UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Component))
			{
				FixtureGroupComponents.Add(FixtureGroupComponent);
			}
			else if (UDMXPixelMappingFixtureGroupItemComponent* FixtureGroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(Component))
			{
				// If Fixture Group Items are selected, use their parent Group instead.
				if (UDMXPixelMappingFixtureGroupComponent* ParentFixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(FixtureGroupItemComponent->GetParent()))
				{
					FixtureGroupComponents.AddUnique(ParentFixtureGroupComponent);
				}
			}
			else if (UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(Component))
			{
				MatrixComponents.Add(MatrixComponent);
			}
			else if (UDMXPixelMappingMatrixCellComponent* MatrixCellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(Component))
			{					
				// If Matrix Cells are selected, use their parent Matrix instead.
				if (UDMXPixelMappingMatrixComponent* ParentMatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(MatrixCellComponent->GetParent()))
				{
					MatrixComponents.AddUnique(ParentMatrixComponent);
				}
			}
		}
	}
}

void UDMXPixelMappingLayoutViewModel::RefreshLayoutScriptClass()
{
	LayoutScriptClass = nullptr;

	const EDMXPixelMappingLayoutViewModelMode LayoutMode = GetMode();
	if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutRendererComponentChildren)
	{
		if (RendererComponent.IsValid() && RendererComponent->LayoutScript)
		{
			LayoutScriptClass =RendererComponent->LayoutScript->GetClass();
		}
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutFixtureGroupComponentChildren)
	{
		for (TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent> FixtureGroup : FixtureGroupComponents)
		{
			if (!FixtureGroup.IsValid())
			{
				continue;
			}

			if (FixtureGroup->LayoutScript && LayoutScriptClass && FixtureGroup->LayoutScript->GetClass() != LayoutScriptClass)
			{
				// Multiple values
				LayoutScriptClass = nullptr;
				return;
			}
			else if (FixtureGroup->LayoutScript)
			{
				LayoutScriptClass = FixtureGroup->LayoutScript->GetClass();
			}
		}
	}
	else if (LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutMatrixComponentChildren)
	{
		for (TWeakObjectPtr<UDMXPixelMappingMatrixComponent> Matrix : MatrixComponents)
		{
			if (!Matrix.IsValid())
			{
				continue;
			}

			if (Matrix->LayoutScript && LayoutScriptClass && Matrix->LayoutScript->GetClass() != LayoutScriptClass)
			{
				// Multiple values
				LayoutScriptClass = nullptr;
				return;
			}
			else if (Matrix->LayoutScript)
			{
				LayoutScriptClass = Matrix->LayoutScript->GetClass();
			}
		}
	}
	else
	{
		ensureMsgf(LayoutMode == EDMXPixelMappingLayoutViewModelMode::LayoutNone, TEXT("Unhandled Layout Type, layout cannot be applied"));
	}
}

#undef LOCTEXT_NAMESPACE
