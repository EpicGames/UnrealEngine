// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Engine/TimerHandle.h"

#include "DMXPixelMappingLayoutViewModel.generated.h"

struct FDMXPixelMappingLayoutToken;
class FDMXPixelMappingToolkit;
class UDMXPixelMappingFixtureGroupComponent;
class UDMXPixelMappingFixtureGroupItemComponent;
class UDMXPixelMappingLayoutScript;
class UDMXPixelMappingMatrixComponent;
class UDMXPixelMappingOutputComponent;
class UDMXPixelMappingRendererComponent;

enum class EDMXPixelMappingLayoutViewModelMode
{
	LayoutRendererComponentChildren,
	LayoutFixtureGroupComponentChildren,
	LayoutMatrixComponentChildren,
	LayoutNone
};

/** Model for the Layout View */
UCLASS(Transient)
class UDMXPixelMappingLayoutViewModel
	: public UObject
	, public FSelfRegisteringEditorUndoClient
{
	GENERATED_BODY()

protected:
#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	/** Sets the toolkit from which the model sources */
	void SetToolkit(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit);

	/** Returns the mode in which this model currently can work in */
	EDMXPixelMappingLayoutViewModelMode GetMode() const;

	/** Returns the Layout Scripts of relevant components. Looks up each relevant component and returns its script. */
	TArray<UObject*> GetLayoutScriptsObjectsSlow() const;

	/** Returns true if a layout script can be applied. */
	bool CanApplyLayoutScript() const;

	/** Applies the layouts script on the next tick. Test if CanApplyLayoutScript before calling (ensured). */
	void RequestApplyLayoutScript();

	/** Applies the layouts script. Test if CanApplyLayoutScript before calling (ensured). */
	void ForceApplyLayoutScript();

	/** Returns true if the layout script should be auto-applied on property changes */
	bool ShouldAutoApplyLayoutScript() const { return bAutoApply; }

	/** Returns the parent of the components being laid out */
	UDMXPixelMappingOutputComponent* GetParentComponent() const;

	/** Delegate executed when the Model changed */
	FSimpleMulticastDelegate OnModelChanged;

	// Property name getters
	static FName GetAutoApplyPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingLayoutViewModel, bAutoApply); }
	static FName GetLayoutScriptClassPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingLayoutViewModel, LayoutScriptClass); }

protected:
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

private:
	/** Lays out children of the Renderer Component */
	void LayoutRendererComponentChildren();

	/** Lays out children of the Fixture Group Components */
	void LayoutFixtureGroupComponentChildren();

	/** Lays out children of the Matrixx Components */
	void LayoutMatrixComponentChildren();

	/** Applies the layout of the specified tokens */
	void ApplyLayoutTokens(const TArray<FDMXPixelMappingLayoutToken>& LayoutTokens) const;

	/** Called when the Layout Script Class of this Model changed */
	void OnLayoutScriptClassChanged();

	/** Called when selected Components changed */
	void OnSelectedComponentsChanged();

	/** Instantiates layout scripts for relevant components */
	void InstantiateLayoutScripts();

	/** Initializes memeber of the Layout Script in specified Component */
	void InitializeLayoutScript(UDMXPixelMappingOutputComponent* OutputComponent, UDMXPixelMappingLayoutScript* LayoutScript);

	/** Refreshes components from what's currently selected in the Pixel Mapping Toolkit */
	void RefreshComponents();

	/** Refreshes the Layout Script Class */
	void RefreshLayoutScriptClass();

	/** The toolkit from which the Model currently sources */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;

	/** The Renderer Component currently in use */
	UPROPERTY()
	TWeakObjectPtr<UDMXPixelMappingRendererComponent> RendererComponent;

	/** The Fixture Group Components currently in use */
	UPROPERTY()
	TArray<TWeakObjectPtr<UDMXPixelMappingFixtureGroupComponent>> FixtureGroupComponents;

	/** The Matrix Components currently in use */
	UPROPERTY()
	TArray<TWeakObjectPtr<UDMXPixelMappingMatrixComponent>> MatrixComponents;

	/** If true, the script is auto applied on property changes */
	UPROPERTY(EditAnywhere, Category = "Layout Script", Meta = (DisplayName = "Apply Script on Property Changes"))
	bool bAutoApply = true;

	/** The Layout Script class currently in use. Note, custom scripts can be blueprinted. */
	UPROPERTY(EditAnywhere, Category = "Layout Script", Meta = (ShowDisplayNames))
	TSoftClassPtr<UDMXPixelMappingLayoutScript> LayoutScriptClass;

	/** Timer handle to apply the layout script, set when ApplyLayoutScript is requested. */
	FTimerHandle ApplyLayoutScriptTimerHandle;

	/** The Layout Script class currently before it was changed */
	UPROPERTY(Transient)
	TSoftClassPtr<UDMXPixelMappingLayoutScript> PreEditChangeLayoutScriptClass;
};
