// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/DMXPixelMappingOutputComponentModel.h"

#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingOutputComponent.h"
#include "DMXPixelMappingTypes.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingOutputComponentModel"

namespace UE::DMX
{
	FDMXPixelMappingOutputComponentModel::FDMXPixelMappingOutputComponentModel(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingOutputComponent> InOutputComponent)
		: WeakToolkit(InToolkit)
	{
		if (!InOutputComponent.IsValid())
		{
			return;
		}
		OutputComponent = InOutputComponent.Get();

		UpdateFixtureNode();
		bSelected = InToolkit->GetSelectedComponents().Contains(FDMXPixelMappingComponentReference(InToolkit, OutputComponent));

		// Handle selection changes
		InToolkit->GetOnSelectedComponentsChangedDelegate().AddRaw(this, &FDMXPixelMappingOutputComponentModel::OnSelectedComponentsChanged);
	}

	FDMXPixelMappingOutputComponentModel::~FDMXPixelMappingOutputComponentModel()
	{
		if (TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin())
		{
			Toolkit->GetOnSelectedComponentsChangedDelegate().RemoveAll(this);
		}
	}

	FVector2D FDMXPixelMappingOutputComponentModel::GetPosition() const
	{
		if (OutputComponent)
		{
			return OutputComponent->GetPosition();
		}
		return FVector2D::ZeroVector;
	}

	FVector2D FDMXPixelMappingOutputComponentModel::GetSize() const
	{
		if (OutputComponent)
		{
			return OutputComponent->GetSize();
		}
		return FVector2D::ZeroVector;
	}

	FQuat2D FDMXPixelMappingOutputComponentModel::GetQuaternion() const
	{
		if (OutputComponent)
		{
			const FQuat2D Quat(FMath::DegreesToRadians(OutputComponent->GetRotation()));
			return Quat;
		}
		return FQuat2D();
	}

	FText FDMXPixelMappingOutputComponentModel::GetName() const
	{
		if (OutputComponent)
		{
			return FText::FromString(OutputComponent->GetUserName());
		}

		return FText::GetEmpty();
	}

	bool FDMXPixelMappingOutputComponentModel::ShouldDraw() const
	{
		if (OutputComponent)
		{
			const bool bIsVisble = OutputComponent->IsVisible();
			if (UDMXPixelMappingMatrixCellComponent* CellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(OutputComponent))
			{
				const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;
				return bIsVisble && DesignerSettings.bShowMatrixCells;
			}

			return bIsVisble;
		}

		return true;
	}

	bool FDMXPixelMappingOutputComponentModel::ShouldDrawName() const
	{
		const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;

		return 
			DesignerSettings.bShowComponentNames &&
			OutputComponent &&
			OutputComponent->GetClass() != UDMXPixelMappingMatrixCellComponent::StaticClass();
	}

	bool FDMXPixelMappingOutputComponentModel::ShouldDrawNameAbove() const
	{
		return
			OutputComponent && OutputComponent->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass();
	}

	bool FDMXPixelMappingOutputComponentModel::ShouldDrawCellID() const
	{
		const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;

		return
			DesignerSettings.bShowCellIDs &&
			OutputComponent &&
			OutputComponent->GetClass() == UDMXPixelMappingMatrixCellComponent::StaticClass();
	}

	bool FDMXPixelMappingOutputComponentModel::ShouldDrawPatchInfo() const
	{
		const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;

		return
			DesignerSettings.bShowPatchInfo &&
			GetFixturePatch() != nullptr;
	}

	bool FDMXPixelMappingOutputComponentModel::ShouldDrawPivot() const
	{
		const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;

		return 
			bSelected &&
			DesignerSettings.bShowPivot;
	}

	FText FDMXPixelMappingOutputComponentModel::GetAddressesText() const
	{
		if (const UDMXEntityFixturePatch* FixturePatch = GetFixturePatch())
		{
			const FString AddressesString = FString::Printf(TEXT("%i.%i"), FixturePatch->GetUniverseID(), FixturePatch->GetStartingChannel());

			return FText::Format(LOCTEXT("PatchTextWithLabel", "Patch: {0}"), FText::FromString(AddressesString));
		}

		return FText::GetEmpty();
	}

	FText FDMXPixelMappingOutputComponentModel::GetFixtureIDText() const
	{
		if (UDMXMVRFixtureNode* FixtureNode = WeakFixtureNode.Get())
		{
			const FText FixtureIDText = FixtureNode->FixtureID.IsEmpty() ? LOCTEXT("NoFixtureID", "<not set>") : FText::FromString(FixtureNode->FixtureID);
			return FText::Format(LOCTEXT("FixtureIDTextWithLabel", "FID: {0}"), FixtureIDText);
		}

		return FText::GetEmpty();
	}

	FText FDMXPixelMappingOutputComponentModel::GetCellIDText() const
	{
		if (UDMXPixelMappingMatrixCellComponent* MatrixCell = Cast<UDMXPixelMappingMatrixCellComponent>(OutputComponent))
		{
			return FText::FromString(FString::FromInt(MatrixCell->CellID));
		}

		return FText::GetEmpty();
	}

	FLinearColor FDMXPixelMappingOutputComponentModel::GetColor() const
	{
		if (OutputComponent)
		{
			if (OutputComponent->IsOverParent())
			{
				if (bSelected)
				{
					return OutputComponent->GetEditorColor();
				}
				else
				{
					const float Opacity = OutputComponent->GetEditorColor().A * .5f;
					return OutputComponent->GetEditorColor().CopyWithNewOpacity(Opacity);
				}
			}
		}

		return FLinearColor::Red;
	}

	bool FDMXPixelMappingOutputComponentModel::Equals(UDMXPixelMappingBaseComponent* Other) const
	{
		return OutputComponent && OutputComponent == Other;
	}

	void FDMXPixelMappingOutputComponentModel::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(OutputComponent);
	}

	void FDMXPixelMappingOutputComponentModel::OnSelectedComponentsChanged()
	{
		TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
		if (!Toolkit.IsValid())
		{
			return;
		}

		if (OutputComponent)
		{
			bSelected = Toolkit->GetSelectedComponents().Contains(FDMXPixelMappingComponentReference(Toolkit, OutputComponent));
		}
	}

	const UDMXEntityFixturePatch* FDMXPixelMappingOutputComponentModel::GetFixturePatch() const
	{
		const UDMXEntityFixturePatch* FixturePatch = nullptr;
		if (UDMXPixelMappingMatrixComponent* Matrix = Cast<UDMXPixelMappingMatrixComponent>(OutputComponent))
		{
			FixturePatch = Matrix->FixturePatchRef.GetFixturePatch();
		}
		else if (UDMXPixelMappingFixtureGroupItemComponent* GroupItem = Cast<UDMXPixelMappingFixtureGroupItemComponent>(OutputComponent))
		{
			FixturePatch = GroupItem->FixturePatchRef.GetFixturePatch();
		}

		return FixturePatch;
	}

	void FDMXPixelMappingOutputComponentModel::UpdateFixtureNode()
	{
		WeakFixtureNode.Reset();

		const UDMXEntityFixturePatch* FixturePatch = GetFixturePatch();
		if (!FixturePatch)
		{
			return;
		}

		const FGuid& MVRUUID = FixturePatch->GetMVRFixtureUUID();
		if (!MVRUUID.IsValid())
		{
			return;
		}

		const UDMXLibrary* DMXLibrary = FixturePatch->GetParentLibrary();
		if (!DMXLibrary)
		{
			return;
		}

		// We can use the lazy General Scene Description
		UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary->GetLazyGeneralSceneDescription();
		if (!GeneralSceneDescription)
		{
			return;
		}

		WeakFixtureNode = GeneralSceneDescription->FindFixtureNode(MVRUUID);
	}
}

#undef LOCTEXT_NAMESPACE
