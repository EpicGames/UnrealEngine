// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/TransformCalculus2D.h"
#include "Math/Vector2D.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

enum class EDMXPixelMappingDistribution : uint8;
enum class EDMXCellFormat : uint8;
class FDMXPixelMappingToolkit;
class UDMXEntityFixturePatch;
class UDMXMVRFixtureNode;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingOutputComponent;


namespace UE::DMX
{
	/** Model for the Pixel Mapping Output Components. For Screen Component, see DMXPixelMappingScreenComponentModel. */
	class FDMXPixelMappingOutputComponentModel
		: public FGCObject
		, public TSharedFromThis<FDMXPixelMappingOutputComponentModel>
	{
	public:
		/** Constructor */
		FDMXPixelMappingOutputComponentModel(const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingOutputComponent> InOutputComponent);

		/** Destructor */
		virtual ~FDMXPixelMappingOutputComponentModel();

		/** Returns the position of the component */
		FVector2D GetPosition() const;

		/** Returns the size of the component */
		FVector2D GetSize() const;

		/** Returns the rotation, in degrees */
		FQuat2D GetQuaternion() const;
		
		/** Returns the Name of the component */
		FText GetName() const;

		/** Returns true if a widget ever should be drawn */
		bool ShouldDraw() const;

		/** Returns true if the name should drawn its name */
		bool ShouldDrawName() const;

		/** Returns true if the name should drawn its name above the widget */
		bool ShouldDrawNameAbove() const;

		/** Returns true if the cell ID should be shown */
		bool ShouldDrawCellID() const;

		/** Returns true if the patch info should be shown */
		bool ShouldDrawPatchInfo() const;

		/** Returns true if the pivot should be shown */
		bool ShouldDrawPivot() const;

		/** Returns the addresses of the patch as text */
		FText GetAddressesText() const;

		/** Returns the Fixture ID as text. Empty unless a Matrix and Fixture Group Item component */
		FText GetFixtureIDText() const;

		/** Returns the Cell ID as text. Empty unless a Matrix Cell component. */
		FText GetCellIDText() const;

		/** Returns the color of the widget */
		FLinearColor GetColor() const;

		/** Returns true if this Model handles the Other component */
		bool Equals(UDMXPixelMappingBaseComponent* Other) const;

	protected:
		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("FDMXPixelMappingOutputComponentModel");
		}
		//~ End FGCObject interface

	private:
		/** Called when selected Components changed */
		void OnSelectedComponentsChanged();

		/** Returns the Fixture Patch of the component, or nullptr if the component does not use a Fixture Patch. */
		const UDMXEntityFixturePatch* GetFixturePatch() const;

		/** Updates the Fixture Node that correlates to this component */
		void UpdateFixtureNode();

		/** The Fixture Node that corresponds to this component, only valid for Group Item and Matrix */
		TWeakObjectPtr<UDMXMVRFixtureNode> WeakFixtureNode;

		/** True if the component is selected */
		bool bSelected = false;

		/** The actual output component */
		TObjectPtr<UDMXPixelMappingOutputComponent> OutputComponent;

		/** The toolkit from which the Model currently sources */
		TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
	};
}
