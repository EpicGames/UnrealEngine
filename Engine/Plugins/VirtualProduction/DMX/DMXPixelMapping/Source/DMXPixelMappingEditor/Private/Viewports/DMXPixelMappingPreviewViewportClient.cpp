// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewports/DMXPixelMappingPreviewViewportClient.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMappingRenderElement.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Slate/SceneViewport.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "UnrealClient.h"
#include "Views/SDMXPixelMappingPreviewView.h"
#include "Widgets/SDMXPixelMappingPreviewViewport.h"


FDMXPixelMappingPreviewViewportClient::FDMXPixelMappingPreviewViewportClient(TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit, TWeakPtr<SDMXPixelMappingPreviewViewport> InWeakViewport)
	: WeakToolkit(InWeakToolkit)
	, WeakViewport(InWeakViewport)
{}

bool FDMXPixelMappingPreviewViewportClient::IsDrawingVisibleRectOnly() const
{
	if (!WeakToolkit.IsValid() || !WeakViewport.IsValid())
	{
		return false;
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();
	const TSharedRef<SDMXPixelMappingPreviewView> PreviewView = Toolkit->GetOrCreatePreviewView();

	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (!InputTexture)
	{
		return false;
	}

	const int32 ExcessThreshold = GMaxTextureDimensions / 4;

	return
		InputTexture->GetSurfaceWidth() * PreviewView->GetZoomAmount() > ExcessThreshold ||
		InputTexture->GetSurfaceHeight() * PreviewView->GetZoomAmount() > ExcessThreshold;
}

FBox2D FDMXPixelMappingPreviewViewportClient::GetVisibleTextureBoxGraphSpace() const
{
	if (!WeakToolkit.IsValid())
	{
		return FBox2D();
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();
	const TSharedRef<SDMXPixelMappingPreviewView> PreviewView = Toolkit->GetOrCreatePreviewView();

	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (!InputTexture || !InputTexture->GetResource())
	{
		return FBox2D();
	}

	// Compute texture box in graph space
	const FVector2D TextureDimensions = FVector2D(InputTexture->GetSurfaceWidth(), InputTexture->GetSurfaceHeight());
	const FBox2D TextureBox(FVector2D::ZeroVector, TextureDimensions);

	// Compute graph box in graph space
	const FGeometry& GraphGeometry = PreviewView->GetGraphTickSpaceGeometry();
	const FVector2D GraphSize = GraphGeometry.GetAbsoluteSize() / PreviewView->GetZoomAmount();
	const FVector2D ViewOffset = PreviewView->GetViewOffset();
	const FBox2D GraphBox(ViewOffset, ViewOffset + GraphSize);

	return TextureBox.Overlap(GraphBox);
}

void FDMXPixelMappingPreviewViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	check(IsInGameThread());

	Canvas->Clear(FColor::Black);

	if (!WeakToolkit.IsValid())
	{
		return;
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (!InputTexture || !InputTexture->GetResource())
	{
		return;
	}

	const TSharedPtr<SDMXPixelMappingPreviewViewport> ViewportWidget = WeakViewport.Pin();
	if (!ViewportWidget.IsValid())
	{
		return;
	}

	const TSharedPtr<FSceneViewport> SceneViewport = ViewportWidget->GetViewport();
	if (!SceneViewport.IsValid())
	{
		return;
	}

	// Draw the current render elements
	const bool bDrawVisibleRectOnly = IsDrawingVisibleRectOnly();

	using namespace UE::DMXPixelMapping::Rendering;
	const TArray<TSharedRef<FPixelMapRenderElement>> RenderElements = RendererComponent->GetPixelMapRenderElements();
	for (const TSharedRef<FPixelMapRenderElement>& Element : RenderElements)
	{
		const FVector2D UV = Element->GetParameters().UV;
		const FVector2D UVSize = Element->GetParameters().UVSize;

		if (bDrawVisibleRectOnly)
		{
			const FVector2D PositionGraphSpace = UV * FVector2D{ InputTexture->GetSurfaceWidth(), InputTexture->GetSurfaceHeight() };
			const FVector2D SizeGraphSpace = UVSize * FVector2D{ InputTexture->GetSurfaceWidth(), InputTexture->GetSurfaceHeight() };
			
			const FBox2D ComponentBox(PositionGraphSpace, PositionGraphSpace + SizeGraphSpace);
			const FBox2D VisibleTextureBox = GetVisibleTextureBoxGraphSpace();
			const FBox2D VisibleComponentBox = VisibleTextureBox.Overlap(ComponentBox);

			if (ComponentBox.bIsValid)
			{
				const TSharedRef<SDMXPixelMappingPreviewView> PreviewView = Toolkit->GetOrCreatePreviewView();
				const FVector2D Position = (VisibleComponentBox.Min - VisibleTextureBox.Min) * PreviewView->GetZoomAmount();
				const FVector2D Size = VisibleComponentBox.GetSize() * PreviewView->GetZoomAmount();

				FCanvasTileItem TileItem(Position, Size, Element->GetColor());
				TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_MAX;
				TileItem.PivotPoint = FVector2D(0.5, 0.5);
				TileItem.Rotation = FRotator(0.0, Element->GetParameters().Rotation, 0.0);
				Canvas->DrawItem(TileItem);
			}
		}
		else
		{
			const FVector2D Position = UV * InViewport->GetSizeXY();
			const FVector2D Size = UVSize * InViewport->GetSizeXY();

			FCanvasTileItem TileItem(Position, Size, Element->GetColor());
			TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_MAX;
			TileItem.PivotPoint = FVector2D(0.5, 0.5);
			TileItem.Rotation = FRotator(0.0, Element->GetParameters().Rotation, 0.0);
			Canvas->DrawItem(TileItem);
		}
	}

	Canvas->Flush_GameThread();
}
