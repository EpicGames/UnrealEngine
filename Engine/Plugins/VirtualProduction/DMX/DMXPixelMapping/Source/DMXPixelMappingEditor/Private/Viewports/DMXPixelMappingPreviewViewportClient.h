// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportClient.h"

class FDMXPixelMappingToolkit;
class SDMXPixelMappingPreviewViewport;


class FDMXPixelMappingPreviewViewportClient
	: public FViewportClient
{
public:
	/** Constructor */
	FDMXPixelMappingPreviewViewportClient(TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit, TWeakPtr<SDMXPixelMappingPreviewViewport> InWeakViewport);

	/** If true, the widget draws the visible rect of the source texture */
	bool IsDrawingVisibleRectOnly() const;

	/** Returns the visible texture size in graph space */
	FBox2D GetVisibleTextureBoxGraphSpace() const;

protected:
	//~ Begin FViewportClient Interface
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	//~ End FViewportClient Interface

private:
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
	TWeakPtr<SDMXPixelMappingPreviewViewport> WeakViewport;
};
