// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Slate/SceneViewport.h"

/*
 * Pixel Mapping Scene Viewport disable viewport input handle, it let use a custom handle in designer
 */
class FDMXPixelMappingSceneViewport :
	public FSceneViewport
{
public:
	FDMXPixelMappingSceneViewport(const TSharedPtr<SViewport>& InViewportWidget)
		: FSceneViewport(InViewportWidget)
	{
	}

	static TSharedRef<FDMXPixelMappingSceneViewport> Create(const TSharedPtr<FViewportClient>& InViewportClient, const TSharedPtr<SViewport>& InViewportWidget)
	{
		return FSceneViewport::Create<FDMXPixelMappingSceneViewport>(InViewportClient, InViewportWidget);
	}

protected:
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& MouseEvent) override
	{
		FSceneViewport::OnMouseButtonDown(InGeometry, MouseEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& MouseEvent) override
	{
		FSceneViewport::OnMouseButtonDown(InGeometry, MouseEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnMouseMove(const FGeometry& InGeometry, const FPointerEvent& MouseEvent) override
	{
		FSceneViewport::OnMouseButtonDown(InGeometry, MouseEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnMouseWheel(const FGeometry& InGeometry, const FPointerEvent& MouseEvent) override
	{
		FSceneViewport::OnMouseWheel(InGeometry, MouseEvent);

		return FReply::Handled();
	}

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override
	{
		FSceneViewport::OnMouseButtonDoubleClick(InGeometry, InMouseEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnTouchStarted(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		FSceneViewport::OnTouchStarted(MyGeometry, InTouchEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnTouchMoved(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		FSceneViewport::OnTouchMoved(MyGeometry, InTouchEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override
	{
		FSceneViewport::OnTouchEnded(MyGeometry, InTouchEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnTouchForceChanged(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent) override
	{
		FSceneViewport::OnTouchForceChanged(MyGeometry, TouchEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnTouchFirstMove(const FGeometry& MyGeometry, const FPointerEvent& TouchEvent) override
	{
		FSceneViewport::OnTouchFirstMove(MyGeometry, TouchEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& InGestureEvent) override
	{
		FSceneViewport::OnTouchGesture(MyGeometry, InGestureEvent);

		return FReply::Unhandled();
	}

	virtual FReply OnMotionDetected(const FGeometry& MyGeometry, const FMotionEvent& InMotionEvent) override
	{
		FSceneViewport::OnMotionDetected(MyGeometry, InMotionEvent);

		return FReply::Unhandled();
	}
};
