// Copyright Epic Games, Inc. All Rights Reserved.

#include "Viewports/DMXPixelMappingSourceTextureViewportClient.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMapping.h"
#include "Engine/Texture.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/SceneViewport.h"
#include "Texture2DPreview.h"
#include "TextureResource.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Views/SDMXPixelMappingDesignerView.h"
#include "Widgets/SDMXPixelMappingSourceTextureViewport.h"


FDMXPixelMappingSourceTextureViewportClient::FDMXPixelMappingSourceTextureViewportClient(const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit, TWeakPtr<SDMXPixelMappingSourceTextureViewport> InSourceTextureViewport)
	: WeakToolkit(InToolkit)
	, WeakSourceTextureViewport(InSourceTextureViewport)
{}

bool FDMXPixelMappingSourceTextureViewportClient::IsDrawingVisibleRectOnly() const
{
	if (!WeakToolkit.IsValid())
	{
		return false;
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();
	const TSharedRef<SDMXPixelMappingDesignerView> DesignerView = Toolkit->GetOrCreateDesignerView();

	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (!InputTexture)
	{
		return false;
	}

	const int32 ExcessThreshold = GMaxTextureDimensions / 4;

	return
		InputTexture->GetSurfaceWidth() * DesignerView->GetZoomAmount() > ExcessThreshold ||
		InputTexture->GetSurfaceHeight() * DesignerView->GetZoomAmount() > ExcessThreshold;
}

FBox2D FDMXPixelMappingSourceTextureViewportClient::GetVisibleTextureBoxGraphSpace() const
{
	if (!WeakToolkit.IsValid())
	{
		return FBox2D();
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();
	const TSharedRef<SDMXPixelMappingDesignerView> DesignerView = Toolkit->GetOrCreateDesignerView();

	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (!InputTexture || !InputTexture->GetResource())
	{
		return FBox2D();
	}

	// Compute texture box in graph space
	const FVector2D TextureDimensions = FVector2D(InputTexture->GetSurfaceWidth(), InputTexture->GetSurfaceHeight());
	const FBox2D TextureBox(FVector2D::ZeroVector, TextureDimensions);

	// Compute graph box in graph space
	const FGeometry& GraphGeometry = DesignerView->GetGraphTickSpaceGeometry();
	const FVector2D GraphSize = GraphGeometry.GetAbsoluteSize() / DesignerView->GetZoomAmount();
	const FVector2D ViewOffset = DesignerView->GetViewOffset();
	const FBox2D GraphBox(ViewOffset, ViewOffset + GraphSize);

	return TextureBox.Overlap(GraphBox);
}

void FDMXPixelMappingSourceTextureViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	Canvas->Clear(FColor::Transparent);

	if (!WeakToolkit.IsValid())
	{
		return;
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

	UDMXPixelMapping* PixelMapping = Toolkit->GetDMXPixelMapping();
	if (!PixelMapping)
	{
		return;
	}

	const TSharedRef<SDMXPixelMappingDesignerView> DesignerView = Toolkit->GetOrCreateDesignerView();
	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	if (!InputTexture || !InputTexture->GetResource())
	{
		return;
	}

	const TSharedPtr<SDMXPixelMappingSourceTextureViewport> ViewportWidget = WeakSourceTextureViewport.Pin();
	if (!ViewportWidget.IsValid())
	{
		return;
	}

	// Compute texture box in graph space
	const FVector2D TextureDimensions = FVector2D(InputTexture->GetSurfaceWidth(), InputTexture->GetSurfaceHeight());
	const FBox2D TextureBox(FVector2D::ZeroVector, TextureDimensions);

	// Get the visible rect in grap space, and the UVs
	const FBox2D VisibleRect = GetVisibleTextureBoxGraphSpace();
	const FVector2D UV0 = VisibleRect.Min / TextureBox.GetSize();
	const FVector2D UV1 = VisibleRect.Max / TextureBox.GetSize();

	const FLinearColor ColorWithExposure = (FLinearColor::White * PixelMapping->DesignerExposure).CopyWithNewOpacity(1.f);
	if (IsDrawingVisibleRectOnly())
	{
		const TSharedPtr<FSceneViewport> SceneViewport = ViewportWidget->GetViewport();
		if (!SceneViewport.IsValid())
		{
			return;
		}
	
		FCanvasTileItem TileItem(FVector2D::ZeroVector, InputTexture->GetResource(), SceneViewport->GetSizeXY(), UV0, UV1, ColorWithExposure);
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_MAX;

		bUseDPIScaling = false;
		Canvas->DrawItem(TileItem);
	}
	else
	{
		FCanvasTileItem TileItem(VisibleRect.Min * DesignerView->GetZoomAmount(), InputTexture->GetResource(), VisibleRect.GetSize() * DesignerView->GetZoomAmount(), UV0, UV1, ColorWithExposure);
		TileItem.BlendMode = ESimpleElementBlendMode::SE_BLEND_MAX;
		
		bUseDPIScaling = true;
		Canvas->DrawItem(TileItem);
	}
}

float FDMXPixelMappingSourceTextureViewportClient::UpdateViewportClientWindowDPIScale() const
{
	float DPIScale = 1.f;
	if (WeakSourceTextureViewport.IsValid())
	{
		TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(WeakSourceTextureViewport.Pin().ToSharedRef());
		if (WidgetWindow.IsValid())
		{
			DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
		}
	}

	return DPIScale;
}
