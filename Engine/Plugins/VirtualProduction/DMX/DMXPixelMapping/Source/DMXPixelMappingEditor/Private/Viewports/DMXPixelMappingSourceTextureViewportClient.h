// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ViewportClient.h"


class SDMXPixelMappingSourceTextureViewport;
class FDMXPixelMappingToolkit;

class FDMXPixelMappingSourceTextureViewportClient
	: public FCommonViewportClient
{
public:
	/** Constructor */
	FDMXPixelMappingSourceTextureViewportClient(const TSharedPtr<FDMXPixelMappingToolkit>& Toolkit, TWeakPtr<SDMXPixelMappingSourceTextureViewport> InViewport);
	
	/** If true, the widget draws the visible rect of the source texture */
	bool IsDrawingVisibleRectOnly() const;

	/** Returns the visible texture size in graph space */
	FBox2D GetVisibleTextureBoxGraphSpace() const;

protected:
	//~ Begin FViewportClient Interface
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual bool ShouldDPIScaleSceneCanvas() const override { return bUseDPIScaling; }
	//~ End FViewportClient Interface

	//~ Begin FViewport Interface
	virtual float UpdateViewportClientWindowDPIScale() const override;
	//~ End FViewport Interface

private:
	bool bUseDPIScaling = false;

	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
	TWeakPtr<SDMXPixelMappingSourceTextureViewport> WeakSourceTextureViewport;
};
