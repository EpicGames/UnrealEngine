// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDMXPixelMappingDMXLibraryView.h"

#include "Algo/Copy.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "Customizations/DMXPixelMappingDetailCustomization_FixtureGroup.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingComponentReference.h"
#include "IDetailsView.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Templates/DMXPixelMappingComponentTemplate.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "ViewModels/DMXPixelMappingDMXLibraryViewModel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SDMXPixelMappingFixturePatchList.h"

#define LOCTEXT_NAMESPACE "SDMXPixelMappingDMXLibraryView"


SDMXPixelMappingDMXLibraryView::~SDMXPixelMappingDMXLibraryView()
{
	if (ViewModel && FixturePatchList.IsValid())
	{
		ViewModel->SaveFixturePatchListDescriptor(FixturePatchList->MakeListDescriptor());
	}
}

void SDMXPixelMappingDMXLibraryView::Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;
	if (!WeakToolkit.IsValid())
	{
		return;
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();
	ViewModel = NewObject<UDMXPixelMappingDMXLibraryViewModel>();

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	ModelDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	ModelDetailsView->SetObject(ViewModel);

	const TAttribute<bool> AddFixtureGroupEnabledAttribute = TAttribute<bool>::CreateLambda(
		[this]()
		{
			return bRenderComponentContainedInSelection;
		});

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			// Add button
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Left)
			.Padding(4.f)
			.AutoHeight()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.ForegroundColor(FLinearColor::White)
				.ToolTipText_Lambda([this]()
					{
						return bRenderComponentContainedInSelection ?
							LOCTEXT("AddFixtureGroupTooltip", "Adds a Fixture Group to the Pixel Mapping. In the fixture group, a DMX Library can be selected.") :
							LOCTEXT("CannotAddFixtureGroupTooltip", "Please select a source texture");
					})
				.ContentPadding(FMargin(5.0f, 1.0f))
				.OnClicked(this, &SDMXPixelMappingDMXLibraryView::OnAddFixtureGroupButtonClicked)
				.IsEnabled(AddFixtureGroupEnabledAttribute)
				.Content()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(FMargin(0.f, 1.f))
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Plus"))
					]

					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.AutoWidth()
					.Padding(FMargin(2.f, 0.f, 2.f, 0.f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddFixtureGroupLabel", "Add Fixture Group"))
					]
				]
			]

			// Fixture group name 
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
				.Padding(4.f)
				[
					SAssignNew(FixtureGroupNameTextBlock, STextBlock)
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				]
			]

			
			// DMX Library (Model details)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				ModelDetailsView.ToSharedRef()
			]

			// Add Patches
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(AddPatchesWrapBox, SWrapBox)
				.UseAllottedSize(true)

				+ SWrapBox::Slot()
				.Padding(4.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
					.ForegroundColor(FLinearColor::White)
					.ToolTipText(LOCTEXT("AddSelectedPatchesTooltip", "Adds the selected patches to the Pixel Mapping"))
					.ContentPadding(FMargin(5.0f, 1.0f))
					.OnClicked(this, &SDMXPixelMappingDMXLibraryView::OnAddSelectedPatchesClicked)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddSelectedPatchesLabel", "Add Selected Patches"))
					]
				]

				+ SWrapBox::Slot()
				.Padding(8.f, 4.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
					.ForegroundColor(FLinearColor::White)
					.ToolTipText(LOCTEXT("AddAllPatchesTooltip", "Adds all patches to the Pixel Mapping"))
					.ContentPadding(FMargin(5.0f, 1.0f))
					.OnClicked(this, &SDMXPixelMappingDMXLibraryView::OnAddAllPatchesClicked)
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("AddAllPatchesLabel", "Add All Patches"))
					]
				]

				+ SWrapBox::Slot()
				.Padding(8.f, 4.f)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SDMXPixelMappingDMXLibraryView::GetUsePatchColorCheckState)
					.OnCheckStateChanged(this, &SDMXPixelMappingDMXLibraryView::OnUsePatchColorCheckStateChanged)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("UsePatchColorCheckBoxLabel", "Use Patch Color"))
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					]
				]
			]

			// The fixture patch list
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
				.Padding(FMargin(8.f, 2.f, 2.f, 0.f))
				[
					SAssignNew(ListOrAllPatchesAddedSwitcher, SWidgetSwitcher)

					// Fixture patch list
					+ SWidgetSwitcher::Slot()
					[
						SAssignNew(FixturePatchList, SDMXPixelMappingFixturePatchList, Toolkit, ViewModel)
					]

					// All patches added info box
					+ SWidgetSwitcher::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(AllPatchesAddedTextBlock, STextBlock)
						.Text(LOCTEXT("AllPatchesAddedHint", "All Patches added to Pixel Mapping"))
						.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					]
				]
			]
		]
	];

	ForceRefresh();

	// Refresh on changes
	ViewModel->OnDMXLibraryChanged.AddSP(this, &SDMXPixelMappingDMXLibraryView::RequestRefresh);
	UDMXLibrary::GetOnEntitiesAdded().AddSP(this, &SDMXPixelMappingDMXLibraryView::OnEntityAddedOrRemoved);
	UDMXLibrary::GetOnEntitiesRemoved().AddSP(this, &SDMXPixelMappingDMXLibraryView::OnEntityAddedOrRemoved);
	UDMXPixelMappingBaseComponent::GetOnComponentAdded().AddSP(this, &SDMXPixelMappingDMXLibraryView::OnComponentAddedOrRemoved);
	UDMXPixelMappingBaseComponent::GetOnComponentRemoved().AddSP(this, &SDMXPixelMappingDMXLibraryView::OnComponentAddedOrRemoved);
	
	// Follow selection
	Toolkit->GetOnSelectedComponentsChangedDelegate().AddSP(this, &SDMXPixelMappingDMXLibraryView::OnComponentSelected);

	// Apply the current selection
	OnComponentSelected();
}

void SDMXPixelMappingDMXLibraryView::RequestRefresh()
{
	if (!RefreshTimerHandle.IsValid())
	{
		RefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXPixelMappingDMXLibraryView::ForceRefresh));
	}
}

void SDMXPixelMappingDMXLibraryView::AddReferencedObjects(FReferenceCollector& Collector) 
{
	Collector.AddReferencedObject(ViewModel);
}

void SDMXPixelMappingDMXLibraryView::PostUndo(bool bSuccess)
{
	RequestRefresh();
}

void SDMXPixelMappingDMXLibraryView::PostRedo(bool bSuccess)
{
	RequestRefresh();
}

void SDMXPixelMappingDMXLibraryView::ForceRefresh()
{
	RefreshTimerHandle.Invalidate();

	if (ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		ViewModel->UpdateFixtureGroupFromSelection(WeakToolkit);

		// Update the details view visibility
		const UDMXPixelMappingFixtureGroupComponent* EditedFixtureGroup = ViewModel->GetFixtureGroupComponent();
		const EVisibility DMXLibraryVisibility = EditedFixtureGroup ? EVisibility::Visible : EVisibility::Collapsed;
		ModelDetailsView->SetVisibility(DMXLibraryVisibility);

		// Update the displayed fixture group name
		const FText FixtureGroupNameText = [EditedFixtureGroup, this]()
		{
			if (EditedFixtureGroup)
			{
				return FText::FromString(EditedFixtureGroup->GetUserName());
			}
			else if (ViewModel->IsMoreThanOneFixtureGroupSelected())
			{
				return LOCTEXT("ManyFixtureGroupsSelectedHint", "More than one Fixture Group selected");
			}
			else
			{
				return LOCTEXT("NoFixtureGroupSelectedHint", "No Fixture Group selected");
			}
		}();
		FixtureGroupNameTextBlock->SetText(FixtureGroupNameText);

		// Update the fixture patch list
		UDMXLibrary* DMXLibrary = EditedFixtureGroup ? EditedFixtureGroup->DMXLibrary.Get() : nullptr;
		const EVisibility FixturePatchListVisibility = DMXLibrary ? EVisibility::Visible : EVisibility::Collapsed;
		FixturePatchList->SetDMXLibrary(DMXLibrary);

		// Update the list to only display patches that are not added to pixel mapping
		const TArray<UDMXEntityFixturePatch*> FixturePatchesInDMXLibrary = GetFixturePatchesInDMXLibrary();
		const TArray<UDMXEntityFixturePatch*> FixturePatchesInPixelMapping = GetFixturePatchesInPixelMapping();

		TArray<UDMXEntityFixturePatch*> HiddenFixturePatches;
		Algo::CopyIf(FixturePatchesInDMXLibrary, HiddenFixturePatches,
			[&FixturePatchesInPixelMapping](const UDMXEntityFixturePatch* FixturePatchInDMXLibrary)
			{
				return FixturePatchesInPixelMapping.Contains(FixturePatchInDMXLibrary);
			});
		FixturePatchList->SetExcludedFixturePatches(HiddenFixturePatches);

		// Only show the list and options to add or select when fixture patches are available
		const EVisibility AddFixturePatchOptionsVisibility = FixturePatchesInDMXLibrary.Num() > HiddenFixturePatches.Num() ? EVisibility::Visible : EVisibility::Collapsed;
		AddPatchesWrapBox->SetVisibility(AddFixturePatchOptionsVisibility);

		if (AddFixturePatchOptionsVisibility == EVisibility::Visible)
		{
			ListOrAllPatchesAddedSwitcher->SetActiveWidget(FixturePatchList.ToSharedRef());
		}
		else
		{
			ListOrAllPatchesAddedSwitcher->SetActiveWidget(AllPatchesAddedTextBlock.ToSharedRef());
		}

		FixturePatchList->RequestRefresh();
	}
}

void SDMXPixelMappingDMXLibraryView::OnComponentSelected()
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	bRenderComponentContainedInSelection = false;

	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = Toolkit->GetSelectedComponents();
	for (const FDMXPixelMappingComponentReference& ComponentReference : SelectedComponents)
	{
		UDMXPixelMappingBaseComponent* Component = ComponentReference.GetComponent();
		do
		{
			// Is renderer component contained in selection?
			bRenderComponentContainedInSelection |= Component->GetClass() == UDMXPixelMappingRendererComponent::StaticClass();

			Component = Component->GetParent();
		} while (Component);
	}

	RequestRefresh();
}

void SDMXPixelMappingDMXLibraryView::OnEntityAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities)
{
	RequestRefresh();
}

void SDMXPixelMappingDMXLibraryView::OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	RequestRefresh();
}

FReply SDMXPixelMappingDMXLibraryView::OnAddFixtureGroupButtonClicked()
{
	if (ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		const FScopedTransaction AddFixtureGroupTransaction(LOCTEXT("AddFixtureGroupTransaction", "Add Fixture Group"));
		ViewModel->CreateAndSetNewFixtureGroup(WeakToolkit);

		RequestRefresh();
	}

	return FReply::Handled();
}

FReply SDMXPixelMappingDMXLibraryView::OnAddSelectedPatchesClicked()
{
	if (WeakToolkit.IsValid() &&
		ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		const FScopedTransaction AddSelectedFixturePatchesTransaction(LOCTEXT("AddSelectedFixturePatchesTransaction", "Add Fixture Patches to Pixel Mapping"));

		const TArray<UDMXEntityFixturePatch*> SelectedFixturePatches = FixturePatchList->GetSelectedFixturePatches();
		ViewModel->AddFixturePatchesEnsured(SelectedFixturePatches);

		// Select the next fixture patches in the list
		FixturePatchList->SelectAfter(SelectedFixturePatches);

		RequestRefresh();
	}

	return FReply::Handled();
}

FReply SDMXPixelMappingDMXLibraryView::OnAddAllPatchesClicked()
{
	if (ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		const FScopedTransaction AddAllFixturePatchesTransaction(LOCTEXT("AddAllFixturePatchesTransaction", "Add Fixture Patches to Pixel Mapping"));

		const TArray<UDMXEntityFixturePatch*> FixturePatchesInList = FixturePatchList->GetFixturePatchesInList();
		ViewModel->AddFixturePatchesEnsured(FixturePatchesInList);

		RequestRefresh();
	}

	return FReply::Handled();
}

ECheckBoxState SDMXPixelMappingDMXLibraryView::GetUsePatchColorCheckState() const
{
	return ViewModel && ViewModel->ShouldNewComponentsUsePatchColor() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SDMXPixelMappingDMXLibraryView::OnUsePatchColorCheckStateChanged(ECheckBoxState NewCheckState)
{
	if (ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		const bool bNewPatchesShouldUsePatchColor = NewCheckState == ECheckBoxState::Checked;
		ViewModel->SetNewComponentsUsePatchColor(bNewPatchesShouldUsePatchColor);
	}
}

TArray<UDMXEntityFixturePatch*> SDMXPixelMappingDMXLibraryView::GetFixturePatchesInDMXLibrary() const
{
	if (!ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		return TArray<UDMXEntityFixturePatch*>();
	}

	UDMXLibrary* DMXLibrary = ViewModel->GetDMXLibrary();
	return DMXLibrary ? DMXLibrary->GetEntitiesTypeCast<UDMXEntityFixturePatch>() : TArray<UDMXEntityFixturePatch*>();
}

TArray<UDMXEntityFixturePatch*> SDMXPixelMappingDMXLibraryView::GetFixturePatchesInPixelMapping() const
{
	TArray<UDMXEntityFixturePatch*> FixturePatchesInPixelMapping;
	if (!ensureMsgf(ViewModel, TEXT("Invalid view model for PixelMapping DMX Library View, cannot display view.")))
	{
		return FixturePatchesInPixelMapping;
	}

	UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	UDMXLibrary* DMXLibrary = ViewModel->GetDMXLibrary();

	if (PixelMapping && DMXLibrary)
	{
		PixelMapping->ForEachComponent([&FixturePatchesInPixelMapping, DMXLibrary](UDMXPixelMappingBaseComponent* Component)
			{
				if (UDMXPixelMappingFixtureGroupItemComponent* FixtureGroupItem = Cast<UDMXPixelMappingFixtureGroupItemComponent>(Component))
				{
					if (FixtureGroupItem->FixturePatchRef.DMXLibrary == DMXLibrary)
					{
						FixturePatchesInPixelMapping.Add(FixtureGroupItem->FixturePatchRef.GetFixturePatch());
					}
				}
				else if (UDMXPixelMappingMatrixComponent* Matrix = Cast<UDMXPixelMappingMatrixComponent>(Component))
				{
					if (Matrix->FixturePatchRef.DMXLibrary == DMXLibrary)
					{
						FixturePatchesInPixelMapping.Add(Matrix->FixturePatchRef.GetFixturePatch());
					}
				}
			});
	}

	return FixturePatchesInPixelMapping;
}

#undef LOCTEXT_NAMESPACE
