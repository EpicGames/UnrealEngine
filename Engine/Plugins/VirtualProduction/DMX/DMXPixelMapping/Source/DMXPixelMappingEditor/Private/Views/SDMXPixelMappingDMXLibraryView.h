// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Engine/EngineTypes.h"
#include "Library/DMXEntityReference.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/GCObject.h"

enum class ECheckBoxState : uint8; 
class FDMXPixelMappingToolkit;
class FReply;
class IDetailsView;
class SBorder;
class SDMXPixelMappingFixturePatchList;
class STextBlock;
class SWidgetSwitcher;
class SWrapBox;
class UDMXEntity;
class UDMXLibrary;
class UDMXPixelMapping;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingDMXLibraryViewModel;


/** Displays the DMX Library of the currently selected fixture group component */
class SDMXPixelMappingDMXLibraryView
	: public SCompoundWidget
	, public FGCObject
	, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingDMXLibraryView) 
	{}

	SLATE_END_ARGS()

	virtual ~SDMXPixelMappingDMXLibraryView();

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit);

	/** Refreshes the widget on the next tick */
	void RequestRefresh();

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SDMXPixelMappingDMXLibraryView");
	}
	//~ End FGCObject interface

	//~ Begin FSelfRegisteringEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FSelfRegisteringEditorUndoClient interface

private:
	/** Refreshes the widget */
	void ForceRefresh();

	/** Called when a component was selected */
	void OnComponentSelected();

	/** Called when an entity was added or removed from the DMX Library */
	void OnEntityAddedOrRemoved(UDMXLibrary* DMXLibrary, TArray<UDMXEntity*> Entities);

	/** Called when a component was added or removed from pixel mapping */
	void OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** Called when the 'Add Fixture Group' button was clicked */
	FReply OnAddFixtureGroupButtonClicked();

	/** Called when the 'Add Selected Patches' button was clicked */
	FReply OnAddSelectedPatchesClicked();

	/** Called when the 'Add All Patches' button was clicked */
	FReply OnAddAllPatchesClicked();

	/** Returns the check state of the 'use patch color' check box */
	ECheckBoxState GetUsePatchColorCheckState() const;

	/** Called when the check state of the 'use patch color' check box changed */
	void OnUsePatchColorCheckStateChanged(ECheckBoxState NewCheckState);

	/** Helper that returns all fixture patches in use in the pixel mapping */
	TArray<UDMXEntityFixturePatch*> GetFixturePatchesInDMXLibrary() const;

	/** Helper that returns all fixture patches in use in the pixel mapping */
	TArray<UDMXEntityFixturePatch*> GetFixturePatchesInPixelMapping() const;

	/** Border that holds the details views */
	TSharedPtr<SBorder> DMXLibraryBorder;

	/** Timer handle for the Request Refresh method */
	FTimerHandle RefreshTimerHandle;

	/** True if a renderere component is contained in the current selection */
	bool bRenderComponentContainedInSelection = false;

	/** Text block displaying the selected fixture group name */
	TSharedPtr<STextBlock> FixtureGroupNameTextBlock;

	/** Switches between the list and the all patches added text blocok */
	TSharedPtr<SWidgetSwitcher> ListOrAllPatchesAddedSwitcher;

	/** List of fixture patches the user can select from */
	TSharedPtr<SDMXPixelMappingFixturePatchList> FixturePatchList;

	/** Text block displaying 'all patches added' when no patches are available to add from the dmx library */
	TSharedPtr<STextBlock> AllPatchesAddedTextBlock;

	/** Wrap box containing the 'add selected patches' and 'add all patches' buttons and the 'Use Patch Color' checkbox */
	TSharedPtr<SWrapBox> AddPatchesWrapBox;

	/** Details view for the model, displays the library picker */
	TSharedPtr<IDetailsView> ModelDetailsView;

	/** The view model of the focused DMX Library */
	TObjectPtr<UDMXPixelMappingDMXLibraryViewModel> ViewModel;

	/** The toolkit of this editor */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
