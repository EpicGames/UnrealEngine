// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDMXPixelMappingDesignerView.h"

#include "Algo/Find.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorCommands.h"
#include "DMXPixelMappingEditorStyle.h"
#include "DragDrop/DMXPixelMappingGroupChildDragDropHelper.h"
#include "Framework/Application/SlateApplication.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingScreenComponent.h"
#include "DragDrop/DMXPixelMappingDragDropOp.h"
#include "ScopedTransaction.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SCanvas.h"
#include "Widgets/SDMXPixelMappingDesignerCanvas.h"
#include "Widgets/SDMXPixelMappingDesignerToolbar.h"
#include "Widgets/SDMXPixelMappingOutputComponent.h"
#include "Widgets/SDMXPixelMappingRuler.h"
#include "Widgets/SDMXPixelMappingSourceTextureViewport.h"
#include "Widgets/SDMXPixelMappingTransformHandle.h"
#include "Widgets/SDMXPixelMappingZoomPan.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingDesignerView"

void SDMXPixelMappingDesignerView::Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit)
{
	using namespace UE::DMX;

	SDMXPixelMappingSurface::Construct(SDMXPixelMappingSurface::FArguments()
		.AllowContinousZoomInterpolation(false)
		.Content()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SDMXPixelMappingDesignerView::GetHoveredComponentParentNameText)
							.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("BreadcrumbTrail.Delimiter"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SDMXPixelMappingDesignerView::GetHoveredComponentNameText)
							.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
						]
					]
				]
			]

			+ SVerticalBox::Slot()
			[
				SNew(SGridPanel)
				.FillColumn(1, 1.0f)
				.FillRow(1, 1.0f)

				// Corner
				+ SGridPanel::Slot(0, 0)
				[
					SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("GenericWhiteBox"))
					.BorderBackgroundColor(FLinearColor(FColor(48, 48, 48)))
				]

				// Top Ruler
				+ SGridPanel::Slot(1, 0)
				[
					SAssignNew(TopRuler, SDMXPixelMappingRuler)
					.Orientation(Orient_Horizontal)
					.Visibility(this, &SDMXPixelMappingDesignerView::GetRulerVisibility)
				]

				// Side Ruler
				+ SGridPanel::Slot(0, 1)
				[
					SAssignNew(SideRuler, SDMXPixelMappingRuler)
					.Orientation(Orient_Vertical)
					.Visibility(this, &SDMXPixelMappingDesignerView::GetRulerVisibility)
				]

				+ SGridPanel::Slot(1, 1)
				[
					SAssignNew(PreviewHitTestRoot, SOverlay)
					.Visibility(EVisibility::Visible)
					.Clipping(EWidgetClipping::ClipToBoundsAlways)
						
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(ZoomPan, SDMXPixelMappingZoomPan)
						.ZoomAmount(this, &SDMXPixelMappingDesignerView::GetZoomAmount)
						.ViewOffset(this, &SDMXPixelMappingDesignerView::GetViewOffset)
						.Visibility(this, &SDMXPixelMappingDesignerView::GetZoomPanVisibility)
						

						[
							SNew(SOverlay)

							// Viewport
							+ SOverlay::Slot()
							[
								SAssignNew(SourceTextureViewport, SDMXPixelMappingSourceTextureViewport, InToolkit)
							]

							// Designer canvas
							+ SOverlay::Slot()
							[
								SAssignNew(DesignCanvasBorder, SBorder)
								.HAlign(HAlign_Fill)
								.VAlign(VAlign_Fill)
								.BorderImage(FAppStyle::GetBrush("NoBorder"))
								.Padding(0.f)
								[
									SAssignNew(DesignCanvas, SConstraintCanvas)
									.Visibility(EVisibility::HitTestInvisible)
								]
							]
						]
					]

					// A layer in the overlay where we put all the tools for the user
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(ExtensionWidgetCanvas, SCanvas)
						.Visibility(this, &SDMXPixelMappingDesignerView::GetExtensionCanvasVisibility)
					]

					// Top toolbar UI
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)

						// Zoom text
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.f, 2.0f)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
							.Text(this, &SDMXPixelMappingDesignerView::GetZoomText)
							.ColorAndOpacity(this, &SDMXPixelMappingDesignerView::GetZoomTextColorAndOpacity)
						]

						// Cursor position
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.f, 2.0f)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
							.Font(FCoreStyle::GetDefaultFontStyle(TEXT("BoldCondensed"), 14))
							.Text(this, &SDMXPixelMappingDesignerView::GetCursorPositionText)
							.ColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 0.25f))
							.Visibility(this, &SDMXPixelMappingDesignerView::GetCursorPositionTextVisibility)
						]

						// Spacer
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						[
							SNew(SSpacer)
							.Size(FVector2D(1, 1))
						]

						// Toolbar
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.f, 2.0f)
						[
							SNew(SDMXPixelMappingDesignerToolbar, InToolkit)
							.OnZoomToFitClicked(this, &SDMXPixelMappingDesignerView::OnZoomToFitClicked)
						]
					]
				]
			]
		]
	, InToolkit);

	ZoomToFit(true);

	// Bind to selection changes
	InToolkit->GetOnSelectedComponentsChangedDelegate().AddSP(this, &SDMXPixelMappingDesignerView::OnSelectedComponentsChanged);

	// Bind to component changes
	UDMXPixelMappingBaseComponent::GetOnComponentAdded().AddSP(this, &SDMXPixelMappingDesignerView::OnComponentAdded);
	UDMXPixelMappingBaseComponent::GetOnComponentRemoved().AddSP(this, &SDMXPixelMappingDesignerView::OnComponentRemoved);
}

const FGeometry& SDMXPixelMappingDesignerView::GetGraphTickSpaceGeometry() const
{
	return ZoomPan->GetTickSpaceGeometry();
}

int32 SDMXPixelMappingDesignerView::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = SDMXPixelMappingSurface::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// Draw the grid snapping grid overlay if required
	const UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	if (!PixelMapping || 
		!PixelMapping->bGridSnappingEnabled || 
		PixelMapping->SnapGridColor.A == 0.f || 
		!CachedRendererComponent.IsValid())
	{
		return LayerId;
	}
	
	if (!ensureMsgf(PixelMapping->SnapGridColumns > 0 && PixelMapping->SnapGridRows, TEXT("Invalid snap grid. Cannot draw grid.")))
	{
		return LayerId;
	}

	// Get the line spacings
	const FVector2D RelativeOrigin = AllottedGeometry.AbsoluteToLocal(GridOrigin);
	const FVector2D TextureSizeZoomed = CachedRendererComponent->GetSize() * GetZoomAmount();
	const float LineSpacingX = TextureSizeZoomed.X / PixelMapping->SnapGridColumns;
	const float LineSpacingY = TextureSizeZoomed.Y / PixelMapping->SnapGridRows;

	// Draw the grid above the surface, above the hit test root overlay, above the the zoom pan overlay, above the texture overlay
	const int32 GridLayer = LayerId + 4;

	// Horizontal
	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);
	for (int32 RowIndex = 0; RowIndex <= PixelMapping->SnapGridRows; RowIndex++)
	{
		LinePoints[0] = RelativeOrigin + FVector2D(0.f, LineSpacingY * RowIndex);
		LinePoints[1] = RelativeOrigin + FVector2D(TextureSizeZoomed.X, LineSpacingY * RowIndex);

		constexpr bool bAntialias = false;
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			GridLayer,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			PixelMapping->SnapGridColor,
			bAntialias);
	}

	// Vertical
	for (int32 ColumnIndex = 0; ColumnIndex <= PixelMapping->SnapGridColumns; ColumnIndex++)
	{
		LinePoints[0] = RelativeOrigin + FVector2D(LineSpacingX * ColumnIndex, 0.f);
		LinePoints[1] = RelativeOrigin + FVector2D(LineSpacingX * ColumnIndex, TextureSizeZoomed.Y);

		constexpr bool bAntialias = false;
		FSlateDrawElement::MakeLines(
			OutDrawElements, 
			GridLayer,
			AllottedGeometry.ToPaintGeometry(), 
			LinePoints, 
			ESlateDrawEffect::None, 
			PixelMapping->SnapGridColor,
			bAntialias);
	}

	return GridLayer;
}

void SDMXPixelMappingDesignerView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SDMXPixelMappingSurface::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	// Update test if the renderer changed, and if so, rebuild the designer
	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	if (RendererComponent != CachedRendererComponent)
	{
		CachedRendererComponent = RendererComponent;
		RebuildDesigner();
	}

	// Cache geometry
	CachedWidgetGeometry.Reset();
	FArrangedWidget WindowWidgetGeometry(PreviewHitTestRoot.ToSharedRef(), GetDesignerGeometry());
	PopulateWidgetGeometryCache(WindowWidgetGeometry);

	// Compute the origin in absolute screen space.
	FGeometry RootGeometry = CachedWidgetGeometry.FindChecked(SourceTextureViewport.ToSharedRef()).Geometry;
	GridOrigin = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(FVector2D::ZeroVector);

	// Update the rulers
	TopRuler->SetRuling(GridOrigin, 1.0f / GetZoomAmount());
	SideRuler->SetRuling(GridOrigin, 1.0f / GetZoomAmount());

	if (IsHovered())
	{
		// Get cursor in absolute window space.
		FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();
		CursorPos = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(RootGeometry.AbsoluteToLocal(CursorPos));

		TopRuler->SetCursor(CursorPos);
		SideRuler->SetCursor(CursorPos);
	}
	else
	{
		TopRuler->SetCursor(TOptional<FVector2D>());
		SideRuler->SetCursor(TOptional<FVector2D>());
	}

	// Handle pending drag drop ops
	if (PendingDragDropOp.IsValid())
	{
		FVector2D GraphSpaceCursorPosition;
		if (GetGraphSpaceCursorPosition(GraphSpaceCursorPosition))
		{
			// Layout childs with the group item drag drop helper if possible, else use the default layout method
			if (TSharedPtr<FDMXPixelMappingGroupChildDragDropHelper> GroupChildDragDropHelper = PendingDragDropOp->GetGroupChildDragDropHelper())
			{
				// If it was created from template, it was dragged in from details or palette. In this case align the components.
				const bool bAlignComponents = PendingDragDropOp->WasCreatedAsTemplate();

				GroupChildDragDropHelper->LayoutComponents(GraphSpaceCursorPosition, bAlignComponents);
			}
			else
			{
				PendingDragDropOp->LayoutOutputComponents(GraphSpaceCursorPosition);
			}
		}

		PendingDragDropOp.Reset();
	}
}

FReply SDMXPixelMappingDesignerView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnMouseButtonDown(MyGeometry, MouseEvent);

	UDMXPixelMappingOutputComponent* ClickedComponent = GetComponentUnderCursor();
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && ClickedComponent)
	{
		PendingSelectedComponent = ClickedComponent;

		// Handle reclicking a selected component depending on type
		const bool bClickedSelectedComponent = [ClickedComponent, this]()
		{
			for (const FDMXPixelMappingComponentReference& SelectedComponentRef : GetSelectedComponents())
			{
				if (ClickedComponent == SelectedComponentRef.GetComponent())
				{
					return true;
				}
				else if (UDMXPixelMappingMatrixCellComponent* ClickedMatrixCellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(ClickedComponent))
				{
					if (ClickedMatrixCellComponent->GetParent() == SelectedComponentRef.GetComponent())
					{
						return true;
					}
				}
				else if (UDMXPixelMappingFixtureGroupItemComponent* ClickedGroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(ClickedComponent))
				{
					if (ClickedGroupItemComponent->GetParent() == SelectedComponentRef.GetComponent() &&
						ClickedGroupItemComponent->IsLockInDesigner())
					{
						return true;
					}
				}
			}

			return false;
		}();

		const bool bClearPreviousSelection = !MouseEvent.IsShiftDown() && !MouseEvent.IsControlDown() && !bClickedSelectedComponent;
		ResolvePendingSelectedComponents(bClearPreviousSelection);

		FVector2D GraphSpaceCursorPosition;
		if (GetGraphSpaceCursorPosition(GraphSpaceCursorPosition))
		{
			DragAnchor = GraphSpaceCursorPosition;

			return
				FReply::Handled()
				.PreventThrottling()
				.SetUserFocus(AsShared(), EFocusCause::Mouse)
				.CaptureMouse(AsShared())
				.DetectDrag(AsShared(), EKeys::LeftMouseButton);
		}
	}

	// Capture mouse for the drag handle and general mouse actions
	return 
		FReply::Handled()
		.PreventThrottling()
		.SetUserFocus(AsShared(), EFocusCause::Mouse)
		.CaptureMouse(AsShared());
}

FReply SDMXPixelMappingDesignerView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		// Select the output component under the mouse
		PendingSelectedComponent = GetComponentUnderCursor();

		// Select the Renderer if no Output Component was under the mouse
		if (!PendingSelectedComponent.IsValid())
		{
			if (TSharedPtr<FDMXPixelMappingToolkit> ToolkitPtr = WeakToolkit.Pin())
			{
				if (UDMXPixelMappingRendererComponent* RendererComponent = ToolkitPtr->GetActiveRendererComponent())
				{
					PendingSelectedComponent = RendererComponent;
				}
			}
		}

		const bool bClearPreviousSelection = !MouseEvent.IsShiftDown() && !MouseEvent.IsControlDown();
		ResolvePendingSelectedComponents(bClearPreviousSelection);
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton &&
		!bIsPanning &&
		!bIsZooming)
	{
		OpenContextMenu();
	}
	
	SDMXPixelMappingSurface::OnMouseButtonUp(MyGeometry, MouseEvent);

	return FReply::Handled().ReleaseMouseCapture();
}

FReply SDMXPixelMappingDesignerView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		if (const TSharedPtr<FDMXPixelMappingToolkit> ToolkitPtr = WeakToolkit.Pin())
		{
			const FScopedTransaction Transaction(FText::Format(LOCTEXT("DMXPixelMapping.RemoveComponents", "Remove {0}|plural(one=Component, other=Components)"), ToolkitPtr->GetSelectedComponents().Num()));

			ToolkitPtr->DeleteSelectedComponents();

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SDMXPixelMappingDesignerView::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnDragDetected(MyGeometry, MouseEvent);

	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = GetSelectedComponents();
	if (!WeakToolkit.IsValid() || SelectedComponents.IsEmpty())
	{
		return FReply::Handled();
	}
	const TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

	TArray<UDMXPixelMappingOutputComponent*> DraggedComponentCandidates;
	for (const FDMXPixelMappingComponentReference& SelectedComponent : SelectedComponents)
	{
		if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(SelectedComponent.GetComponent()))
		{
			DraggedComponentCandidates.Add(OutputComponent);
		}
	}

	TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>> DraggedComponents;
	for (UDMXPixelMappingOutputComponent* Candidate : DraggedComponentCandidates)
	{
		// Check the parent chain of each dragged component and ignore those that are children of other dragged components
		bool bIsChild = false;
		for (TWeakObjectPtr<UDMXPixelMappingBaseComponent> Parent = Candidate->GetParent(); Parent.IsValid(); Parent = Parent->GetParent())
		{
			bIsChild = DraggedComponentCandidates.ContainsByPredicate([Parent](const UDMXPixelMappingOutputComponent* BaseComponent)
				{
					return BaseComponent == Parent.Get();
				});

			if (bIsChild)
			{
				break;
			}
		}

		if (!bIsChild && !Candidate->IsLockInDesigner())
		{
			DraggedComponents.Add(Candidate);
		}
	}

	// We know the first element is the one that was selected via mouse
	if (DraggedComponents.Num() > 0)
	{
		// We rely on the first being the clicked component. There's an according comment in the DragDropOp and the detected drag is raised here.
		UDMXPixelMappingOutputComponent* ClickedComponent = CastChecked<UDMXPixelMappingOutputComponent>(DraggedComponents[0]);

		const FVector2D GraphSpaceDragOffset = DragAnchor - ClickedComponent->GetPositionRotated();

		TSharedRef<FDMXPixelMappingDragDropOp> DragDropOp = FDMXPixelMappingDragDropOp::New(Toolkit, GraphSpaceDragOffset, DraggedComponents);
		DragDropOp->SetDecoratorVisibility(false);

		// Clear any pending selected widgets
		PendingSelectedComponent = nullptr;

		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Handled();
}

void SDMXPixelMappingDesignerView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	const TSharedPtr<FDMXPixelMappingDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FDMXPixelMappingDragDropOp>();
	if (DragDropOp.IsValid())
	{
		const FGeometry WidgetUnderCursorGeometry = SourceTextureViewport->GetTickSpaceGeometry();
		const FVector2D ScreenSpacePosition = DragDropEvent.GetScreenSpacePosition();
		const FVector2D LocalPosition = WidgetUnderCursorGeometry.AbsoluteToLocal(ScreenSpacePosition);

		// If this was dragged into designer, but has no components yet, it was dragged in from Palette or Details
		if (DragDropOp->WasCreatedAsTemplate())
		{
			HandleDragEnterFromDetailsOrPalette(DragDropOp);
		}
	}

	SDMXPixelMappingSurface::OnDragEnter(MyGeometry, DragDropEvent);
}

void SDMXPixelMappingDesignerView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDragLeave(DragDropEvent);

	const TSharedPtr<FDMXPixelMappingDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FDMXPixelMappingDragDropOp>();
	if (DragDropOp.IsValid() && WeakToolkit.IsValid())
	{
		TSharedRef<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin().ToSharedRef();

		// If the drag drop op was dragged in from details or palette, remove the components
		if (DragDropOp->WasCreatedAsTemplate())
		{
			TSet<FDMXPixelMappingComponentReference> Parents;
			for (const TWeakObjectPtr<UDMXPixelMappingBaseComponent>& Component : DragDropOp->GetDraggedComponents())
			{
				if (Component.IsValid() && Component->GetParent())
				{
					Parents.Add(Toolkit->GetReferenceFromComponent(Component->GetParent()));
					Component->GetParent()->RemoveChild(Component.Get());
				}
			}

			// Select parents instead
			Toolkit->SelectComponents(Parents);

			RebuildDesigner();
		}
	}
}

FReply SDMXPixelMappingDesignerView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDragOver(MyGeometry, DragDropEvent);

	const TSharedPtr<FDMXPixelMappingDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FDMXPixelMappingDragDropOp>();
	if (DragDropOp.IsValid())
	{
		// Handle the drag drop op on tick
		PendingDragDropOp = DragDropOp;
	}

	return FReply::Handled();
}

FReply SDMXPixelMappingDesignerView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDrop(MyGeometry, DragDropEvent);

	return FReply::Handled().EndDragDrop().SetUserFocus(AsShared());
}

FSlateRect SDMXPixelMappingDesignerView::ComputeAreaBounds() const
{
	if (SourceTextureViewport.IsValid())
	{
		const FVector2D Size
		{
			FMath::Max(SourceTextureViewport->GetWidthGraphSpace().Get(), 1.0),
			FMath::Max(SourceTextureViewport->GetHeightGraphSpace().Get(), 1.0),
		};

		return FSlateRect(0.f, 0.f, Size.X, Size.Y);
	}

	return FSlateRect();
}

int32 SDMXPixelMappingDesignerView::GetGraphRulePeriod() const
{
	return 10;
}

float SDMXPixelMappingDesignerView::GetGridScaleAmount() const
{
	return 1.f;
}

int32 SDMXPixelMappingDesignerView::GetGridSize() const
{
	return 4;
}

void SDMXPixelMappingDesignerView::PostUndo(bool bSuccess)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent();
	CachedRendererComponent = RendererComponent;
	RebuildDesigner();
}

void SDMXPixelMappingDesignerView::PostRedo(bool bSuccess)
{
	// Same as post undo
	PostUndo(bSuccess);
}

void SDMXPixelMappingDesignerView::RebuildDesigner()
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	if (DesignCanvas.IsValid())
	{
		DesignCanvas->ClearChildren();
	}

	DesignCanvasBorder->SetContent
	(
		SAssignNew(DesignCanvas, SDMXPixelMappingDesignerCanvas)
	);

	OutputComponentWidgets.Reset();
	if (CachedRendererComponent.IsValid())
	{
		const TSharedRef<SConstraintCanvas> ComponentCanvas = SNew(SConstraintCanvas);
		DesignCanvas->AddSlot()
			.Alignment(FVector2D::ZeroVector)
			[
				ComponentCanvas
			];

		CachedRendererComponent->ForEachChild([this, &Toolkit, &ComponentCanvas](UDMXPixelMappingBaseComponent* Component)
			{
				using namespace UE::DMX;

				if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component))
				{
					const TSharedRef<SDMXPixelMappingOutputComponent> ComponentWidget = SNew(SDMXPixelMappingOutputComponent, Toolkit.ToSharedRef(), OutputComponent);
					ComponentWidget->AddToCanvas(ComponentCanvas);

					OutputComponentWidgets.Add(ComponentWidget);
				}
			}, true);
	}
}

void SDMXPixelMappingDesignerView::CreateExtensionWidgetsForSelection()
{
	// Remove all the current extension widgets
	ClearExtensionWidgets();

	// Create new handles if possible
	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = GetSelectedComponents();
	if (SelectedComponents.Num() != 1)
	{
		return;
	}

	UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(SelectedComponents.Array()[0].GetComponent());
	if (OutputComponent && 
		OutputComponent->GetClass() != UDMXPixelMappingRendererComponent::StaticClass() &&
		OutputComponent->IsVisible() &&
		!OutputComponent->IsLockInDesigner())
	{
		// Add transform handles
		constexpr float Offset = 0.1f;
		const TSharedPtr<SDMXPixelMappingDesignerView> Self = SharedThis(this);
		TransformHandles.Add(SNew(SDMXPixelMappingTransformHandle, Self, EDMXPixelMappingTransformDirection::CenterRight, FVector2D(Offset, 0.f)));
		TransformHandles.Add(SNew(SDMXPixelMappingTransformHandle, Self, EDMXPixelMappingTransformDirection::BottomCenter, FVector2D(0.f, Offset)));
		TransformHandles.Add(SNew(SDMXPixelMappingTransformHandle, Self, EDMXPixelMappingTransformDirection::BottomRight, FVector2D(Offset, Offset)));

		// Add Widgets to designer surface
		for (TSharedPtr<SDMXPixelMappingTransformHandle>& Handle : TransformHandles)
		{
			ExtensionWidgetCanvas->AddSlot()
				.Position(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(this, &SDMXPixelMappingDesignerView::GetExtensionPosition, Handle)))
				.Size(TAttribute<FVector2D>::Create(TAttribute<FVector2D>::FGetter::CreateSP(this, &SDMXPixelMappingDesignerView::GetExtensionSize, Handle)))
				[
					Handle.ToSharedRef()
				];
		}
	}
}

void SDMXPixelMappingDesignerView::ClearExtensionWidgets()
{
	ExtensionWidgetCanvas->ClearChildren();
}

void SDMXPixelMappingDesignerView::OpenContextMenu()
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	if (!Toolkit.IsValid())
	{
		return;
	}

	constexpr bool bShouldCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, Toolkit->GetToolkitCommands());

	MenuBuilder.BeginSection("EditorSettings", LOCTEXT("EditorSettingsSection", "Editor Settings"));
	{
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleAlwaysSelectGroup);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleScaleChildrenWithParent);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TransformModes", LOCTEXT("TransformModesSection", "Transform Modes"));
	{		
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().EnableResizeMode);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().EnableRotateMode);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("DisplaySettings", LOCTEXT("SettingsSection", "Display Settings"));
	{
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowComponentNames);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowPatchInfo);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowMatrixCells);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowCellIDs);
		MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowPivot);
	}
	MenuBuilder.EndSection();

	if (Toolkit->CanPerformCommandsOnGroup())
	{
		MenuBuilder.BeginSection("FixtureGroupActions", LOCTEXT("FixtureGroupActionsSection", "Fixture Group Actions"));
		{
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().FlipGroupHorizontally);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().FlipGroupVertically);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().SizeGroupToTexture);
		}
		MenuBuilder.EndSection();
	}

	const TSharedRef<SWidget> MenuContent = 
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			MenuBuilder.MakeWidget()
		];

	FSlateApplication::Get().PushMenu(
		AsShared(),
		FWidgetPath(),
		MenuContent,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
	);
}

EVisibility SDMXPixelMappingDesignerView::GetExtensionCanvasVisibility() const
{
	for (const FDMXPixelMappingComponentReference& Component : GetSelectedComponents())
	{
		UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component.GetComponent());

		if (!OutputComponent ||
			!OutputComponent->IsVisible() ||
			OutputComponent->IsLockInDesigner())
		{
			return EVisibility::Hidden;
		}
	}
	return EVisibility::SelfHitTestInvisible;
}

FVector2D SDMXPixelMappingDesignerView::GetExtensionPosition(TSharedPtr<SDMXPixelMappingTransformHandle> Handle)
{
	const FDMXPixelMappingComponentReference& SelectedComponent = GetSelectedComponent();

	if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(SelectedComponent.GetComponent()))
	{
		FGeometry SelectedComponentGeometry;

		if (GetComponentGeometry(OutputComponent, SelectedComponentGeometry))
		{
			const FVector2D LocalPosition = [Handle, &SelectedComponentGeometry]() -> FVector2D
			{
				// Get the initial offset based on the location around the selected object.
				switch (Handle->GetTransformDirection())
				{
				case EDMXPixelMappingTransformDirection::CenterRight:
					return FVector2D(SelectedComponentGeometry.GetLocalSize().X, SelectedComponentGeometry.GetLocalSize().Y * 0.5f);
				case EDMXPixelMappingTransformDirection::BottomCenter:
					return FVector2D(SelectedComponentGeometry.GetLocalSize().X * 0.5f, SelectedComponentGeometry.GetLocalSize().Y);
				case EDMXPixelMappingTransformDirection::BottomRight:
					return SelectedComponentGeometry.GetLocalSize();
				default:
					checkNoEntry(); // Unhandled enum value
				}
				return FVector2D::ZeroVector;
			}();


			const FVector2D SelectedWidgetScale = FVector2D(SelectedComponentGeometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector());

			const FVector2D ApplicationScaledOffset = Handle->GetOffset() * GetDesignerGeometry().Scale;

			const FVector2D LocalOffsetFull = ApplicationScaledOffset / SelectedWidgetScale;
			const FVector2D PositionFullOffset = GetDesignerGeometry().AbsoluteToLocal(SelectedComponentGeometry.LocalToAbsolute(LocalPosition + LocalOffsetFull));
			const FVector2D LocalOffsetHalf = (ApplicationScaledOffset / 2.0f) / SelectedWidgetScale;
			const FVector2D PositionHalfOffset = GetDesignerGeometry().AbsoluteToLocal(SelectedComponentGeometry.LocalToAbsolute(LocalPosition + LocalOffsetHalf));

			const FVector2D PivotCorrection = PositionHalfOffset - (PositionFullOffset + FVector2D(5.0f, 5.0f));

			const FVector2D FinalPosition = PositionFullOffset + PivotCorrection;

			return FVector2D(FinalPosition);
		}
	}

	return FVector2D(0.f, 0.f);
}

FVector2D SDMXPixelMappingDesignerView::GetExtensionSize(TSharedPtr<SDMXPixelMappingTransformHandle> Handle)
{
	return Handle->GetDesiredSize();
}

FText SDMXPixelMappingDesignerView::GetHoveredComponentNameText() const
{
	if (UDMXPixelMappingOutputComponent* ComponentUnderCursor = GetComponentUnderCursor())
	{
		return FText::FromString(ComponentUnderCursor->GetUserName());
	}

	return FText();
}

FText SDMXPixelMappingDesignerView::GetHoveredComponentParentNameText() const
{
	if (UDMXPixelMappingOutputComponent* ComponentUnderCursor = GetComponentUnderCursor())
	{
		if (UDMXPixelMappingBaseComponent* ParentOfComponentUnderCursor = ComponentUnderCursor->GetParent())
		{
			return FText::FromString(ParentOfComponentUnderCursor->GetUserName());
		}
	}

	return FText();
}

EVisibility SDMXPixelMappingDesignerView::GetRulerVisibility() const
{
	return EVisibility::Visible;
}

FText SDMXPixelMappingDesignerView::GetCursorPositionText() const
{
	FVector2D CursorPosition;
	if (GetGraphSpaceCursorPosition(CursorPosition))
	{
		return FText::Format(LOCTEXT("CursorPositionFormat", "{0} x {1}"), FText::AsNumber(FMath::RoundToInt(CursorPosition.X)), FText::AsNumber(FMath::RoundToInt(CursorPosition.Y)));
	}
	return FText();
}

EVisibility SDMXPixelMappingDesignerView::GetCursorPositionTextVisibility() const
{
	return IsHovered() ? EVisibility::SelfHitTestInvisible : EVisibility::Collapsed;
}

EVisibility SDMXPixelMappingDesignerView::GetZoomPanVisibility() const
{
	if (UDMXPixelMappingRendererComponent* RendererComponent = CachedRendererComponent.Get())
	{
		if (RendererComponent->GetRenderedInputTexture())
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FReply SDMXPixelMappingDesignerView::OnZoomToFitClicked()
{
	ZoomToFit(/*bInstantZoom*/ false);
	return FReply::Handled();
}

void SDMXPixelMappingDesignerView::OnComponentAdded(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	CachedRendererComponent = nullptr;
}

void SDMXPixelMappingDesignerView::OnComponentRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	CachedRendererComponent = nullptr;
}

void SDMXPixelMappingDesignerView::OnSelectedComponentsChanged()
{
	CreateExtensionWidgetsForSelection();
}

void SDMXPixelMappingDesignerView::ResolvePendingSelectedComponents(bool bClearPreviousSelection)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	const bool bHasPendingSelectedComponent = PendingSelectedComponent.IsValid();
	if (Toolkit.IsValid() && bHasPendingSelectedComponent)
	{
		// Select matrix cells only if they're not locked in designer. By default they're locked.
		UDMXPixelMappingMatrixCellComponent* MatrixCellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(PendingSelectedComponent);
		if (MatrixCellComponent && MatrixCellComponent->IsLockInDesigner())
		{
			PendingSelectedComponent = MatrixCellComponent->GetParent();
		}

		// Select group if desired
		const FDMXPixelMappingDesignerSettings& DesignerSettings = GetDefault<UDMXPixelMappingEditorSettings>()->DesignerSettings;
		if (DesignerSettings.bAlwaysSelectGroup)
		{
			if (UDMXPixelMappingOutputDMXComponent* OutputDMXComponent = Cast<UDMXPixelMappingOutputDMXComponent>(PendingSelectedComponent))
			{
				PendingSelectedComponent = OutputDMXComponent->GetParent();
			}
		}

		// Add the newly selected component first. This is important, e.g. for drag drop when iterating and using this as base
		TSet<FDMXPixelMappingComponentReference> SelectedComponents;
		SelectedComponents.Add(Toolkit->GetReferenceFromComponent(PendingSelectedComponent.Get()));

		if (!bClearPreviousSelection)
		{
			SelectedComponents.Append(Toolkit->GetSelectedComponents());
		}
		Toolkit->SelectComponents(SelectedComponents);

		PendingSelectedComponent = nullptr;
	}
}

void SDMXPixelMappingDesignerView::PopulateWidgetGeometryCache(FArrangedWidget& Root)
{
	const FSlateRect Rect = PreviewHitTestRoot->GetTickSpaceGeometry().GetLayoutBoundingRect();
	const FSlateRect PaintRect = PreviewHitTestRoot->GetPaintSpaceGeometry().GetLayoutBoundingRect();

	PopulateWidgetGeometryCache_Loop(Root);
}

void SDMXPixelMappingDesignerView::PopulateWidgetGeometryCache_Loop(FArrangedWidget& CurrentWidget)
{
	FArrangedChildren ArrangedChildren(EVisibility::All);
	CurrentWidget.Widget->ArrangeChildren(CurrentWidget.Geometry, ArrangedChildren);

	CachedWidgetGeometry.Add(CurrentWidget.Widget, CurrentWidget);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];
		PopulateWidgetGeometryCache_Loop(SomeChild);
	}
}

UDMXPixelMappingOutputComponent* SDMXPixelMappingDesignerView::GetComponentUnderCursor() const
{
	UDMXPixelMappingOutputComponent* ComponentUnderCursor = nullptr;
	if (UDMXPixelMappingRendererComponent* RendererComponent = CachedRendererComponent.Get())
	{
		FVector2D GraphSpaceCursorPosition;
		if (GetGraphSpaceCursorPosition(GraphSpaceCursorPosition))
		{
			RendererComponent->ForEachChild([&ComponentUnderCursor, &GraphSpaceCursorPosition](UDMXPixelMappingBaseComponent* InComponent)
				{
					if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(InComponent))
					{
						if (!ComponentUnderCursor ||
							ComponentUnderCursor->GetZOrder() <= OutputComponent->GetZOrder())
						{
							if (OutputComponent->IsVisible() &&
								OutputComponent->IsOverPosition(GraphSpaceCursorPosition))
							{
								ComponentUnderCursor = OutputComponent;
							}
						}
					}
				}, true);
		}
	}

	return ComponentUnderCursor;
}

TSet<FDMXPixelMappingComponentReference> SDMXPixelMappingDesignerView::GetSelectedComponents() const
{
	if (TSharedPtr<FDMXPixelMappingToolkit> ToolkitPtr = WeakToolkit.Pin())
	{
		return ToolkitPtr->GetSelectedComponents();
	}

	return TSet<FDMXPixelMappingComponentReference>();
}

FDMXPixelMappingComponentReference SDMXPixelMappingDesignerView::GetSelectedComponent() const
{
	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = GetSelectedComponents();

	// Only return a selected widget when we have only a single item selected.
	if (SelectedComponents.Num() == 1)
	{
		return *(SelectedComponents.FindArbitraryElement());
	}

	return FDMXPixelMappingComponentReference();
}

bool SDMXPixelMappingDesignerView::GetComponentGeometry(FDMXPixelMappingComponentReference InComponentReference, FGeometry& OutGeometry)
{
	if (UDMXPixelMappingBaseComponent* ComponentPreview = InComponentReference.GetComponent())
	{
		return GetComponentGeometry(ComponentPreview, OutGeometry);
	}

	return false;
}

bool SDMXPixelMappingDesignerView::GetComponentGeometry(UDMXPixelMappingBaseComponent* InBaseComponent, FGeometry& OutGeometry)
{
	if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(InBaseComponent))
	{
		using namespace UE::DMX;
		const TSharedRef<IDMXPixelMappingOutputComponentWidgetInterface>* ComponentViewPtr = Algo::FindByPredicate(OutputComponentWidgets, [InBaseComponent](const TSharedRef<IDMXPixelMappingOutputComponentWidgetInterface>& ComponentWidget)
			{
				return ComponentWidget->Equals(InBaseComponent);
			});

		if (ComponentViewPtr)
		{
			const FArrangedWidget* ArrangedWidget = CachedWidgetGeometry.Find((*ComponentViewPtr)->AsWidget());
			if (ArrangedWidget)
			{
				OutGeometry = ArrangedWidget->Geometry;
				return true;
			}
		}
	}

	return false;
}

bool SDMXPixelMappingDesignerView::GetGraphSpaceCursorPosition(FVector2D& OutGraphSpaceCursorPosition) const
{
	if (const FArrangedWidget* PreviewSurface = CachedWidgetGeometry.Find(SourceTextureViewport.ToSharedRef()))
	{
		OutGraphSpaceCursorPosition = PreviewSurface->Geometry.AbsoluteToLocal(FSlateApplication::Get().GetCursorPos());

		return true;
	}
	return false;
}

FGeometry SDMXPixelMappingDesignerView::GetDesignerGeometry() const
{
	return PreviewHitTestRoot->GetTickSpaceGeometry();
}

FGeometry SDMXPixelMappingDesignerView::MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const
{
	FGeometry NewGeometry = WidgetGeometry;

	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	if (WidgetWindow.IsValid())
	{
		TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

		NewGeometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));
	}

	return NewGeometry;
}

void SDMXPixelMappingDesignerView::HandleDragEnterFromDetailsOrPalette(const TSharedPtr<FDMXPixelMappingDragDropOp>& TemplateDragDropOp)
{
	if (TSharedPtr<FDMXPixelMappingToolkit> ToolkitPtr = WeakToolkit.Pin())
	{
		if (TemplateDragDropOp.IsValid())
		{
			if (UDMXPixelMapping* PixelMapping = ToolkitPtr->GetDMXPixelMapping())
			{
				// Find the target on which the component should be created from its template
				UDMXPixelMappingBaseComponent* Target = (TemplateDragDropOp->Parent.IsValid()) ? TemplateDragDropOp->Parent.Get() : ToolkitPtr->GetActiveRendererComponent();

				if (Target && PixelMapping->RootComponent)
				{
					TArray<UDMXPixelMappingBaseComponent*> NewComponents;
					if (TemplateDragDropOp->GetDraggedComponents().Num() == 0)
					{
						// Create new components if they were first dragged in
						NewComponents = ToolkitPtr->CreateComponentsFromTemplates(PixelMapping->GetRootComponent(), Target, TemplateDragDropOp->GetTemplates());
					}
					else
					{
						// Use the existing components if the components are reentering
						for (const TWeakObjectPtr<UDMXPixelMappingBaseComponent>& BaseComponent : TemplateDragDropOp->GetDraggedComponents())
						{
							if (BaseComponent.IsValid())
							{
								NewComponents.Add(BaseComponent.Get());
							}
						}
					}

					// Build an array of all new componets for dragging
					TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>> DraggedComponents;
					for (UDMXPixelMappingBaseComponent* Component : NewComponents)
					{
						DraggedComponents.Add(Component);
					}

					TemplateDragDropOp->SetDraggedComponents(DraggedComponents);

					// Find if only matrix components and its childs are dragged
					TArray<UDMXPixelMappingBaseComponent*> NewMatrices;
					for (UDMXPixelMappingBaseComponent* NewComponent : NewComponents)
					{
						if (UDMXPixelMappingMatrixComponent* NewMatrix = Cast<UDMXPixelMappingMatrixComponent>(NewComponent))
						{
							NewMatrices.Add(NewMatrix);
						}
						else if (UDMXPixelMappingMatrixCellComponent* NewMatrixCell = Cast<UDMXPixelMappingMatrixCellComponent>(NewComponent))
						{
							continue;
						}
						else
						{
							NewMatrices.Reset();
							break;
						}
					}

					// Update selection depending on types
					TSet<FDMXPixelMappingComponentReference> NewSelection;
					if (NewMatrices.Num() > 0)
					{
						// When only matrices and its childs are dragged, just select the matrix
						for (UDMXPixelMappingBaseComponent* NewMatrix : NewMatrices)
						{
							NewSelection.Add(ToolkitPtr->GetReferenceFromComponent(NewMatrix));
						}
					}
					else
					{
						// Select all new components
						for (UDMXPixelMappingBaseComponent* NewComponent : NewComponents)
						{
							NewSelection.Add(ToolkitPtr->GetReferenceFromComponent(NewComponent));
						}
					}

					ToolkitPtr->SelectComponents(NewSelection);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
