// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/SDMXPixelMappingSurface.h"

class FDMXPixelMappingDragDropOp;
class FDMXPixelMappingToolkit;
class SBorder;
class SBox;
class SCanvas;
class SConstraintCanvas;
class SDMXPixelMappingRuler;
class SDMXPixelMappingSourceTextureViewport;
class SDMXPixelMappingTransformHandle;
class SDMXPixelMappingZoomPan;
class SOverlay;
class UDMXPixelMapping;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingOutputComponent;
class UDMXPixelMappingRendererComponent;
namespace UE::DMX { class IDMXPixelMappingOutputComponentWidgetInterface; }
struct FDMXPixelMappingComponentReference;

class SDMXPixelMappingDesignerView
	: public SDMXPixelMappingSurface
	, public FSelfRegisteringEditorUndoClient
{
private:
	struct FComponentHitResult
	{
	public:
		TWeakObjectPtr<UDMXPixelMappingBaseComponent> Component;
		FArrangedWidget WidgetArranged;
		FName NamedSlot;

	public:
		FComponentHitResult()
			: WidgetArranged(SNullWidget::NullWidget, FGeometry())
		{}
	};

public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingDesignerView) 
	{}
	
	SLATE_END_ARGS()

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit);

	/** Returns the geometry of the graph */
	const FGeometry& GetGraphTickSpaceGeometry() const;

protected:
	//~ Begin SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	//~ End of SWidget interface

	//~ Begin SDMXPixelMappingSurface interface
	virtual FSlateRect ComputeAreaBounds() const override;
	virtual int32 GetGraphRulePeriod() const override;
	virtual float GetGridScaleAmount() const override;
	virtual int32 GetGridSize() const override;
	//~ End SDMXPixelMappingSurface interface

	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient interface

private:
	/** Rebuilds the designer widget */
	void RebuildDesigner();

	/** Creates the extension widgets for the selected component */
	void CreateExtensionWidgetsForSelection();

	/** Clears the extension widgets */
	void ClearExtensionWidgets();

	/** Opens the designer context menu */
	void OpenContextMenu();

	/** Returns if the extension widget canvas should be visible */
	EVisibility GetExtensionCanvasVisibility() const;

	/** Gets the positioin of the extension widget */
	FVector2D GetExtensionPosition(TSharedPtr<SDMXPixelMappingTransformHandle> Handle);

	/** Gets the size of the extension widget */
	FVector2D GetExtensionSize(TSharedPtr<SDMXPixelMappingTransformHandle> Handle);

	/** Returns text for the currently hovered component */
	FText GetHoveredComponentNameText() const;

	/** Returns text for the currently hovered component's parent */
	FText GetHoveredComponentParentNameText() const;

	/** Returns the visibility of the side rulers */
	EVisibility GetRulerVisibility() const;

	/** Returns the cursor position as text */
	FText GetCursorPositionText() const;

	/** Returns the visibility of the cursor position text block */
	EVisibility GetCursorPositionTextVisibility() const;

	/** Returns the visibility of the ZoomPan widget */
	EVisibility GetZoomPanVisibility() const;

	/** Called when zoom to fit was clicked */
	FReply OnZoomToFitClicked();

	/** Called when a component was added */
	void OnComponentAdded(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** Called when a component was removed */
	void OnComponentRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** Called when the selected component in the toolkit changed */
	void OnSelectedComponentsChanged();

	/** Adds any pending selected Components to the selection set */
	void ResolvePendingSelectedComponents(bool bClearPreviousSelection = false);

	void PopulateWidgetGeometryCache(FArrangedWidget& Root);

	void PopulateWidgetGeometryCache_Loop(FArrangedWidget& Parent);

	/** Returns the component under the cursor */
	UDMXPixelMappingOutputComponent* GetComponentUnderCursor() const;

	/** Returns the selected components */
	TSet<FDMXPixelMappingComponentReference> GetSelectedComponents() const;

	/** Returns the first best selected component */
	FDMXPixelMappingComponentReference GetSelectedComponent() const;

	/** Gets the Geometry of the component, returns true if successful */
	bool GetComponentGeometry(FDMXPixelMappingComponentReference InComponentReference, FGeometry& OutGeometry);

	/** Gets the Geometry of the component, returns true if successful */
	bool GetComponentGeometry(UDMXPixelMappingBaseComponent* InBaseComponent, FGeometry& OutGeometry);

	/** Returns the cursor position in graph space */
	bool GetGraphSpaceCursorPosition(FVector2D& OutGraphSpaceCursorPosition) const;

	FGeometry GetDesignerGeometry() const;

	FGeometry MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const;

	/** Updates the drag drop op to use the patches selected in the Palette */
	void HandleDragEnterFromDetailsOrPalette(const TSharedPtr<FDMXPixelMappingDragDropOp>& DragDropOp);

	/** The component that should be selected, but isn't selected yet */
	TWeakObjectPtr<UDMXPixelMappingBaseComponent> PendingSelectedComponent;

	/** The drag over DragDropOp that should be handled on the next tick*/
	TSharedPtr<FDMXPixelMappingDragDropOp> PendingDragDropOp;

	/** Canvas that holds the extension widget */
	TSharedPtr<SCanvas> ExtensionWidgetCanvas;

	/** Widget that holds the source texture */
	TSharedPtr<SDMXPixelMappingSourceTextureViewport> SourceTextureViewport;

	/** Canvas that holds the component widgets */
	TSharedPtr<SConstraintCanvas> DesignCanvas;

	TSharedPtr<SOverlay> PreviewHitTestRoot;

	/** Borer that contains the Design Canvas */
	TSharedPtr<SBorder> DesignCanvasBorder;

	/** Cached cursor position in graph space */
	FVector2D CursorPositionGraphSpace = FVector2D::ZeroVector;

	TMap<TWeakPtr<SWidget>, FArrangedWidget> CachedWidgetGeometry;

	/** Cached current renderer component */
	TWeakObjectPtr<UDMXPixelMappingRendererComponent> CachedRendererComponent;

	/** The widget that handles zoom and pan */
	TSharedPtr<SDMXPixelMappingZoomPan> ZoomPan;

	/** The ruler bar at the top of the designer. */
	TSharedPtr<SDMXPixelMappingRuler> TopRuler;

	/** The ruler bar on the left side of the designer. */
	TSharedPtr<SDMXPixelMappingRuler> SideRuler;

	/** The position in local space where the user began dragging a widget */
	FVector2D DragAnchor;

	/** Output component views currently displayed */
	TArray<TSharedRef<UE::DMX::IDMXPixelMappingOutputComponentWidgetInterface>> OutputComponentWidgets;

	TArray<TSharedPtr<SDMXPixelMappingTransformHandle>> TransformHandles;
};
