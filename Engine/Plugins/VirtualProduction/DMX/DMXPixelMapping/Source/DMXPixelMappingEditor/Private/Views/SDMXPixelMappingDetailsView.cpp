// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDMXPixelMappingDetailsView.h"

#include "ColorSpace/DMXPixelMappingColorSpace_RGBCMY.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingOutputDMXComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "Customizations/DMXPixelMappingColorSpaceDetails_RGBCMY.h"
#include "Customizations/DMXPixelMappingDetailCustomization_FixtureGroup.h"
#include "Customizations/DMXPixelMappingDetailCustomization_FixtureGroupItem.h"
#include "Customizations/DMXPixelMappingDetailCustomization_Matrix.h"
#include "Customizations/DMXPixelMappingDetailCustomization_OutputDMX.h"
#include "Customizations/DMXPixelMappingDetailCustomization_Renderer.h"
#include "DetailsViewArgs.h"
#include "DMXPixelMappingComponentReference.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "Toolkits/DMXPixelMappingToolkit.h"

void SDMXPixelMappingDetailsView::Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;

	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;

	DetailsView = EditModule.CreateDetailView(DetailsViewArgs);

	RegisterCustomizations();

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			DetailsView.ToSharedRef()
		]
	];

	InToolkit->GetOnSelectedComponentsChangedDelegate().AddSP(this, &SDMXPixelMappingDetailsView::OnSelectedComponentsChanged);

	OnSelectedComponentsChanged();
}

void SDMXPixelMappingDetailsView::OnSelectedComponentsChanged()
{
	// Clear selection in the property view.
	SelectedObjects.Empty();
	DetailsView->SetObjects(SelectedObjects);

	TSharedPtr<FDMXPixelMappingToolkit> ToolkitPtr = WeakToolkit.Pin();
	check(ToolkitPtr.IsValid());

	// Add any selected widgets to the list of pending selected objects.
	TSet<FDMXPixelMappingComponentReference> SelectedComponents = ToolkitPtr->GetSelectedComponents();
	if (!SelectedComponents.IsEmpty())
	{
		for (const FDMXPixelMappingComponentReference& ComponentRef : SelectedComponents)
		{
			if (UDMXPixelMappingBaseComponent* Component = ComponentRef.GetComponent())
			{
				SelectedObjects.Add(ComponentRef.GetComponent());
			}
		}
	}

	// Update the preview view to look at the current selection set.
	const bool bForceRefresh = false;

	bool bHideFilterArea = false;
	for (const TWeakObjectPtr<UObject> Object : SelectedObjects)
	{
		if (UObject* Obj = Object.Get())
		{
			if (Cast<UDMXPixelMappingRootComponent>(Obj))
			{
				bHideFilterArea = true;
				break;
			}
		}
	}

	DetailsView->HideFilterArea(bHideFilterArea);
	DetailsView->SetObjects(SelectedObjects, bForceRefresh);
}

void SDMXPixelMappingDetailsView::RegisterCustomizations()
{
	FOnGetDetailCustomizationInstance RendererCustomizationInstance = FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingDetailCustomization_Renderer::MakeInstance, WeakToolkit);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingRendererComponent::StaticClass(), RendererCustomizationInstance);

	FOnGetDetailCustomizationInstance FixtureGroupCustomizationInstance = FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingDetailCustomization_FixtureGroup::MakeInstance, WeakToolkit);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingFixtureGroupComponent::StaticClass(), FixtureGroupCustomizationInstance);

	FOnGetDetailCustomizationInstance OutputDMXCustomizationInstance = FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingDetailCustomization_OutputDMX::MakeInstance, WeakToolkit);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingOutputDMXComponent::StaticClass(), OutputDMXCustomizationInstance);

	FOnGetDetailCustomizationInstance FixtureGroupItemCustomizationInstance = FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingDetailCustomization_FixtureGroupItem::MakeInstance, WeakToolkit);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingFixtureGroupItemComponent::StaticClass(), FixtureGroupItemCustomizationInstance);

	FOnGetDetailCustomizationInstance MatrixCustomizationInstance = FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingDetailCustomization_Matrix::MakeInstance, WeakToolkit);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingMatrixComponent::StaticClass(), MatrixCustomizationInstance);

	FOnGetDetailCustomizationInstance ColorSpaceCustomizationInstance_RGBCMY = FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingColorSpaceDetails_RGBCMY::MakeInstance);
	DetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingColorSpace_RGBCMY::StaticClass(), ColorSpaceCustomizationInstance_RGBCMY);
}
