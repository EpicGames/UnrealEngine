// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FDMXPixelMappingToolkit;
class IDetailsView;
class UObject;


class SDMXPixelMappingDetailsView
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDMXPixelMappingDetailsView) { }
	SLATE_END_ARGS()

public:
	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit);

private:
	void OnSelectedComponentsChanged();

	/** Registers the designer specific customizations */
	void RegisterCustomizations();

	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;

	/** The details view that is displayed */
	TSharedPtr<IDetailsView> DetailsView;

	/** Selected objects for this detail view */
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
};
