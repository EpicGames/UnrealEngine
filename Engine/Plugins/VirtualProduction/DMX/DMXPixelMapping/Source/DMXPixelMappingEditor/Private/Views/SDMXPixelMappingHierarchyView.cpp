// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDMXPixelMappingHierarchyView.h"

#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorUtils.h"
#include "Exporters/Exporter.h"
#include "Factories.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Framework/Views/TreeFilterHandler.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/TextFilter.h"
#include "ScopedTransaction.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "UnrealExporter.h"
#include "ViewModels/DMXPixelMappingHierarchyItem.h"
#include "Views/SDMXPixelMappingDesignerView.h"
#include "Views/SDMXPixelMappingPreviewView.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/SDMXPixelMappingHierarchyRow.h"
#include "Widgets/Views/STreeView.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingHierarchyView"

namespace UE::DMX::PixelMappingEditor::SDMXPixelMappingHierarchyView::Private
{
	/** Helper to create new components from existing */
	class FBaseComponentTextFactory
		: public FCustomizableTextObjectFactory
	{
	public:

		FBaseComponentTextFactory()
			: FCustomizableTextObjectFactory(GWarn)
		{}

		// FCustomizableTextObjectFactory implementation
		virtual bool CanCreateClass(UClass* InObjectClass, bool& bOmitSubObjs) const override
		{
			return InObjectClass->IsChildOf<UDMXPixelMappingBaseComponent>();
		}

		virtual void ProcessConstructedObject(UObject* NewObject) override
		{
			check(NewObject);

			if (NewObject->IsA<UDMXPixelMappingBaseComponent>())
			{
				UDMXPixelMappingBaseComponent* DMXPixelMappingBaseComponent = Cast<UDMXPixelMappingBaseComponent>(NewObject);
				DMXPixelMappingBaseComponents.Add(DMXPixelMappingBaseComponent);
			}
		}

	public:
		TArray<UDMXPixelMappingBaseComponent*> DMXPixelMappingBaseComponents;
	};

	/** Helper to adopt the expansion of items in theh hierarchy treeview */
	void AdoptItemExpansionFromTree(const TSharedRef<STreeView<TSharedPtr<FDMXPixelMappingHierarchyItem>>>& TreeView, const TSharedPtr<FDMXPixelMappingHierarchyItem>& Parent)
	{
		TreeView->SetItemExpansion(Parent, Parent->IsExpanded());
		for (const TSharedPtr<FDMXPixelMappingHierarchyItem>& Child : Parent->GetChildren())
		{
			TreeView->SetItemExpansion(Child, Child->IsExpanded());
			AdoptItemExpansionFromTree(TreeView, Child);
		}
	}
}

const FName SDMXPixelMappingHierarchyView::FColumnIds::EditorColor = "EditorColor";
const FName SDMXPixelMappingHierarchyView::FColumnIds::ComponentName = "Name";
const FName SDMXPixelMappingHierarchyView::FColumnIds::FixtureID = "FixtureID";
const FName SDMXPixelMappingHierarchyView::FColumnIds::Patch = "Patch";

void SDMXPixelMappingHierarchyView::Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;

	CommandList = MakeShareable(new FUICommandList);

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SDMXPixelMappingHierarchyView::RenameSelectedComponent),
		FCanExecuteAction::CreateSP(this, &SDMXPixelMappingHierarchyView::CanRenameSelectedComponent)
	);

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SDMXPixelMappingHierarchyView::DeleteSelectedComponents)
	);

	using TextFilterType = TTextFilter<TSharedPtr<FDMXPixelMappingHierarchyItem>>;
	SearchFilter = MakeShared<TextFilterType>(TextFilterType::FItemToStringArray::CreateSP(this, &SDMXPixelMappingHierarchyView::GetWidgetFilterStrings));

	FilterHandler = MakeShared<TreeFilterHandler<TSharedPtr<FDMXPixelMappingHierarchyItem>>>();
	FilterHandler->SetFilter(SearchFilter.Get());
	FilterHandler->SetRootItems(&AllRootItems, &FilteredRootItems);
	FilterHandler->SetGetChildrenDelegate(TreeFilterHandler<TSharedPtr<FDMXPixelMappingHierarchyItem>>::FOnGetChildren::CreateSP(this, &SDMXPixelMappingHierarchyView::OnGetChildItems));

	InToolkit->GetOnSelectedComponentsChangedDelegate().AddSP(this, &SDMXPixelMappingHierarchyView::OnEditorSelectionChanged);
	UDMXPixelMappingBaseComponent::GetOnComponentAdded().AddSP(this, &SDMXPixelMappingHierarchyView::OnComponentAddedOrRemoved);
	UDMXPixelMappingBaseComponent::GetOnComponentRemoved().AddSP(this, &SDMXPixelMappingHierarchyView::OnComponentAddedOrRemoved);

	BuildChildSlotAndRefresh();
}

void SDMXPixelMappingHierarchyView::RequestRefresh()
{
	if (!RequestRefreshTimerHandle.IsValid())
	{
		RequestRefreshTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXPixelMappingHierarchyView::ForceRefresh));
	}
}

void SDMXPixelMappingHierarchyView::ForceRefresh()
{
	if (!ensureMsgf(HierarchyTreeView.IsValid(), TEXT("Cannot refresh the hierarchy tree view before BuildChildSlotAndRefresh() was called.")))
	{
		return;
	}

	RequestRefreshTimerHandle.Invalidate();

	// Create the root, and let it construct new tree items
	AllRootItems.Reset();
	AllRootItems.Add(FDMXPixelMappingHierarchyItem::CreateNew(WeakToolkit.Pin()));

	// Adopt expansion of new items from the old tree
	using namespace UE::DMX::PixelMappingEditor::SDMXPixelMappingHierarchyView::Private;
	for (const TSharedPtr<FDMXPixelMappingHierarchyItem>& RootItem : AllRootItems)
	{
		AdoptItemExpansionFromTree(HierarchyTreeView.ToSharedRef(), RootItem);
	}

	FilterHandler->RefreshAndFilterTree();
	AdoptSelectionFromToolkit();
}

void SDMXPixelMappingHierarchyView::BuildChildSlotAndRefresh()
{
	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(8.f, 0.f, 0.f, 0.f)
				[
					SNew(SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
					.HasDownArrow(true)
					.OnGetMenuContent(this, &SDMXPixelMappingHierarchyView::GenerateHeaderRowFilterMenu)
					.ButtonContent()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
	
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[	
					SAssignNew(SearchBox, SSearchBox)
					.OnTextChanged(this, &SDMXPixelMappingHierarchyView::SetFilterText)
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			.Padding(4.f)
			[
				SAssignNew(HierarchyTreeView, STreeView<TSharedPtr<FDMXPixelMappingHierarchyItem>>)
				.SelectionMode(ESelectionMode::Multi)
				.HeaderRow(GenerateHeaderRow())
				.OnGetChildren(FilterHandler.ToSharedRef(), &TreeFilterHandler<TSharedPtr<FDMXPixelMappingHierarchyItem>>::OnGetFilteredChildren)
				.OnGenerateRow(this, &SDMXPixelMappingHierarchyView::OnGenerateRow)
				.OnSelectionChanged(this, &SDMXPixelMappingHierarchyView::OnHierarchySelectionChanged)
				.OnExpansionChanged(this, &SDMXPixelMappingHierarchyView::OnHierarchyExpansionChanged)
				.OnContextMenuOpening(this, &SDMXPixelMappingHierarchyView::OnContextMenuOpening)
				.TreeItemsSource(&FilteredRootItems)
				.OnItemToString_Debug_Lambda([this](TSharedPtr<FDMXPixelMappingHierarchyItem> Item) 
					{ 
						return Item->GetComponentNameText().ToString(); 
					})
			]
		];

	FilterHandler->SetTreeView(HierarchyTreeView.Get());

	ForceRefresh();
}

TSharedRef<SHeaderRow> SDMXPixelMappingHierarchyView::GenerateHeaderRow()
{
	const TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);
	const FDMXPixelMappingHierarchySettings& HierarchySettings = GetDefault<UDMXPixelMappingEditorSettings>()->HierarchySettings;

	if (HierarchySettings.bShowEditorColorColumn)
	{
		HeaderRow->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(FColumnIds::EditorColor)
			.DefaultLabel(FText())
			.FixedWidth(16.f)
			.VAlignHeader(VAlign_Center)
		);
	}
	
	HeaderRow->AddColumn(
		SHeaderRow::FColumn::FArguments()
		.ColumnId(FColumnIds::ComponentName)
		.SortMode(this, &SDMXPixelMappingHierarchyView::GetColumnSortMode, FColumnIds::ComponentName)
		.OnSort(this, &SDMXPixelMappingHierarchyView::SetSortAndRefresh)
		.FillWidth(0.68f)
		.HeaderContentPadding(FMargin(6.f))
		.VAlignHeader(VAlign_Center)
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("FixturePatchNameColumnLabel", "Name"))
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		]
	);

	if (HierarchySettings.bShowFixtureIDColumn)
	{
		HeaderRow->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(FColumnIds::FixtureID)
			.SortMode(this, &SDMXPixelMappingHierarchyView::GetColumnSortMode, FColumnIds::FixtureID)
			.OnSort(this, &SDMXPixelMappingHierarchyView::SetSortAndRefresh)
			.FillWidth(0.16f)
			.HeaderContentPadding(FMargin(6.f))
			.VAlignHeader(VAlign_Center)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FixtureIDColumnLabel", "FID"))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		);
	}

	if (HierarchySettings.bShowPatchColumn)
	{
		HeaderRow->AddColumn(
			SHeaderRow::FColumn::FArguments()
			.ColumnId(FColumnIds::Patch)
			.SortMode(this, &SDMXPixelMappingHierarchyView::GetColumnSortMode, FColumnIds::Patch)
			.OnSort(this, &SDMXPixelMappingHierarchyView::SetSortAndRefresh)
			.FillWidth(0.16f)
			.HeaderContentPadding(FMargin(6.f))
			.VAlignHeader(VAlign_Center)
			.HeaderContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PatchColumnLabel", "Patch"))
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
		);
	}

	return HeaderRow;
}

TSharedRef<SWidget> SDMXPixelMappingHierarchyView::GenerateHeaderRowFilterMenu()
{
	constexpr bool bShouldCloseMenuAfterSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("FilterSection", "Columns"));
	{
		auto AddMenuEntryLambda = [this, &MenuBuilder](const FText& Label, const FText& ToolTip, const FName& ColumnID)
		{
			MenuBuilder.AddMenuEntry(
				Label,
				ToolTip,
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SDMXPixelMappingHierarchyView::ToggleColumnVisility, ColumnID),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SDMXPixelMappingHierarchyView::IsColumVisible, ColumnID)
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		};

		AddMenuEntryLambda(
			LOCTEXT("FixturePatchNameColumn_Label", "Show Editor Color"),
			FText::GetEmpty(),
			FColumnIds::EditorColor
		);

		AddMenuEntryLambda(
			LOCTEXT("FixturePatchTypeColumn_Label", "Show Fixture ID"),
			FText::GetEmpty(),
			FColumnIds::FixtureID
		);

		AddMenuEntryLambda(
			LOCTEXT("FixturePatchModeColumn_Label", "Show Patch"),
			FText::GetEmpty(),
			FColumnIds::Patch
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FReply SDMXPixelMappingHierarchyView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SDMXPixelMappingHierarchyView::PostUndo(bool bSuccess)
{
	RequestRefresh();
}

void SDMXPixelMappingHierarchyView::PostRedo(bool bSuccess)
{ 
	RequestRefresh();
}

void SDMXPixelMappingHierarchyView::OnGetChildItems(TSharedPtr<FDMXPixelMappingHierarchyItem> InParent, TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>>& OutChildren)
{
	// If the parent is a fixture group, sort its children using the current sort order
	if (InParent.IsValid() && 
		InParent->GetComponent() && 
		InParent->GetComponent()->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass())
	{
		const FDMXPixelMappingHierarchySettings& HierarchySettings = GetDefault<UDMXPixelMappingEditorSettings>()->HierarchySettings;
		if (HierarchySettings.SortByColumnId == FColumnIds::ComponentName)
		{
			InParent->StableSortChildren([](const TSharedPtr<FDMXPixelMappingHierarchyItem>& Item)
				{
					return Item->GetComponentNameText().ToString();
				});
		}
		else if (HierarchySettings.SortByColumnId == FColumnIds::Patch)
		{
			InParent->StableSortChildren([](const TSharedPtr<FDMXPixelMappingHierarchyItem>& Item)
				{
					return Item->GetAbsoluteChannel();
				});
		}

		if (!HierarchySettings.bSortAscending)
		{
			InParent->ReverseChildren();
		}
	}

	OutChildren = InParent->GetChildren();
}

TSharedRef<ITableRow> SDMXPixelMappingHierarchyView::OnGenerateRow(TSharedPtr<FDMXPixelMappingHierarchyItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SDMXPixelMappingHierarchyRow, OwnerTable, WeakToolkit, Item.ToSharedRef());
}

TSharedPtr<SWidget> SDMXPixelMappingHierarchyView::OnContextMenuOpening()
{
	if (!WeakToolkit.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.BeginSection("Edit", LOCTEXT("Edit", "Edit"));
	{
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Rename);
		MenuBuilder.AddMenuEntry(FGenericCommands::Get().Delete);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SDMXPixelMappingHierarchyView::OnHierarchySelectionChanged(TSharedPtr<FDMXPixelMappingHierarchyItem> SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}

	const TGuardValue<bool> GuardIsUpdatingSelection(bIsUpdatingSelection, true);

	if (TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin())
	{
		TSet<FDMXPixelMappingComponentReference> ComponentsToSelect;
		const TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> SelectedItems = HierarchyTreeView->GetSelectedItems();
		for (const TSharedPtr<FDMXPixelMappingHierarchyItem>& Item : SelectedItems)
		{
			ComponentsToSelect.Add(FDMXPixelMappingComponentReference(Toolkit, Item->GetComponent()));
		}

		Toolkit->SelectComponents(ComponentsToSelect);
	}
}

void SDMXPixelMappingHierarchyView::OnHierarchyExpansionChanged(TSharedPtr<FDMXPixelMappingHierarchyItem> Item, bool bExpanded)
{
	Item->SetIsExpanded(bExpanded);
}

void SDMXPixelMappingHierarchyView::ToggleColumnVisility(FName ColumnId)
{
	UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>();
	if (ColumnId == FColumnIds::EditorColor)
	{
		EditorSettings->HierarchySettings.bShowEditorColorColumn = !EditorSettings->HierarchySettings.bShowEditorColorColumn;
	}
	else if (ColumnId == FColumnIds::FixtureID)
	{
		EditorSettings->HierarchySettings.bShowFixtureIDColumn = !EditorSettings->HierarchySettings.bShowFixtureIDColumn;
	}
	else if (ColumnId == FColumnIds::Patch)
	{
		EditorSettings->HierarchySettings.bShowPatchColumn = !EditorSettings->HierarchySettings.bShowPatchColumn;
	}
	EditorSettings->SaveConfig();

	// To adopt column changes, the entire child slot needs to be updated
	BuildChildSlotAndRefresh();
}

bool SDMXPixelMappingHierarchyView::IsColumVisible(FName ColumnId) const
{
	UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>();
	if (ColumnId == FColumnIds::EditorColor)
	{
		return EditorSettings->HierarchySettings.bShowEditorColorColumn;
	}
	else if (ColumnId == FColumnIds::FixtureID)
	{
		return EditorSettings->HierarchySettings.bShowFixtureIDColumn;
	}
	else if (ColumnId == FColumnIds::Patch)
	{
		return EditorSettings->HierarchySettings.bShowPatchColumn;
	}

	return false;
}

EColumnSortMode::Type SDMXPixelMappingHierarchyView::GetColumnSortMode(FName ColumnId) const
{
	const FDMXPixelMappingHierarchySettings& HierarchySettings = GetDefault<UDMXPixelMappingEditorSettings>()->HierarchySettings;
	if (HierarchySettings.SortByColumnId != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return HierarchySettings.bSortAscending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
}

void SDMXPixelMappingHierarchyView::SetSortAndRefresh(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	UDMXPixelMappingEditorSettings* EditorSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>();

	EditorSettings->HierarchySettings.bSortAscending = InSortMode == EColumnSortMode::Ascending;
	EditorSettings->HierarchySettings.SortByColumnId = ColumnId;
	EditorSettings->SaveConfig();

	RequestRefresh();
}

void SDMXPixelMappingHierarchyView::OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	RequestRefresh();
}

void SDMXPixelMappingHierarchyView::OnEditorSelectionChanged()
{
	if (!bIsUpdatingSelection)
	{
		AdoptSelectionFromToolkit();
	}
}

void SDMXPixelMappingHierarchyView::AdoptSelectionFromToolkit()
{
	if (bIsUpdatingSelection || !HierarchyTreeView.IsValid())
	{
		return;
	}

	const TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> OldSelection = HierarchyTreeView->GetSelectedItems();
	for (const TSharedPtr<FDMXPixelMappingHierarchyItem>& RootItem : AllRootItems)
	{
		RecursiveAdoptSelectionFromToolkit(RootItem.ToSharedRef());
	}

	const TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> NewSelection = HierarchyTreeView->GetSelectedItems();
	const bool bOldSelectionContainsNewSelection = Algo::FindByPredicate(NewSelection, [&OldSelection](const TSharedPtr<FDMXPixelMappingHierarchyItem>& SelectedItem)
		{
			return OldSelection.Contains(SelectedItem);
		}) != nullptr;

	if (!bOldSelectionContainsNewSelection && !NewSelection.IsEmpty())
	{
		HierarchyTreeView->RequestScrollIntoView(NewSelection[0]);
	}
}

void SDMXPixelMappingHierarchyView::RecursiveAdoptSelectionFromToolkit(const TSharedRef<FDMXPixelMappingHierarchyItem>& Item)
{
	if (!WeakToolkit.IsValid() || !HierarchyTreeView.IsValid())
	{
		return;
	}

	for (const TSharedPtr<FDMXPixelMappingHierarchyItem>& Child : Item->GetChildren())
	{
		RecursiveAdoptSelectionFromToolkit(Child.ToSharedRef());
	}

	const bool bSelected = Algo::FindByPredicate(WeakToolkit.Pin()->GetSelectedComponents(),
		[Item](const FDMXPixelMappingComponentReference& ComponentReference)
		{
			return ComponentReference.GetComponent() && ComponentReference.GetComponent() == Item->GetComponent();
		}) != nullptr;
	
	HierarchyTreeView->SetItemSelection(Item, bSelected);
}

bool SDMXPixelMappingHierarchyView::CanRenameSelectedComponent() const
{
	return HierarchyTreeView.IsValid() ? HierarchyTreeView->GetSelectedItems().Num() == 1 : false;
}

void SDMXPixelMappingHierarchyView::RenameSelectedComponent()
{
	const TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> SelectedItems = HierarchyTreeView.IsValid() ? HierarchyTreeView->GetSelectedItems() : TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>>();
	if (!ensureMsgf(SelectedItems.Num() == 1, TEXT("Cannot rename selected components. Please call CanRenameSelectedComponent() before calling RenameSelectedComponent().")))
	{
		return;
	}

	const TSharedPtr<ITableRow> TableRow = HierarchyTreeView->WidgetFromItem(SelectedItems[0]);
	if (!ensureMsgf(TableRow.IsValid(), TEXT("Cannot find widget for item. Cannot rename component")))
	{
		return;
	}

	StaticCastSharedPtr<SDMXPixelMappingHierarchyRow>(TableRow)->EnterRenameMode();
}

void SDMXPixelMappingHierarchyView::DeleteSelectedComponents()
{
	if (TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin())
	{
		const int32 NumSelectedComponents = Toolkit->GetSelectedComponents().Num();
		const FScopedTransaction Transaction(FText::Format(LOCTEXT("DMXPixelMapping.RemoveComponents", "Remove {0}|plural(one=Component, other=Components)"), NumSelectedComponents));

		Toolkit->DeleteSelectedComponents();
	}
}

void SDMXPixelMappingHierarchyView::SetFilterText(const FText& Text)
{
	FilterHandler->SetIsEnabled(!Text.IsEmpty());
	SearchFilter->SetRawFilterText(Text);
	FilterHandler->RefreshAndFilterTree();
}

void SDMXPixelMappingHierarchyView::GetWidgetFilterStrings(TSharedPtr<FDMXPixelMappingHierarchyItem> InModel, TArray<FString>& OutStrings) const
{
	OutStrings.Add(InModel->GetComponentNameText().ToString());
	OutStrings.Add(InModel->GetFixtureIDText().ToString());
	OutStrings.Add(InModel->GetPatchText().ToString());
}

#undef LOCTEXT_NAMESPACE
