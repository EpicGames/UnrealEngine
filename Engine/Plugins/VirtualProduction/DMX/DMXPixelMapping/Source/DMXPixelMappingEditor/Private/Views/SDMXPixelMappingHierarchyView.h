// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingEditorLog.h"
#include "EditorUndoClient.h"
#include "Engine/EngineTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SHeaderRow.h"

class FDMXPixelMappingHierarchyItem;
class FDMXPixelMappingToolkit;
class FUICommandList;
class ITableRow;
class SHeaderRow;
class SSearchBox;
class STableViewBase;
template<typename ItemType> class STreeView;
template<typename ItemType> class TreeFilterHandler;
template <typename ItemType> class TTextFilter;
class UDMXPixelMapping;
class UDMXPixelMappingBaseComponent;



class SDMXPixelMappingHierarchyView final
	: public SCompoundWidget
	, public FSelfRegisteringEditorUndoClient 
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingHierarchyView) { }
	SLATE_END_ARGS()
		
	/** ColumnIds for the tree view */
	struct FColumnIds
	{
		static const FName EditorColor;
		static const FName ComponentName;
		static const FName FixtureID;
		static const FName Patch;
	};

	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit);

	/** Refreshes the widget on the next tick */
	void RequestRefresh();

private:
	/** Refreshes the widget */
	void ForceRefresh();

	/** Builds the child slot anew and refreshes the widget */
	void BuildChildSlotAndRefresh();

	/** Generates the header row for the tree view */
	TSharedRef<SHeaderRow> GenerateHeaderRow();

	/** Generates the header row filter menu */
	TSharedRef<SWidget> GenerateHeaderRowFilterMenu();
	
	//~ Begin SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	//~ Begin FEditorUndoClient interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	//~ End FEditorUndoClient interface

	/** Called to get child items */
	void OnGetChildItems(TSharedPtr<FDMXPixelMappingHierarchyItem> InParent, TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>>& OutChildren);

	/** Called when a row is generated */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FDMXPixelMappingHierarchyItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called when the context menu is opening */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Called when selection changed in the hierarchy */
	void OnHierarchySelectionChanged(TSharedPtr<FDMXPixelMappingHierarchyItem> Item, ESelectInfo::Type SelectInfo);

	/** Called when selection changed in the hierarchy */
	void OnHierarchyExpansionChanged(TSharedPtr<FDMXPixelMappingHierarchyItem> Item, bool bExpanded);

	/** Toggles the visibility of a column */
	void ToggleColumnVisility(FName ColumnId);

	/** Returns true if the specified column is currently visible */
	bool IsColumVisible(FName ColumnId) const;

	/** Gets the sort mode for the given column ID */
	EColumnSortMode::Type GetColumnSortMode(FName ColumnId) const;

	/** Set the current sorting state of the hierarchy tree view and refreshes the tree, applying the new sorting */
	void SetSortAndRefresh(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/** Called when a component was added or removed from the pixel mapping */
	void OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** Called when the selected widget has changed.  The treeview then needs to match the new selection. */
	void OnEditorSelectionChanged();

	/** Adopts the current selection of the toolkit */
	void AdoptSelectionFromToolkit();

	/** Adopts the selection from toolkit */
	void RecursiveAdoptSelectionFromToolkit(const TSharedRef<FDMXPixelMappingHierarchyItem>& Model);

	/** Returns true if the current selection can to be renamed */
	bool CanRenameSelectedComponent() const;

	/** Renames selected components */
	void RenameSelectedComponent();

	/** Returns true if the current selection can to be deleted */
	bool CanDeleteSelectedComponents() const;

	/** Deletes selected components */
	void DeleteSelectedComponents();

	/** Sets the text for the filter */
	void SetFilterText(const FText& Text);

	/**  Gets an array of strings used for filtering/searching the specified widget. */
	void GetWidgetFilterStrings(TSharedPtr<FDMXPixelMappingHierarchyItem> InModel, TArray<FString>& OutStrings) const;

	/** Flag to ignore selections while the hierarchy view is updating the selection. */
	bool bIsUpdatingSelection = false;

	/** All items in the tree */
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> TreeItems;

	/** Root items being displayed */
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> FilteredRootItems;

	/** All root items */
	TArray<TSharedPtr<FDMXPixelMappingHierarchyItem>> AllRootItems;

	/** The search box widget */
	TSharedPtr<SSearchBox> SearchBox;

	/** The hierarchy tree view widget */
	TSharedPtr<STreeView<TSharedPtr<FDMXPixelMappingHierarchyItem>>> HierarchyTreeView;

	/** Handles filtering the hierarchy based on an IFilter. */
	TSharedPtr<TreeFilterHandler<TSharedPtr<FDMXPixelMappingHierarchyItem>>> FilterHandler;

	/** Text filter for the filter handler, set from the search bbox */
	TSharedPtr<TTextFilter<TSharedPtr<FDMXPixelMappingHierarchyItem>>> SearchFilter;

	/** Timer handle for the request refresh timer */
	FTimerHandle RequestRefreshTimerHandle;

	/** Commands specific to the hierarchy. */
	TSharedPtr<FUICommandList> CommandList;

	/** The toolkit that contains this widget */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
