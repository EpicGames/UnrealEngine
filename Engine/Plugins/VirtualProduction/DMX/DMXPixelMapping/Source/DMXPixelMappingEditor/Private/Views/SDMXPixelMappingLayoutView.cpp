// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDMXPixelMappingLayoutView.h"

#include "Customizations/DMXPixelMappingHorizontalAlignmentCustomization.h"
#include "Customizations/DMXPixelMappingLayoutViewModelDetails.h"
#include "Customizations/DMXPixelMappingVerticalAlignmentCustomization.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "ViewModels/DMXPixelMappingLayoutViewModel.h"
#include "Widgets/Layout/SScrollBox.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingHierarchyView"

SDMXPixelMappingLayoutView::~SDMXPixelMappingLayoutView()
{
	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void SDMXPixelMappingLayoutView::Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;

	Model = NewObject<UDMXPixelMappingLayoutViewModel>();
	Model->SetToolkit(InToolkit);
	Model->OnModelChanged.AddSP(this, &SDMXPixelMappingLayoutView::Refresh);

	InitializeLayoutScriptsDetailsView();

	if (!ensureMsgf(LayoutScriptDetailsView.IsValid(), TEXT("Failed to create Details View for Pixel Mapping Layouts")))
	{
		return;
	}

	ChildSlot
		[
			SNew(SScrollBox)

			// Layout Model Details (select Layout Script class etc.)
			+ SScrollBox::Slot()
			[
				CreateLayoutModelDetailsView()
			]

			// Layout Script Details
			+ SScrollBox::Slot()
			[
				LayoutScriptDetailsView.ToSharedRef()
			]

			// Layout Settings Details
			+ SScrollBox::Slot()
			[
				CreateLayoutSettingsDetailsView()
			]
		];

	Refresh();

	if (GEditor)
	{
		GEditor->RegisterForUndo(this);
	}
}

void SDMXPixelMappingLayoutView::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Model);
}

void SDMXPixelMappingLayoutView::PostUndo(bool bSuccess)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXPixelMappingLayoutView::Refresh));
}

void SDMXPixelMappingLayoutView::PostRedo(bool bSuccess)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateSP(this, &SDMXPixelMappingLayoutView::Refresh));
}

void SDMXPixelMappingLayoutView::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (Model->ShouldAutoApplyLayoutScript())
	{
		Model->RequestApplyLayoutScript();
	}
}

void SDMXPixelMappingLayoutView::Refresh()
{
	const TArray<UObject*> LayoutScriptObjects = Model->GetLayoutScriptsObjectsSlow();
	if (LayoutScriptObjects.IsEmpty())
	{
		LayoutScriptDetailsView->SetVisibility(EVisibility::Collapsed);
	}
	else
	{
		LayoutScriptDetailsView->SetObjects(LayoutScriptObjects);
		LayoutScriptDetailsView->ForceRefresh();

		LayoutScriptDetailsView->SetVisibility(EVisibility::Visible);
	}
}

TSharedRef<SWidget> SDMXPixelMappingLayoutView::CreateLayoutSettingsDetailsView()
{
	UDMXPixelMappingEditorSettings* LayoutSettings = GetMutableDefault<UDMXPixelMappingEditorSettings>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedRef<IDetailsView> LayoutSettingsDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	LayoutSettingsDetailsView->SetObject(LayoutSettings);

	return LayoutSettingsDetailsView;
}

TSharedRef<SWidget> SDMXPixelMappingLayoutView::CreateLayoutModelDetailsView()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	TSharedRef<IDetailsView> ModelDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	ModelDetailsView->RegisterInstancedCustomPropertyLayout(UDMXPixelMappingLayoutViewModel::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FDMXPixelMappingLayoutViewModelDetails::MakeInstance, WeakToolkit));

	ModelDetailsView->SetObject(Model);

	return ModelDetailsView;
}

void SDMXPixelMappingLayoutView::InitializeLayoutScriptsDetailsView()
{
	TArray<UObject*> LayoutScripts = Model->GetLayoutScriptsObjectsSlow();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NotifyHook = this;

	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	LayoutScriptDetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	LayoutScriptDetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("EHorizontalAlignment"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXPixelMappingHorizontalAlignmentCustomization::MakeInstance));
	LayoutScriptDetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("EVerticalAlignment"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FDMXPixelMappingVerticalAlignmentCustomization::MakeInstance));
}

#undef LOCTEXT_NAMESPACE
