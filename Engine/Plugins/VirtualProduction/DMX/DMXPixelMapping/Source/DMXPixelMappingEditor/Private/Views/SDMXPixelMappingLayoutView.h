// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Misc/NotifyHook.h"
#include "UObject/GCObject.h"
#include "Widgets/SCompoundWidget.h"

class UDMXPixelMappingLayoutViewModel;
class FDMXPixelMappingToolkit;

class IDetailsView;


class SDMXPixelMappingLayoutView
	: public SCompoundWidget
	, public FGCObject
	, public FEditorUndoClient
	, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingLayoutView)
	{}
	SLATE_END_ARGS()

	/** Destructor */
	virtual ~SDMXPixelMappingLayoutView();

	/** Constructs the widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit);

protected:
	//~ Begin FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FDMXMVRFixtureActorLibrary");
	}
	//~ End FGCObject interface

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	//~ Begin FNotifyHook Interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	// End of FNotifyHook

private:
	/** Refreshes the widget */
	void Refresh();

	/** Creates a Details view of the Layout Settings */
	TSharedRef<SWidget> CreateLayoutSettingsDetailsView();

	/** Creates a Details view of the Layout Model */
	TSharedRef<SWidget> CreateLayoutModelDetailsView();

	/** Creates a details view of the Layout Script */
	void InitializeLayoutScriptsDetailsView();

	/** True when the details views in this widget ever changed properties */
	bool bDidPropertiesEverChange = false;

	/** The Layout Script Details View */
	TSharedPtr<IDetailsView> LayoutScriptDetailsView;

	/** Model for this view */
	TObjectPtr<UDMXPixelMappingLayoutViewModel> Model;

	/** Toolkit for this view */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
