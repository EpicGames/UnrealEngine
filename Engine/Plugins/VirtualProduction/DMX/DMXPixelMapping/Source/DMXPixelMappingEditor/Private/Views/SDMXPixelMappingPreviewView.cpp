// Copyright Epic Games, Inc. All Rights Reserved.

#include "Views/SDMXPixelMappingPreviewView.h"

#include "DMXPixelMappingEditorStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SDMXPixelMappingPreviewViewport.h"
#include "Widgets/SDMXPixelMappingZoomPan.h"
#include "Components/DMXPixelMappingRendererComponent.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingPreviewView"

void SDMXPixelMappingPreviewView::Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;

	SDMXPixelMappingSurface::Construct(SDMXPixelMappingSurface::FArguments()
		.AllowContinousZoomInterpolation(false)
		.Content()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				.HAlign(HAlign_Fill)
				.Visibility(this, &SDMXPixelMappingPreviewView::GetTitleBarVisibility)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SDMXPixelMappingPreviewView::GetSelectedComponentParentNameText)
							.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("BreadcrumbTrail.Delimiter"))
						]
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(this, &SDMXPixelMappingPreviewView::GetSelectedComponentNameText)
						.TextStyle(FAppStyle::Get(), TEXT("GraphBreadcrumbButtonText"))
						]
					]
				]
			]
			+SVerticalBox::Slot()
			[
				SAssignNew(PreviewHitTestRoot, SOverlay)
				.Visibility(EVisibility::Visible)
				.Clipping(EWidgetClipping::ClipToBoundsAlways)

				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(ZoomPan, SDMXPixelMappingZoomPan)
					.ZoomAmount(this, &SDMXPixelMappingPreviewView::GetZoomAmount)
					.ViewOffset(this, &SDMXPixelMappingPreviewView::GetViewOffset)
					.Visibility(this, &SDMXPixelMappingPreviewView::IsZoomPanVisible)
					[
						SNew(SOverlay)

						+ SOverlay::Slot()
						[
							SAssignNew(PreviewViewport, SDMXPixelMappingPreviewViewport, InToolkit)
						]

						+ SOverlay::Slot()
						[
							SAssignNew(PreviewSizeConstraint, SBox)
						]
					]
				]

				// Designer overlay UI, toolbar, status messages, zoom level...etc
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					CreateOverlayUI()
				]
			]
		]
	, InToolkit);
}

const FGeometry& SDMXPixelMappingPreviewView::GetGraphTickSpaceGeometry() const
{
	return ZoomPan->GetTickSpaceGeometry();
}

FReply SDMXPixelMappingPreviewView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnMouseButtonDown(MyGeometry, MouseEvent);

	// Capture mouse for the drag handle and general mouse actions
	return FReply::Handled().PreventThrottling().SetUserFocus(AsShared(), EFocusCause::Mouse).CaptureMouse(AsShared());
}

FReply SDMXPixelMappingPreviewView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnMouseButtonUp(MyGeometry, MouseEvent);

	return FReply::Handled().ReleaseMouseCapture();
}

FReply SDMXPixelMappingPreviewView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetCursorDelta().IsZero())
	{
		return FReply::Unhandled();
	}

	CachedMousePosition = MouseEvent.GetScreenSpacePosition();

	FReply SurfaceHandled = SDMXPixelMappingSurface::OnMouseMove(MyGeometry, MouseEvent);
	if (SurfaceHandled.IsEventHandled())
	{
		return SurfaceHandled;
	}

	return FReply::Handled();
}

void SDMXPixelMappingPreviewView::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnMouseEnter(MyGeometry, MouseEvent);
}

void SDMXPixelMappingPreviewView::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnMouseLeave(MouseEvent);
}

void SDMXPixelMappingPreviewView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	CachedWidgetGeometry.Reset();
	FArrangedWidget WindowWidgetGeometry(PreviewHitTestRoot.ToSharedRef(), GetDesignerGeometry());
	PopulateWidgetGeometryCache(WindowWidgetGeometry);

	SDMXPixelMappingSurface::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	// Compute the origin in absolute space.
	FGeometry RootGeometry = CachedWidgetGeometry.FindChecked(PreviewSizeConstraint.ToSharedRef()).Geometry;
	FVector2D AbsoluteOrigin = MakeGeometryWindowLocal(RootGeometry).LocalToAbsolute(FVector2D::ZeroVector);

	GridOrigin = AbsoluteOrigin;
}

FReply SDMXPixelMappingPreviewView::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SDMXPixelMappingSurface::OnDragDetected(MyGeometry, MouseEvent);

	return FReply::Handled();
}

void SDMXPixelMappingPreviewView::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDragEnter(MyGeometry, DragDropEvent);
}

void SDMXPixelMappingPreviewView::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDragLeave(DragDropEvent);
}

FReply SDMXPixelMappingPreviewView::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDragOver(MyGeometry, DragDropEvent);

	return FReply::Handled();
}

FReply SDMXPixelMappingPreviewView::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	SDMXPixelMappingSurface::OnDrop(MyGeometry, DragDropEvent);

	return FReply::Handled();
}

void SDMXPixelMappingPreviewView::OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	SDMXPixelMappingSurface::OnPaintBackground(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
}

int32 SDMXPixelMappingPreviewView::GetGraphRulePeriod() const
{
	return 10; // Parent override
}

float SDMXPixelMappingPreviewView::GetGridScaleAmount() const
{
	return 1.0f; // Parent override
}

int32 SDMXPixelMappingPreviewView::GetGridSize() const
{
	return 4; // Parent override
}

FSlateRect SDMXPixelMappingPreviewView::ComputeAreaBounds() const
{
	const FVector2D Size
	{
		FMath::Max(GetWidthGraphSpace().Get(), 1.0),
		FMath::Max(GetHeightGraphSpace().Get(), 1.0),
	};

	return FSlateRect(0, 0, Size.X, Size.Y);
}

FOptionalSize SDMXPixelMappingPreviewView::GetWidthGraphSpace() const
{
	if (PreviewViewport.IsValid())
	{
		return PreviewViewport->GetWidthGraphSpace();
	}

	return FOptionalSize();
}

FOptionalSize SDMXPixelMappingPreviewView::GetHeightGraphSpace() const
{
	if (PreviewViewport.IsValid())
	{
		return PreviewViewport->GetHeightGraphSpace();
	}

	return FOptionalSize();
}

TSharedRef<SWidget> SDMXPixelMappingPreviewView::CreateOverlayUI()
{
	return SNew(SOverlay)

		// Outline and text for important state.
		+ SOverlay::Slot()
		.Padding(0)
		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Top)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(6, 2, 0, 0)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Graph.ZoomText")
				.Text(this, &SDMXPixelMappingPreviewView::GetZoomText)
				.ColorAndOpacity(this, &SDMXPixelMappingPreviewView::GetZoomTextColorAndOpacity)
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSpacer)
				.Size(FVector2D(1, 1))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonColorAndOpacity(FLinearColor::Transparent)
				.ToolTipText(LOCTEXT("ZoomToFit_ToolTip", "Zoom To Fit"))
				.OnClicked(this, &SDMXPixelMappingPreviewView::HandleZoomToFitClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("UMGEditor.ZoomToFit"))
				]
			]
		];
}

EVisibility SDMXPixelMappingPreviewView::IsZoomPanVisible() const
{
	if (TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin())
	{
		if (UDMXPixelMappingRendererComponent* RendererComponent = Toolkit->GetActiveRendererComponent())
		{
			if (RendererComponent->GetRenderedInputTexture())
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

FGeometry SDMXPixelMappingPreviewView::MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const
{
	FGeometry NewGeometry = WidgetGeometry;

	TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this));
	if (WidgetWindow.IsValid())
	{
		TSharedRef<SWindow> CurrentWindowRef = WidgetWindow.ToSharedRef();

		NewGeometry.AppendTransform(FSlateLayoutTransform(Inverse(CurrentWindowRef->GetPositionInScreen())));
	}

	return NewGeometry;
}

FGeometry SDMXPixelMappingPreviewView::GetDesignerGeometry() const
{
	return PreviewHitTestRoot->GetTickSpaceGeometry();
}

void SDMXPixelMappingPreviewView::PopulateWidgetGeometryCache(FArrangedWidget& Root)
{
	PopulateWidgetGeometryCache_Loop(Root);
}

void SDMXPixelMappingPreviewView::PopulateWidgetGeometryCache_Loop(FArrangedWidget& CurrentWidget)
{
	FArrangedChildren ArrangedChildren(EVisibility::All);
	CurrentWidget.Widget->ArrangeChildren(CurrentWidget.Geometry, ArrangedChildren);

	CachedWidgetGeometry.Add(CurrentWidget.Widget, CurrentWidget);

	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& SomeChild = ArrangedChildren[ChildIndex];
		PopulateWidgetGeometryCache_Loop(SomeChild);
	}
}

FReply SDMXPixelMappingPreviewView::HandleZoomToFitClicked()
{
	ZoomToFit(/*bInstantZoom*/ false);
	return FReply::Handled();
}

const TSet<FDMXPixelMappingComponentReference>& SDMXPixelMappingPreviewView::GetSelectedComponents() const
{
	check(WeakToolkit.Pin().IsValid());

	return WeakToolkit.Pin()->GetSelectedComponents();
}

FDMXPixelMappingComponentReference SDMXPixelMappingPreviewView::GetSelectedComponent() const
{
	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = GetSelectedComponents();

	// Only return a selected widget when we have only a single item selected.
	if (SelectedComponents.Num() == 1)
	{
		return *(SelectedComponents.FindArbitraryElement());
	}

	return FDMXPixelMappingComponentReference();
}

FText SDMXPixelMappingPreviewView::GetSelectedComponentNameText() const
{
	FDMXPixelMappingComponentReference SelectedComponentRef = GetSelectedComponent();
	UDMXPixelMappingBaseComponent* BaseComponent = SelectedComponentRef.GetComponent();
	if (BaseComponent)
	{
		return FText::FromString(BaseComponent->GetUserName());
	}

	return FText();
}

FText SDMXPixelMappingPreviewView::GetSelectedComponentParentNameText() const
{
	FDMXPixelMappingComponentReference SelectedComponentRef = GetSelectedComponent();
	UDMXPixelMappingBaseComponent* BaseComponent = SelectedComponentRef.GetComponent();
	if (BaseComponent && BaseComponent->GetParent())
	{
		return FText::FromString(BaseComponent->GetParent()->GetUserName());
	}

	return FText();
}

EVisibility SDMXPixelMappingPreviewView::GetTitleBarVisibility() const
{
	FDMXPixelMappingComponentReference SelectedComponentRef = GetSelectedComponent();
	UDMXPixelMappingBaseComponent* BaseComponent = SelectedComponentRef.GetComponent();

	if (BaseComponent && BaseComponent->GetParent())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
