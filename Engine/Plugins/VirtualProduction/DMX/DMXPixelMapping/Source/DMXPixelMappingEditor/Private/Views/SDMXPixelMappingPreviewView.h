// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Widgets/SDMXPixelMappingSurface.h"

struct FDMXPixelMappingComponentReference;
class SDMXPixelMappingPreviewViewport;
class FDMXPixelMappingToolkit;
struct FOptionalSize;
class SBox;
class SDMXPixelMappingZoomPan;
class SOverlay;

class SDMXPixelMappingPreviewView
	: public SDMXPixelMappingSurface
{
public:

	SLATE_BEGIN_ARGS(SDMXPixelMappingPreviewView) { }
	SLATE_END_ARGS()

public:
	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit);

	/** Returns the geometry of the graph */
	const FGeometry& GetGraphTickSpaceGeometry() const;

protected:
	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	// End of SWidget interface

	//~ Begin SDMXPixelMappingSurface interface
	virtual void OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;
	virtual int32 GetGraphRulePeriod() const override;
	virtual float GetGridScaleAmount() const override;
	virtual int32 GetGridSize() const override;
	virtual FSlateRect ComputeAreaBounds() const override;
	//~ End SDMXPixelMappingSurface interface

private:
	/** Gets the height in graph space */
	FOptionalSize GetHeightGraphSpace() const;

	/** Gets the width in graph space */
	FOptionalSize GetWidthGraphSpace() const;

	TSharedRef<SWidget> CreateOverlayUI();

	EVisibility IsZoomPanVisible() const;

	FGeometry MakeGeometryWindowLocal(const FGeometry& WidgetGeometry) const;

	FGeometry GetDesignerGeometry() const;
	void PopulateWidgetGeometryCache(FArrangedWidget& Root);
	void PopulateWidgetGeometryCache_Loop(FArrangedWidget& Parent);

	FReply HandleZoomToFitClicked();

	const TSet<FDMXPixelMappingComponentReference>& GetSelectedComponents() const;

	FDMXPixelMappingComponentReference GetSelectedComponent() const;

	FText GetSelectedComponentNameText() const;
	FText GetSelectedComponentParentNameText() const;
	EVisibility GetTitleBarVisibility() const;

	/** Zoom pan widget */
	TSharedPtr<SDMXPixelMappingZoomPan> ZoomPan;

	TSharedPtr<SDMXPixelMappingPreviewViewport> PreviewViewport;

	/** Cache last mouse position to be used as a paste drop location */
	FVector2D CachedMousePosition;

	TSharedPtr<SBox> PreviewSizeConstraint;

	TSharedPtr<SOverlay> PreviewHitTestRoot;

	TMap<TSharedRef<SWidget>, FArrangedWidget> CachedWidgetGeometry;
};
