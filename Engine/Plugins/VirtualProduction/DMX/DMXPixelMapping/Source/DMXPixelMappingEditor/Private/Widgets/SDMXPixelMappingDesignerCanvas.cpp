// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingDesignerCanvas.h"

#include "Layout/ArrangedChildren.h"
#include "SlateSettings.h"
void SDMXPixelMappingDesignerCanvas::Construct(const FArguments& InArgs)
{
	SConstraintCanvas::Construct(SConstraintCanvas::FArguments());
}
void SDMXPixelMappingDesignerCanvas::ArrangeLayeredChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, FArrangedChildLayers& ArrangedChildLayers) const
{
	if (Children.Num() > 0)
	{
		// Using a Project setting here to decide whether we automatically put children in front of all previous children
		// or allow the explicit ZOrder value to place children in the same layer. This will allow users to have non-touching
		// children render into the same layer and have a chance of being batched by the Slate renderer for better performance.
#if WITH_EDITOR
		const bool bExplicitChildZOrder = GetDefault<USlateSettings>()->bExplicitCanvasChildZOrder;
#else
		const static bool bExplicitChildZOrder = GetDefault<USlateSettings>()->bExplicitCanvasChildZOrder;
#endif
		// Sort the children based on zorder.
		TArray< FChildZOrder, TInlineAllocator<64> > SlotOrder;
		SlotOrder.Reserve(Children.Num());
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const SConstraintCanvas::FSlot& CurChild = Children[ChildIndex];
			FChildZOrder Order;
			Order.ChildIndex = ChildIndex;
			Order.ZOrder = CurChild.GetZOrder();
			SlotOrder.Add(Order);
		}
		SlotOrder.Sort(FSortSlotsByZOrder());
		float LastZOrder = -FLT_MAX;
		// Arrange the children now in their proper z-order.
		for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const FChildZOrder& CurSlot = SlotOrder[ChildIndex];
			const SConstraintCanvas::FSlot& CurChild = Children[CurSlot.ChildIndex];
			const TSharedRef<SWidget>& CurWidget = CurChild.GetWidget();
			const EVisibility ChildVisibility = CurWidget->GetVisibility();
			if (ArrangedChildren.Accepts(ChildVisibility))
			{
				const FMargin Offset = CurChild.GetOffset();
				const FVector2D Alignment = CurChild.GetAlignment();
				const FAnchors Anchors = CurChild.GetAnchors();
				const bool AutoSize = CurChild.GetAutoSize();
				const FMargin AnchorPixels =
					FMargin(Anchors.Minimum.X * AllottedGeometry.GetLocalSize().X,
					Anchors.Minimum.Y * AllottedGeometry.GetLocalSize().Y,
					Anchors.Maximum.X * AllottedGeometry.GetLocalSize().X,
					Anchors.Maximum.Y * AllottedGeometry.GetLocalSize().Y);
				const bool bIsHorizontalStretch = Anchors.Minimum.X != Anchors.Maximum.X;
				const bool bIsVerticalStretch = Anchors.Minimum.Y != Anchors.Maximum.Y;
				const FVector2D SlotSize = FVector2D(Offset.Right, Offset.Bottom);
				const FVector2D Size = AutoSize ? CurWidget->GetDesiredSize() : SlotSize;
				// Calculate the offset based on the pivot position.
				FVector2D AlignmentOffset = Size * Alignment;
				// Calculate the local position based on the anchor and position offset.
				FVector2D LocalPosition, LocalSize;
				// Calculate the position and size based on the horizontal stretch or non-stretch
				if (bIsHorizontalStretch)
				{
					LocalPosition.X = AnchorPixels.Left + Offset.Left;
					LocalSize.X = AnchorPixels.Right - LocalPosition.X - Offset.Right;
				}
				else
				{
					LocalPosition.X = AnchorPixels.Left + Offset.Left - AlignmentOffset.X;
					LocalSize.X = Size.X;
				}
				// Calculate the position and size based on the vertical stretch or non-stretch
				if (bIsVerticalStretch)
				{
					LocalPosition.Y = AnchorPixels.Top + Offset.Top;
					LocalSize.Y = AnchorPixels.Bottom - LocalPosition.Y - Offset.Bottom;
				}
				else
				{
					LocalPosition.Y = AnchorPixels.Top + Offset.Top - AlignmentOffset.Y;
					LocalSize.Y = Size.Y;
				}
				// Add the information about this child to the output list (ArrangedChildren)
				ArrangedChildren.AddWidget(ChildVisibility, AllottedGeometry.MakeChild(
					// The child widget being arranged
					CurWidget,
					// Child's local position (i.e. position within parent)
					LocalPosition,
					// Child's size
					LocalSize
				));
				bool bNewLayer = true;
				if (bExplicitChildZOrder)
				{
					// Split children into discrete layers for the paint method
					bNewLayer = false;
					if (CurSlot.ZOrder > LastZOrder + DELTA)
					{
						if (ArrangedChildLayers.Num() > 0)
						{
							bNewLayer = true;
						}
						LastZOrder = CurSlot.ZOrder;
					}
				}
				ArrangedChildLayers.Add(bNewLayer);
			}
		}
	}
}

int32 SDMXPixelMappingDesignerCanvas::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	SCOPED_NAMED_EVENT_TEXT("SConstraintCanvas", FColor::Orange);
	FArrangedChildren ArrangedChildren(EVisibility::Visible);
	FArrangedChildLayers ChildLayers;
	ArrangeLayeredChildren(AllottedGeometry, ArrangedChildren, ChildLayers);
	const bool bForwardedEnabled = ShouldBeEnabled(bParentEnabled);
	// Because we paint multiple children, we must track the maximum layer id that they produced in case one of our parents
	// wants to an overlay for all of its contents.
	int32 MaxLayerId = LayerId;
	int32 ChildLayerId = LayerId;
	const FPaintArgs NewArgs = Args.WithNewParent(this);
	for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
	{
		FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
		// Bools in ChildLayers tell us whether to paint the next child in front of all previous
		if (ChildLayers[ChildIndex])
		{
			ChildLayerId = MaxLayerId + 1;
		}
		const int32 CurWidgetsMaxLayerId = CurWidget.Widget->Paint(NewArgs, CurWidget.Geometry, MyCullingRect, OutDrawElements, ChildLayerId, InWidgetStyle, bForwardedEnabled);
		MaxLayerId = FMath::Max(MaxLayerId, CurWidgetsMaxLayerId);
	}
	return MaxLayerId;
}
