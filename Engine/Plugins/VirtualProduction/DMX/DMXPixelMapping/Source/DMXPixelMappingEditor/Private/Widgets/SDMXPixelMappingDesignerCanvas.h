// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Widgets/Layout/SConstraintCanvas.h"
class FArrangedChildren;
class FPaintArgs;


/**
 * Most of the logic copied from a Engine/Source/Runtime/Slate/Public/Widgets/Layout/SConstraintCanvas.h
 * We need to remove condition if (!IsChildWidgetCulled(MyCullingRect, CurWidget)) from OnPaint function
 */
class SDMXPixelMappingDesignerCanvas
	: public SConstraintCanvas
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingDesignerCanvas) 
	{}
	SLATE_END_ARGS();

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

private:
	struct FChildZOrder
	{
		int32 ChildIndex;
		float ZOrder;
	};
	struct FSortSlotsByZOrder
	{
		bool operator()(const FChildZOrder& A, const FChildZOrder& B) const
		{
			return A.ZOrder == B.ZOrder ? A.ChildIndex < B.ChildIndex : A.ZOrder < B.ZOrder;
		}
	};
	/** An array matching the length and order of ArrangedChildren. True means the child must be placed in a layer in front of all previous children. */
	typedef TArray<bool, TInlineAllocator<16>> FArrangedChildLayers;
	/** Like ArrangeChildren but also generates an array of layering information (see FArrangedChildLayers). */
	void ArrangeLayeredChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren, FArrangedChildLayers& ArrangedChildLayers) const;
protected:
	// SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
};
