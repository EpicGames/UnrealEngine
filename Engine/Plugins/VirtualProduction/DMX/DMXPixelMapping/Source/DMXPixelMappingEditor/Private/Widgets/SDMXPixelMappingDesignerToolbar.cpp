// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingDesignerToolbar.h"

#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorCommands.h"
#include "SEditorViewportToolBarMenu.h"
#include "Styling/AppStyle.h"
#include "Styling/ToolBarStyle.h"
#include "SViewportToolBarComboMenu.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SDMXPixelMappingSnapGridMenu.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingDesignerToolbar"

namespace UE::DMX
{
	void SDMXPixelMappingDesignerToolbar::Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit)
	{
		WeakToolkit = InToolkit;

		constexpr TCHAR ToolBarStyleName[] = TEXT("EditorViewportToolBar");
		const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(ToolBarStyleName);

		const TSharedRef<FUICommandList> CommandList = InToolkit->GetToolkitCommands();

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 2.0f)
				[
					GenerateTransformHandleModeSection()
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 2.0f)
				[
					GenerateGridSnappingSection()
				]
					
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 2.0f)
				[
					GenerateZoomToFitSection(InArgs)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 2.0f)
				[
					GenerateSettingsSection()
				]
			]
		];
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateTransformHandleModeSection()
	{
		// Resize mode
		const FCheckBoxStyle& CheckBoxStartStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.Start");

		TSharedRef<SCheckBox> EnableResizeModeToggleButton = 
			SNew(SCheckBox)
			.Style(&CheckBoxStartStyle)
			.ToolTipText(LOCTEXT("TransformHandleResizeMode", "Resize Components"))
			.OnCheckStateChanged(this, &SDMXPixelMappingDesignerToolbar::OnTransformHandleModeSelected, EDMXPixelMappingTransformHandleMode::Resize)
			.IsChecked(this, &SDMXPixelMappingDesignerToolbar::GetCheckboxStateForTransormHandleMode, EDMXPixelMappingTransformHandleMode::Resize)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("EditorViewport.ScaleMode"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

		// Rotate mode
		const FCheckBoxStyle& CheckBoxEndStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("EditorViewportToolBar.ToggleButton.End");

		TSharedRef<SCheckBox> EnableRotateModeToggleButton = 
			SNew(SCheckBox)
			.Style(&CheckBoxEndStyle)
			.ToolTipText(LOCTEXT("TransformHandleRotateMode", "Rotate Components"))
			.OnCheckStateChanged(this, &SDMXPixelMappingDesignerToolbar::OnTransformHandleModeSelected, EDMXPixelMappingTransformHandleMode::Rotate)
			.IsChecked(this, &SDMXPixelMappingDesignerToolbar::GetCheckboxStateForTransormHandleMode, EDMXPixelMappingTransformHandleMode::Rotate)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("EditorViewport.RotateMode"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

		// As a single widget
		const TSharedRef<SWidget> TransformHandleModeWidget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.f)
			[
				EnableResizeModeToggleButton
			]
			+ SHorizontalBox::Slot()
			.Padding(0.f)
			[
				EnableRotateModeToggleButton
			];

		return TransformHandleModeWidget;
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateGridSnappingSection()
	{
		const FUICommandInfo* ToggleSnapGridCommand = FDMXPixelMappingEditorCommands::Get().ToggleGridSnapping.Get();
		check(ToggleSnapGridCommand);

		const TSharedRef<SViewportToolBarComboMenu> GridSnappingComboMenu =
			SNew(SViewportToolBarComboMenu)
			.Cursor(EMouseCursor::Default)
			.IsChecked(this, &SDMXPixelMappingDesignerToolbar::GetSnapGridEnabledCheckState)
			.OnCheckStateChanged(this, &SDMXPixelMappingDesignerToolbar::OnSnapGridCheckStateChanged)
			.Label(this, &SDMXPixelMappingDesignerToolbar::GetSnapGridLabel)
			.OnGetMenuContent(this, &SDMXPixelMappingDesignerToolbar::GenerateSnapGridMenu)
			.ToggleButtonToolTip(ToggleSnapGridCommand->GetDescription())
			.MenuButtonToolTip(LOCTEXT("SnapGridMenuTooltip", "Grid Snapping Settings"))
			.Icon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.LocationGridSnap"))
			.ParentToolBar(SharedThis(this));

		return GridSnappingComboMenu;
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateZoomToFitSection(const FArguments& InArgs)
	{
		constexpr TCHAR ToolBarStyleName[] = TEXT("EditorViewportToolBar");
		const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(ToolBarStyleName);

		const TSharedRef<SButton> ZoomToFitButton = 
			SNew(SButton)
			.ButtonStyle(&ToolBarStyle.ButtonStyle)
			.ToolTipText(LOCTEXT("ZoomToFitToolTip", "Zoom To Fit"))
			.OnClicked(InArgs._OnZoomToFitClicked)
			.ContentPadding(ToolBarStyle.ButtonPadding)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("UMGEditor.ZoomToFit"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];

		return ZoomToFitButton;
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateSettingsSection()
	{
		constexpr TCHAR ToolBarStyleName[] = TEXT("EditorViewportToolBar");
		const FToolBarStyle& ToolBarStyle = FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(ToolBarStyleName);

		const TSharedRef<SEditorViewportToolbarMenu> SettingsButton =
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Label(LOCTEXT("SettingsButtonLabel", "Settings"))
			.LabelIcon(FAppStyle::Get().GetBrush("Icons.Settings"))
			.OnGetMenuContent(this, &SDMXPixelMappingDesignerToolbar::GenerateSettingsMenuContent);

		return SettingsButton;
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateSnapGridMenu()
	{
		if (!WeakToolkit.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		const TSharedRef<SWidget> SnapGridMenu = 
			SNew(SDMXPixelMappingSnapGridMenu, WeakToolkit.Pin().ToSharedRef())
			.IsChecked(this, &SDMXPixelMappingDesignerToolbar::GetSnapGridEnabledCheckState)
			.OnCheckStateChanged(this, &SDMXPixelMappingDesignerToolbar::OnSnapGridCheckStateChanged);

		return SnapGridMenu;
	}

	FText SDMXPixelMappingDesignerToolbar::GetSnapGridLabel() const
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			if (PixelMapping->bGridSnappingEnabled)
			{
				const FString ColumnString = FString::FromInt(PixelMapping->SnapGridColumns);
				const FString RowString = FString::FromInt(PixelMapping->SnapGridRows);

				return FText::FromString(ColumnString + TEXT("x") + RowString);
			}

			return FText::GetEmpty();
		}

		return FText::GetEmpty();
	}

	ECheckBoxState SDMXPixelMappingDesignerToolbar::GetSnapGridEnabledCheckState() const
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->bGridSnappingEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}

		return ECheckBoxState::Undetermined;
	}

	void SDMXPixelMappingDesignerToolbar::OnSnapGridCheckStateChanged(ECheckBoxState NewCheckBoxState)
	{
		if (const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin())
		{
			Toolkit->ToggleGridSnapping();
		}
	}

	void SDMXPixelMappingDesignerToolbar::OnTransformHandleModeSelected(ECheckBoxState DummyCheckBoxState, UE::DMX::EDMXPixelMappingTransformHandleMode NewTransformHandleMode)
	{
		const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
		if (!Toolkit.IsValid())
		{
			return;
		}

		Toolkit->SetTransformHandleMode(NewTransformHandleMode);
	}

	ECheckBoxState SDMXPixelMappingDesignerToolbar::GetCheckboxStateForTransormHandleMode(UE::DMX::EDMXPixelMappingTransformHandleMode TransformHandleMode) const
	{
		const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
		if (!Toolkit.IsValid())
		{
			return ECheckBoxState::Undetermined;
		}

		return Toolkit->GetTransformHandleMode() == TransformHandleMode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateSettingsMenuContent()
	{
		const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
		if (!Toolkit.IsValid())
		{
			return SNullWidget::NullWidget;
		}

		constexpr bool bShouldCloseMenuAfterSelection = true;
		FMenuBuilder MenuBuilder(bShouldCloseMenuAfterSelection, Toolkit->GetToolkitCommands());

		MenuBuilder.BeginSection("EditorSettings", LOCTEXT("EditorSettingsSection", "Editor Settings"));
		{
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleAlwaysSelectGroup);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleScaleChildrenWithParent);
		}
		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("DisplaySettings", LOCTEXT("SettingsSection", "Display Settings"));
		{
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowComponentNames);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowPatchInfo);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowMatrixCells);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowCellIDs);
			MenuBuilder.AddMenuEntry(FDMXPixelMappingEditorCommands::Get().ToggleShowPivot);

			constexpr bool bNoIndent = true;
			MenuBuilder.AddWidget(
				GenerateComponentFontSizeEditWidget(),
				LOCTEXT("FontSizeLabel", "Font Size"),
				bNoIndent
			);

			constexpr bool bSearchable = false;
			MenuBuilder.AddWidget(
				GenerateDesignerExposureEditWidget(),
				LOCTEXT("ExposureLabel", "Display Exposure"),
				bNoIndent,
				bSearchable,
				LOCTEXT("ExposureTooltip", "Adjusts the exposure of the Designer.\nHas no effect on the Pixel Mapping rendering and the DMX output.")
			);
		}
		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateComponentFontSizeEditWidget()
	{
		return
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.HAlign(HAlign_Right)
			[
				SNew(SNumericEntryBox<uint8>)
				.MinDesiredValueWidth(40.f)
				.MinSliderValue(6)
				.MaxSliderValue(24)
				.AllowSpin(true)
				.Value(this, &SDMXPixelMappingDesignerToolbar::GetComponentFontSize)
				.OnValueChanged(this, &SDMXPixelMappingDesignerToolbar::SetComponentFontSize)
			];
	}

	TOptional<uint8> SDMXPixelMappingDesignerToolbar::GetComponentFontSize() const
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->ComponentLabelFontSize;
		}

		return TOptional<uint8>();
	}

	void SDMXPixelMappingDesignerToolbar::SetComponentFontSize(uint8 FontSize)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			constexpr uint8 MinFontSize = 1;
			PixelMapping->ComponentLabelFontSize = FMath::Max(FontSize, MinFontSize);
		}
	}

	TSharedRef<SWidget> SDMXPixelMappingDesignerToolbar::GenerateDesignerExposureEditWidget()
	{
		return
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.HAlign(HAlign_Right)
			[
				SNew(SNumericEntryBox<float>)
				.MinDesiredValueWidth(40.f)
				.MinSliderValue(0.f)
				.MaxSliderValue(1.f)
				.AllowSpin(true)
				.Value(this, &SDMXPixelMappingDesignerToolbar::GetDesignerExposure)
				.OnValueChanged(this, &SDMXPixelMappingDesignerToolbar::SetDesignerExposure)
			];
	}

	TOptional<float> SDMXPixelMappingDesignerToolbar::GetDesignerExposure() const
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->DesignerExposure;
		}

		return TOptional<float>();
	}

	void SDMXPixelMappingDesignerToolbar::SetDesignerExposure(float Exposure)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			constexpr uint8 MinExposure = 0.f;
			PixelMapping->DesignerExposure = FMath::Max(MinExposure, Exposure);
		}
	}
}

#undef LOCTEXT_NAMESPACE
