// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Styling/SlateTypes.h"
#include "SViewportToolBar.h"

enum class ECheckBoxState : uint8;
class FDMXPixelMappingToolkit;
class FReply;
namespace UE::DMX { enum class EDMXPixelMappingTransformHandleMode : uint8; }


namespace UE::DMX
{
	class SDMXPixelMappingDesignerToolbar
		: public SViewportToolBar
	{
	public:
		SLATE_BEGIN_ARGS(SDMXPixelMappingDesignerToolbar)
			{}

			/** Event raised when zoom to fit was clicked */
			SLATE_EVENT(FOnClicked, OnZoomToFitClicked)

		SLATE_END_ARGS()

		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit);

	private:
		TSharedRef<SWidget> GenerateTransformHandleModeSection();
		TSharedRef<SWidget> GenerateGridSnappingSection();
		TSharedRef<SWidget> GenerateZoomToFitSection(const FArguments& InArgs);
		TSharedRef<SWidget> GenerateSettingsSection();

		/** Generates the grid snapping menu */
		TSharedRef<SWidget> GenerateSnapGridMenu();

		/** Returns the grid snapping label */
		FText GetSnapGridLabel() const;

		/** Gets the check box state according to grid snapping being enabled */
		ECheckBoxState GetSnapGridEnabledCheckState() const;

		/** Called when the grid snapping checkbox state changed */
		void OnSnapGridCheckStateChanged(ECheckBoxState NewCheckBoxState);

		/** Called when a transform handle mode was selected. The checkbox state can be ignored. Instead clicking always enables. */
		void OnTransformHandleModeSelected(ECheckBoxState DummyCheckBoxState, UE::DMX::EDMXPixelMappingTransformHandleMode NewTransformHandleMode);

		/** Returns the current checkbox state of the transform mode */
		ECheckBoxState GetCheckboxStateForTransormHandleMode(UE::DMX::EDMXPixelMappingTransformHandleMode TransformHandleMode) const;

		/** Generates the contents of the settings menu */
		TSharedRef<SWidget> GenerateSettingsMenuContent();

		/** Generates a widget to edit the font size */
		TSharedRef<SWidget> GenerateComponentFontSizeEditWidget();

		/** Gets the font size for component labels in the designer view */
		TOptional<uint8> GetComponentFontSize() const;
		
		/** Sets the font size for component labels in the designer view */
		void SetComponentFontSize(uint8 FontSize);

		/** Generates a widget to edit the display exposure */
		TSharedRef<SWidget> GenerateDesignerExposureEditWidget();

		/** Gets the exposure of the designer view */
		TOptional<float> GetDesignerExposure() const;

		/** Sets the exposure of the designer view */
		void SetDesignerExposure(float Exposure);

		/** The toolkit that owns this widget */
		TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
	};
}
