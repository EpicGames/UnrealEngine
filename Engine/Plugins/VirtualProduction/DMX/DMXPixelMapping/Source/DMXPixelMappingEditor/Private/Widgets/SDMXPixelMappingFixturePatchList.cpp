// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingFixturePatchList.h"

#include "Algo/MaxElement.h"
#include "DMXPixelMapping.h"
#include "DragDrop/DMXPixelMappingDragDropOp.h"
#include "Components/DMXPixelMappingBaseComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Templates/DMXPixelMappingComponentTemplate.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "ViewModels/DMXPixelMappingDMXLibraryViewModel.h"
#include "Widgets/DMXReadOnlyFixturePatchListItem.h"


void SDMXPixelMappingFixturePatchList::Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingDMXLibraryViewModel> InDMXLibraryModel)
{
	if (!InToolkit.IsValid() || !InDMXLibraryModel.IsValid())
	{
		return;
	}
	WeakToolkit = InToolkit;
	WeakDMXLibraryViewModel = InDMXLibraryModel;

	SDMXReadOnlyFixturePatchList::Construct(
		SDMXReadOnlyFixturePatchList::FArguments()
		.ListDescriptor(InArgs._ListDescriptor)
		.OnContextMenuOpening(InArgs._OnContextMenuOpening)
		.OnRowDragDetected(this, &SDMXPixelMappingFixturePatchList::OnRowDragDetected)
	);
}

FReply SDMXPixelMappingFixturePatchList::OnRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	UDMXPixelMappingFixtureGroupComponent* FixtureGroup = WeakDMXLibraryViewModel.IsValid() ? WeakDMXLibraryViewModel->GetFixtureGroupComponent() : nullptr;
	if (!Toolkit.IsValid() || !FixtureGroup)
	{
		return FReply::Unhandled();
	}

	TArray<TSharedPtr<FDMXPixelMappingComponentTemplate>> Templates;
	for (UDMXEntityFixturePatch* FixturePatch : GetSelectedFixturePatches())
	{
		const FDMXFixtureMode* FixtureModePtr = FixturePatch->GetActiveMode();

		const FDMXEntityFixturePatchRef FixturePatchReference(FixturePatch);
		if (FixtureModePtr && FixtureModePtr->bFixtureMatrixEnabled)
		{
			TSharedRef<FDMXPixelMappingComponentTemplate> FixturePatchMatrixTemplate = MakeShared<FDMXPixelMappingComponentTemplate>(UDMXPixelMappingMatrixComponent::StaticClass(), FixturePatchReference);
			Templates.Add(FixturePatchMatrixTemplate);
		}
		else
		{
			TSharedRef<FDMXPixelMappingComponentTemplate> FixturePatchItemTemplate = MakeShared<FDMXPixelMappingComponentTemplate>(UDMXPixelMappingFixtureGroupItemComponent::StaticClass(), FixturePatchReference);
			Templates.Add(FixturePatchItemTemplate);
		}
	}

	if (Templates.IsEmpty())
	{
		return FReply::Handled();
	}
	else
	{
		return FReply::Handled().BeginDragDrop(FDMXPixelMappingDragDropOp::New(Toolkit.ToSharedRef(), FVector2D::ZeroVector, Templates, FixtureGroup));
	}
}

void SDMXPixelMappingFixturePatchList::SelectAfter(const TArray<UDMXEntityFixturePatch*>& FixturePatches)
{
	const UDMXEntityFixturePatch* const* MaxFixturePatchPtr = Algo::MaxElementBy(FixturePatches, [](const UDMXEntityFixturePatch* FixturePatch)
		{
			return FixturePatch ? FixturePatch->GetUniverseID() * DMX_UNIVERSE_SIZE + FixturePatch->GetStartingChannel() : -1;
		});

	const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> Items = GetListItems();
	if (MaxFixturePatchPtr)
	{
		const int32 IndexOfPatch = Items.IndexOfByPredicate([MaxFixturePatchPtr](const TSharedPtr<FDMXReadOnlyFixturePatchListItem>& Item)
			{			
				return MaxFixturePatchPtr && Item->GetFixturePatch() == *MaxFixturePatchPtr;
			});

		if (Items.IsValidIndex(IndexOfPatch + 1))
		{
			const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> NewSelection{ Items[IndexOfPatch + 1] };
			SelectItems(NewSelection);
		}
		else if (Items.IsValidIndex(IndexOfPatch - 1))
		{
			const TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>> NewSelection{ Items[IndexOfPatch - 1] };
			SelectItems(NewSelection);
		}
	}
}

void SDMXPixelMappingFixturePatchList::ForceRefresh() 
{
	SDMXReadOnlyFixturePatchList::ForceRefresh();

	// Always make a selection if possible
	if (GetSelectedFixturePatches().IsEmpty() && !GetListItems().IsEmpty())
	{
		SelectItems(TArray<TSharedPtr<FDMXReadOnlyFixturePatchListItem>>({ GetListItems()[0] }));
	}
}
