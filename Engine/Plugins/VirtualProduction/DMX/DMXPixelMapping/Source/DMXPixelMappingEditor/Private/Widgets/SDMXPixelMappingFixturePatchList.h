// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXEntityReference.h"
#include "Widgets/SDMXReadOnlyFixturePatchList.h"

class FDMXPixelMappingToolkit;
class UDMXEntityFixturePatch;
class UDMXPixelMappingDMXLibraryViewModel;


/** Displays the DMX Library of the currently selected fixture group component */
class SDMXPixelMappingFixturePatchList final
	: public SDMXReadOnlyFixturePatchList
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingFixturePatchList) 
	{}
		/** Describes the initial state of the list */
		SLATE_ARGUMENT(FDMXReadOnlyFixturePatchListDescriptor, ListDescriptor)

		/** Called when a row of the list is right clicked */
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingDMXLibraryViewModel> InDMXLibraryModel);

	/** Selects the first patch after the specified fixture patches */
	void SelectAfter(const TArray<UDMXEntityFixturePatch*>& FixturePatches);

private:
	//~ Begin SDMXReadOnlyFixturePatchList interface
	virtual void ForceRefresh() override;
	//~ End SDMXReadOnlyFixturePatchList interface

	/** Called when a row in the list was dragged */
	FReply OnRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Holds the fixutre patches which are hidden the list */
	TArray<TSharedPtr<FDMXEntityFixturePatchRef>> HiddenFixturePatches;

	/** View model of the displayed dmx library */
	TWeakObjectPtr<UDMXPixelMappingDMXLibraryViewModel> WeakDMXLibraryViewModel;

	/** The toolkit of the editor that displays this widget */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
