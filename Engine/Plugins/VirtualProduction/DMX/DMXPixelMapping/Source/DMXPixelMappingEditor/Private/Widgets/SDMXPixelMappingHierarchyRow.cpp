// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingHierarchyRow.h"

#include "Components/DMXPixelMappingOutputComponent.h"
#include "DMXEditorStyle.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingEditorUtils.h"
#include "DragDrop/DMXPixelMappingDragDropOp.h"
#include "ScopedTransaction.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SListView.h"
#include "ViewModels/DMXPixelMappingHierarchyItem.h"
#include "Views/SDMXPixelMappingHierarchyView.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingHierarchyRow"

void SDMXPixelMappingHierarchyRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit, const TSharedRef<FDMXPixelMappingHierarchyItem>& InItem)
{
	WeakToolkit = InWeakToolkit;
	Item = InItem;

	SMultiColumnTableRow<TSharedPtr<FDMXPixelMappingHierarchyItem>>::Construct(
		FSuperRowType::FArguments()
		.Padding(0.0f)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row"))
		.OnDragDetected(this, &SDMXPixelMappingHierarchyRow::OnRowDragDetected),
		InOwnerTableView);
}

void SDMXPixelMappingHierarchyRow::EnterRenameMode()
{
	EditableNameTextBox->EnterEditingMode();
}

TSharedRef<SWidget> SDMXPixelMappingHierarchyRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == SDMXPixelMappingHierarchyView::FColumnIds::EditorColor)
	{
		return GenerateEditorColorWidget();
	}
	else if (ColumnName == SDMXPixelMappingHierarchyView::FColumnIds::ComponentName)
	{			
		// The name column gets the tree expansion arrow
		return SNew(SBox)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SExpanderArrow, SharedThis(this))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					GenerateComponentNameWidget()
				]
			];
	}
	else if (ColumnName == SDMXPixelMappingHierarchyView::FColumnIds::FixtureID)
	{
		return GenerateFixtureIDWidget();
	}
	else if (ColumnName == SDMXPixelMappingHierarchyView::FColumnIds::Patch)
	{
		return GeneratePatchWidget();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDMXPixelMappingHierarchyRow::GenerateEditorColorWidget()
{
	if (Item.IsValid() && Item->HasEditorColor())
	{
		return
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.Padding(5.f, 2.f)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			[
				SAssignNew(EditorColorImageWidget, SImage)
				.Image(FDMXEditorStyle::Get().GetBrush("DMXEditor.WhiteRoundedPropertyBorder"))
			.ColorAndOpacity_Lambda([this]() -> FSlateColor
				{
					return Item.IsValid() ? Item->GetEditorColor() : FLinearColor::Red;
				})
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SDMXPixelMappingHierarchyRow::GenerateComponentNameWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SAssignNew(EditableNameTextBox, SInlineEditableTextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXPixelMappingHierarchyItem::GetComponentNameText)
			.OnVerifyTextChanged(this, &SDMXPixelMappingHierarchyRow::OnVerifyNameTextChanged)
			.OnTextCommitted(this, &SDMXPixelMappingHierarchyRow::OnNameTextCommited)
			.ColorAndOpacity(FLinearColor::White)
		];
}

TSharedRef<SWidget> SDMXPixelMappingHierarchyRow::GenerateFixtureIDWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXPixelMappingHierarchyItem::GetFixtureIDText)
			.ColorAndOpacity(FLinearColor::White)
		];
}

TSharedRef<SWidget> SDMXPixelMappingHierarchyRow::GeneratePatchWidget()
{
	return
		SNew(SBorder)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(4.f)
		.BorderImage(FAppStyle::GetBrush("NoBorder"))
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(Item.Get(), &FDMXPixelMappingHierarchyItem::GetPatchText)
			.ColorAndOpacity(FLinearColor::White)
		];
}

bool SDMXPixelMappingHierarchyRow::OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage)
{
	UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	const UDMXPixelMappingBaseComponent* MyComponent = Item->GetComponent();
	if (PixelMapping && MyComponent)
	{
		TArray<const UDMXPixelMappingBaseComponent*> AllComponents;
		PixelMapping->GetAllComponentsOfClass(AllComponents);

		const UClass* MyComponentClass = MyComponent->GetClass();
		const FString DesiredName = InText.ToString();
		
		// Allow to clear the name to revert to the generated name
		if (DesiredName.IsEmpty())
		{
			return true;
		}

		const bool bUniqueName = Algo::FindByPredicate(AllComponents, [MyComponent, MyComponentClass, &DesiredName](const UDMXPixelMappingBaseComponent* OtherComponent)
			{
				return
					OtherComponent &&
					OtherComponent != MyComponent &&
					OtherComponent->GetUserName() == DesiredName;
			}) == nullptr;
		if (!bUniqueName)
		{
			OutErrorMessage = LOCTEXT("InvalidComponentNameInfo", "A component with this name already exists");
		}

		return bUniqueName;
	}

	return true;
}

void SDMXPixelMappingHierarchyRow::OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = WeakToolkit.Pin();
	UDMXPixelMappingBaseComponent* Component = Item.IsValid() ? Item->GetComponent() : nullptr;
	if (Toolkit.IsValid() && Component)
	{
		const FScopedTransaction RenameComponentTransaction(LOCTEXT("RenameComponentTransaction", "Rename Pixel Mapping Component"));

		Component->Modify();
		Component->SetUserName(InText.ToString());

		// To ease debugging also rename the UObject
		const FName ObjectName = MakeObjectNameFromDisplayLabel(InText.ToString(), Component->GetFName());
		const FName UniqueObjectName = MakeUniqueObjectName(Component->GetOuter(), Component->GetClass(), ObjectName);
		Component->Rename(*UniqueObjectName.ToString());
	}
}

FReply SDMXPixelMappingHierarchyRow::OnRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// SMultiColumnTableRow for some reason does not provide the mouse key, so testing for LeftMouseButton is not required.
	if (!WeakToolkit.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSet<FDMXPixelMappingComponentReference>& SelectedComponents = WeakToolkit.Pin()->GetSelectedComponents();
	TArray<TWeakObjectPtr<UDMXPixelMappingBaseComponent>> DraggedComponents;
	Algo::TransformIf(SelectedComponents, DraggedComponents,
		[](const FDMXPixelMappingComponentReference& ComponentReference)
		{
			return ComponentReference.IsValid();
		},
		[](const FDMXPixelMappingComponentReference& ComponentReference)
		{
			return ComponentReference.GetComponent();
		}
	);

	return FReply::Handled().BeginDragDrop(FDMXPixelMappingDragDropOp::New(WeakToolkit.Pin().ToSharedRef(), FVector2D::ZeroVector, DraggedComponents));
}

#undef LOCTEXT_NAMESPACE
