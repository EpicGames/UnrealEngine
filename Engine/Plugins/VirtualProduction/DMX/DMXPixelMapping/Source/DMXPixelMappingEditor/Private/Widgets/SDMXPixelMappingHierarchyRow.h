// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FDMXPixelMappingHierarchyItem;
class FDMXPixelMappingToolkit;
class SInlineEditableTextBlock;
class SDMXPixelMappingHierarchyView;
class SImage;
class SInlineEditableTextBox;
class STableViewBase;


/** Displays a single item in a hierarchy view */
class SDMXPixelMappingHierarchyRow final
	: public SMultiColumnTableRow<TSharedPtr<FDMXPixelMappingHierarchyItem>>
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingHierarchyRow) {}

		/** Delegate executed when a row was dragged */
		SLATE_EVENT(FOnDragDetected, OnItemDragDetected)

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit, const TSharedRef<FDMXPixelMappingHierarchyItem>& InItem);

	/** Starts editing the name of the component of this row */
	void EnterRenameMode();

private:
	//~ Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	//~ End SMultiColumnTableRow interface

	/** Generates a widget to display the editor color */
	TSharedRef<SWidget> GenerateEditorColorWidget();

	/** Generates a widget to display the name */
	TSharedRef<SWidget> GenerateComponentNameWidget();

	/** Generates a widget to display the fixture ID */
	TSharedRef<SWidget> GenerateFixtureIDWidget();

	/** Generates a widget to display the patch */
	TSharedRef<SWidget> GeneratePatchWidget();

	/** Called to verify the changed name in the related editable text block */
	bool OnVerifyNameTextChanged(const FText& InText, FText& OutErrorMessage);

	/** Called when text is committed on the node */
	void OnNameTextCommited(const FText& InText, ETextCommit::Type CommitInfo);

	/** Called when this row was dragged */
	FReply OnRowDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	/** Model of the component being displayed */
	TSharedPtr<FDMXPixelMappingHierarchyItem> Item;

	/** The editor color image widget */
	TSharedPtr<SImage> EditorColorImageWidget;

	/** Text box that displays and allows to edit the name */
	TSharedPtr<SInlineEditableTextBlock> EditableNameTextBox;

	/** Weak toolkit this widget belongs to */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
