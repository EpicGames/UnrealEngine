// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingOutputComponent.h"

#include "DMXPixelMappingEditorStyle.h"
#include "DMXPixelMappingTypes.h"
#include "DMXPixelMappingUtils.h"
#include "DMXRuntimeUtils.h"
#include "Fonts/FontMeasure.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "ViewModels/DMXPixelMappingOutputComponentModel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SDMXPixelMappingOutputComponentText.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingOutputComponent"

namespace UE::DMX
{
	IDMXPixelMappingOutputComponentWidgetInterface::~IDMXPixelMappingOutputComponentWidgetInterface()
	{
		RemoveFromCanvas();
	}

	void IDMXPixelMappingOutputComponentWidgetInterface::AddToCanvas(const TSharedRef<SConstraintCanvas>& InCanvas)
	{
		RemoveFromCanvas();

		ParentCanvas = InCanvas;

		InCanvas->AddSlot()
			.ZOrder(0)
			.AutoSize(true)
			.Alignment(FVector2D::ZeroVector)
			.Offset_Lambda([this]()
				{
					const FVector2D Position = GetPosition();
					return FMargin(Position.X, Position.Y);
				})
			.Expose(Slot)
			[
				AsWidget()
			];
	}

	void IDMXPixelMappingOutputComponentWidgetInterface::RemoveFromCanvas()
	{
		if (ParentCanvas.IsValid() && Slot)
		{
			ParentCanvas.Pin()->RemoveSlot(Slot->GetWidget());
		}

		Slot = nullptr;
		ParentCanvas.Reset();
	}

	void SDMXPixelMappingOutputComponent::Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingOutputComponent> InOutputComponent)
	{
		WeakToolkit = InToolkit;
		Model = MakeShared<FDMXPixelMappingOutputComponentModel>(InToolkit, InOutputComponent);
		if (!InOutputComponent.IsValid())
		{
			return;
		}

		SetRenderTransform(TAttribute<TOptional<FSlateRenderTransform>>::CreateSP(this, &SDMXPixelMappingOutputComponent::GetRenderTransform));
		SetRenderTransformPivot(FVector2D(0.5, 0.5));

		// Define the bounding box, rest is painted OnPaint.
		ChildSlot
		[
			SNew(SBox)
			.WidthOverride_Lambda([this]()
				{
					return Model->GetSize().X;
				})
			.HeightOverride_Lambda([this]()
				{
					return Model->GetSize().Y;
				})
			[
				SNew(SDMXPixelMappingOutputComponentText, InToolkit, InOutputComponent)
			]
		];
	}

	bool SDMXPixelMappingOutputComponent::Equals(UDMXPixelMappingBaseComponent* Component) const
	{
		return Model->Equals(Component);
	}

	FVector2D SDMXPixelMappingOutputComponent::GetPosition() const
	{
		return Model->GetPosition();
	}

	int32 SDMXPixelMappingOutputComponent::OnPaint(
		const FPaintArgs& Args,
		const FGeometry& AllottedGeometry,
		const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements,
		int32 LayerId,
		const FWidgetStyle& InWidgetStyle,
		bool bParentEnabled) const
	{
		if (!Model->ShouldDraw())
		{
			return LayerId;
		}
		LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

		const FSlateBrush* BorderBrush = FDMXPixelMappingEditorStyle::Get().GetBrush("DMXPixelMappingEditor.ComponentBorder");
		const FLinearColor Color = Model->GetColor();

		const FVector2D Size = Model->GetSize();
		
		// Draw the component box
		TArray<FVector2D> BoxPoints
		{
			FVector2D::ZeroVector,
			FVector2D(0.0, Size.Y),
			Size,
			FVector2D(Size.X, 0.0),
			FVector2D::ZeroVector
		};

		constexpr bool bAntialias = true;
		constexpr float BoxLineThickness = 2.f;
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			BoxPoints,
			ESlateDrawEffect::None,
			Color,
			bAntialias,
			BoxLineThickness);

		// Selectively draw the pivot
		const UDMXPixelMappingEditorSettings* Settings = GetDefault<UDMXPixelMappingEditorSettings>();
		if (Model->ShouldDrawPivot())
		{
			const float PivotLength = FMath::Min(Size.X / 16.0, Size.Y / 16.0); // Y is up
			TArray<FVector2D> PivotPointsX
			{
				FVector2D(Size.X / 2.0, Size.Y / 2.0),
				FVector2D(Size.X / 2.0 + PivotLength, Size.Y / 2.0)
			};

			TArray<FVector2D> PivotPointsY
			{
				FVector2D(Size.X / 2.0, Size.Y / 2.0),
				FVector2D(Size.X / 2.0, Size.Y / 2.0 - PivotLength)
			};

			constexpr float PivotLineThickness = 1.f;
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				PivotPointsX,
				ESlateDrawEffect::None,
				FLinearColor::Red,
				bAntialias,
				PivotLineThickness);

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				PivotPointsY,
				ESlateDrawEffect::None,
				FLinearColor::Green,
				bAntialias,
				PivotLineThickness);
		}

		return LayerId + 1;
	}

	TOptional<FSlateRenderTransform> SDMXPixelMappingOutputComponent::GetRenderTransform() const
	{	
		const FQuat2D Quaternion = Model->GetQuaternion();
	
		return FSlateRenderTransform(Quaternion, FVector2D::ZeroVector);
	}
}

#undef LOCTEXT_NAMESPACE
