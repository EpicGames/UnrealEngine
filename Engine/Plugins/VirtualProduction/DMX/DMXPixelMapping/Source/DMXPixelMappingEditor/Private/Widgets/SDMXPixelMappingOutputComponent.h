// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SConstraintCanvas.h"

class FDMXPixelMappingToolkit;
class SBorder;
class SBox;
class STextBlock;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingOutputComponent;

namespace UE::DMX
{
	class FDMXPixelMappingOutputComponentModel;

	/** Interface for Output Component Widgets */
	class IDMXPixelMappingOutputComponentWidgetInterface
	{
	public:
		/** Destructor */
		virtual ~IDMXPixelMappingOutputComponentWidgetInterface();

		/** Adds the widgets to a canvas. If it already resides in a canvas it is removed from that first. */
		void AddToCanvas(const TSharedRef<SConstraintCanvas>& InCanvas);

		/** Removes the widget from the canvas, if it was added to one. */
		void RemoveFromCanvas();

		/** Returns the actual widget implementation */
		virtual TSharedRef<SWidget> AsWidget() = 0;

		/** Returns true if the widet equals the component */
		virtual bool Equals(UDMXPixelMappingBaseComponent* Component) const = 0;

	protected:
		/** Returns the position of the widget */
		virtual FVector2D GetPosition() const = 0;

	private:
		/** When added to a parent, the canvas it was added to */
		TWeakPtr<SConstraintCanvas> ParentCanvas;

		/** The canvas slot of the component widget */
		SConstraintCanvas::FSlot* Slot = nullptr;
	};


	/** Widget that draws an Output Component. */
	class SDMXPixelMappingOutputComponent
		: public IDMXPixelMappingOutputComponentWidgetInterface
		, public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXPixelMappingOutputComponent)
		{}

		SLATE_END_ARGS()

		/** Constructs the widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingOutputComponent> InOutputComponent);

		//~ Begin IDMXPixelMappingOutputComponentWidgetInterface
		virtual TSharedRef<SWidget> AsWidget() override { return AsShared(); };
		virtual bool Equals(UDMXPixelMappingBaseComponent* Component) const override;

	protected:
		virtual FVector2D GetPosition() const override;
		//~ End IDMXPixelMappingOutputComponentWidgetInterface

		//~ Begin SWidget Interface
		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
		//~ End SWidget Interface

	private:
		/** Returns the render transform for this widget */
		TOptional<FSlateRenderTransform> GetRenderTransform() const;

		/** The model for this widget */
		TSharedPtr<FDMXPixelMappingOutputComponentModel> Model;

		/** The toolkit that owns */
		TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
	};
}