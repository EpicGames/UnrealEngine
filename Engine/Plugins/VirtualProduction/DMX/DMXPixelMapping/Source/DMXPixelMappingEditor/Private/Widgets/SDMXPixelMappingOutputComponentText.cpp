// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingOutputComponentText.h"

#include "DMXPixelMapping.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/CoreStyle.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "ViewModels/DMXPixelMappingOutputComponentModel.h"


void SDMXPixelMappingOutputComponentText::Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingOutputComponent> OutputComponent)
{
	if (!OutputComponent.IsValid())
	{
		return;
	}

	WeakToolkit = InToolkit;
	Model = MakeShared<UE::DMX::FDMXPixelMappingOutputComponentModel>(InToolkit, OutputComponent);

	// Don't clip if the name has to be painted above the component box, otherwise clip. 
	const EWidgetClipping WidgetClipping = Model->ShouldDrawNameAbove() ? EWidgetClipping::Inherit : EWidgetClipping::ClipToBoundsAlways;
	SetClipping(WidgetClipping);
}

int32 SDMXPixelMappingOutputComponentText::OnPaint(
	const FPaintArgs& Args,
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled) const
{
	if (!Model->ShouldDraw())
	{
		return LayerId;
	}
	LayerId = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

	// Selectively paint labels
	const FVector2f Scale = AllottedGeometry.GetAccumulatedRenderTransform().GetMatrix().GetScale().GetVector();
	if (Model->ShouldDrawName())
	{
		OnPaintComponentName(Args, AllottedGeometry, OutDrawElements, LayerId, Scale);
	}

	if (Model->ShouldDrawCellID())
	{
		OnPaintCellID(Args, AllottedGeometry, OutDrawElements, LayerId, Scale);
	}

	if (Model->ShouldDrawPatchInfo())
	{
		OnPaintPatchInfo(Args, AllottedGeometry, OutDrawElements, LayerId, Scale);
	}

	return LayerId + 1;
}

void SDMXPixelMappingOutputComponentText::OnPaintComponentName(const FPaintArgs& Args, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2f& Scale) const
{
	const float Margin = 8.f / Scale.Y;
	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();

	// Upscale the font so it stays the same size, regardless of scaling (zoom)
	const float ScaledFontSize = GetFontSize() / Scale.Y;

	// Always take the upper third section
	const float MaxFontSize = LocalSize.Y - ScaledFontSize;
	const float ClampedFontSize = FMath::Clamp(ScaledFontSize, UE_DELTA, MaxFontSize);

	const float OffsetX = Margin;
	const float OffsetY = Model->ShouldDrawNameAbove() ? -ScaledFontSize * 1.5f : Margin;
	
	const FSlateLayoutTransform LayoutTransform(FVector2f(OffsetX, OffsetY));

	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(LayoutTransform),
		Model->GetName(),
		FCoreStyle::GetDefaultFontStyle("Regular", ClampedFontSize),
		ESlateDrawEffect::None,
		FLinearColor::White.CopyWithNewOpacity(.8f));
}

void SDMXPixelMappingOutputComponentText::OnPaintCellID(const FPaintArgs& Args, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2f& Scale) const
{
	const float Margin = 8.f / Scale.Y;
	const FVector2f LocalSize = AllottedGeometry.GetLocalSize();

	// Upscale the font so it stays the same size, regardless of scaling (zoom)
	const float ScaledFontSize = GetFontSize() / Scale.Y;

	// Draw in the center
	const float MaxFontSize = LocalSize.Y - ScaledFontSize - Margin * 2.f;
	const float ClampedFontSize = FMath::Clamp(ScaledFontSize, UE_DELTA, MaxFontSize);

	const FText CellIDText = Model->GetCellIDText();
	const FSlateFontInfo Font = FCoreStyle::GetDefaultFontStyle("Regular", ScaledFontSize);

	const TSharedRef<FSlateFontMeasure>& FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FVector2D TextSize = FontMeasureService->Measure(CellIDText, Font) / Scale;

	const FSlateLayoutTransform LayoutTransform(FVector2f(LocalSize.X / 2.f - TextSize.X, LocalSize.Y / 2.f - TextSize.Y));

	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(LayoutTransform),
		Model->GetCellIDText(),
		Font,
		ESlateDrawEffect::None,
		FLinearColor::White.CopyWithNewOpacity(.6f));
}

void SDMXPixelMappingOutputComponentText::OnPaintPatchInfo(const FPaintArgs& Args, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2f& Scale) const
{
	const float Margin = 8.f / Scale.Y;
	const float LocalSizeY = AllottedGeometry.GetLocalSize().Y;

	// Upscale the font so it stays the same size, regardless of scaling (zoom)
	const float ScaledFontSize = GetFontSize() / Scale.Y;
	const float LineSpacing = 4.f / Scale.Y;

	// Find a font size that fits below the component name. Consider two lines of text are painted
	const float MaxFontSize = LocalSizeY - ScaledFontSize * 2.f - LineSpacing - Margin * 2.f;
	const float ClampedFontSize = FMath::Clamp(ScaledFontSize, UE_DELTA, MaxFontSize);

	// Arrange both text elements in the lower section, align bottom
	const float FixtureIDTextOffsetY = LocalSizeY - ClampedFontSize * 2.f - Margin - LineSpacing;
	const FSlateLayoutTransform FixtureIDLayoutTransform(FVector2f(Margin, FixtureIDTextOffsetY));

	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(FixtureIDLayoutTransform),
		Model->GetFixtureIDText(),
		FCoreStyle::GetDefaultFontStyle("Regular", ClampedFontSize),
		ESlateDrawEffect::None,
		FLinearColor::Green.CopyWithNewOpacity(.8f));

	const float AddressesTextOffsetY = LocalSizeY - ClampedFontSize - Margin;
	const FSlateLayoutTransform AddressesLayoutTransform(FVector2f(Margin, AddressesTextOffsetY));

	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(AddressesLayoutTransform),
		Model->GetAddressesText(),
		FCoreStyle::GetDefaultFontStyle("Regular", ClampedFontSize),
		ESlateDrawEffect::None,
		FLinearColor::Green.CopyWithNewOpacity(.8f));
}

uint8 SDMXPixelMappingOutputComponentText::GetFontSize() const
{
	UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
	if (PixelMapping)
	{
		return PixelMapping->ComponentLabelFontSize;
	}

	return 8;
}
