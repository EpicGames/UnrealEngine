// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/TransformCalculus2D.h"
#include "Widgets/SCompoundWidget.h"

class FDMXPixelMappingToolkit;
class UDMXPixelMappingOutputComponent;
namespace UE::DMX { class FDMXPixelMappingOutputComponentModel; }


/** Draws the text in an output component widget */
class SDMXPixelMappingOutputComponentText
	: public SCompoundWidget
{

public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingOutputComponentText)
		{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit, TWeakObjectPtr<UDMXPixelMappingOutputComponent> OutputComponent);

	/** The model for this widget */
	TSharedPtr<UE::DMX::FDMXPixelMappingOutputComponentModel> Model;

	/** The toolkit that owns */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;

protected:
	//~ Begin SWidget Interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	//~ End SWidget Interface

private:
	/** Helper to paint the component name */
	void OnPaintComponentName(const FPaintArgs& Args, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2f& Scale) const;

	/** Helper to paint the cell ID */
	void OnPaintCellID(const FPaintArgs& Args, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2f& Scale) const;

	/** Helper to paint the patch info */
	void OnPaintPatchInfo(const FPaintArgs& Args, const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FVector2f& Scale) const;

	/** Returns the current font size */
	uint8 GetFontSize() const;
};
