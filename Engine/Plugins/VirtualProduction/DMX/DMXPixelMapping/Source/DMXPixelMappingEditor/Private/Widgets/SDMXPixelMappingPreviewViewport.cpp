// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingPreviewViewport.h"

#include "Components/DMXPixelMappingRendererComponent.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Viewports/DMXPixelMappingSceneViewport.h"
#include "Viewports/DMXPixelMappingPreviewViewportClient.h"
#include "Widgets/SViewport.h"
#include "Widgets/Layout/SBox.h"


void SDMXPixelMappingPreviewViewport::Construct(const FArguments& InArgs, TWeakPtr<FDMXPixelMappingToolkit> InWeakToolkit)
{
	WeakToolkit = InWeakToolkit;

	ChildSlot
		[
			SNew(SBox)
			.Visibility(EVisibility::HitTestInvisible)
			.Padding(this, &SDMXPixelMappingPreviewViewport::GetPaddingGraphSpace)
			.WidthOverride(this, &SDMXPixelMappingPreviewViewport::GetWidthGraphSpace)
			.HeightOverride(this, &SDMXPixelMappingPreviewViewport::GetHeightGraphSpace)
			[
				SAssignNew(ViewportWidget, SViewport)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				.EnableGammaCorrection(false)
				.ShowEffectWhenDisabled(false)
			]
		];

	ViewportClient = MakeShared<FDMXPixelMappingPreviewViewportClient>(InWeakToolkit, SharedThis(this));
	Viewport = FDMXPixelMappingSceneViewport::Create(ViewportClient, ViewportWidget);
}

FOptionalSize SDMXPixelMappingPreviewViewport::GetWidthGraphSpace() const
{
	if (ViewportClient->IsDrawingVisibleRectOnly())
	{
		return ViewportClient->GetVisibleTextureBoxGraphSpace().Max.X;
	}
	else if (UTexture* InputTexture = GetInputTexture())
	{
		return InputTexture->GetSurfaceWidth();
	}
	return FOptionalSize();
}

FOptionalSize SDMXPixelMappingPreviewViewport::GetHeightGraphSpace() const
{
	if (ViewportClient->IsDrawingVisibleRectOnly())
	{
		return ViewportClient->GetVisibleTextureBoxGraphSpace().Max.Y;
	}
	else if (UTexture* InputTexture = GetInputTexture())
	{
		return InputTexture->GetSurfaceHeight();
	}
	return FOptionalSize();
}

void SDMXPixelMappingPreviewViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Viewport->Invalidate();
}

FMargin SDMXPixelMappingPreviewViewport::GetPaddingGraphSpace() const
{
	if (ViewportClient->IsDrawingVisibleRectOnly())
	{
		return FMargin(ViewportClient->GetVisibleTextureBoxGraphSpace().Min.X, ViewportClient->GetVisibleTextureBoxGraphSpace().Min.Y, 0.0, 0.0);
	}

	return FMargin(0.f);
}

UTexture* SDMXPixelMappingPreviewViewport::GetInputTexture() const
{
	UDMXPixelMappingRendererComponent* RendererComponent = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetActiveRendererComponent() : nullptr;
	
	return RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
}
