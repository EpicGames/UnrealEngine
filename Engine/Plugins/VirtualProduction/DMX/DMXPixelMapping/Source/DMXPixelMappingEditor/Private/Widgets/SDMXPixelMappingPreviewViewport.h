// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FSceneViewport;
class SViewport;
class FDMXPixelMappingPreviewViewportClient;
class FDMXPixelMappingToolkit;
class UTexture;
class FTextureResource;
class UDMXPixelMappingOutputComponent;
struct FOptionalSize;

class SDMXPixelMappingPreviewViewport
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDMXPixelMappingPreviewViewport) { }
	SLATE_END_ARGS()

public:
	/**
	 * Constructs the widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	void Construct(const FArguments& InArgs, TWeakPtr<FDMXPixelMappingToolkit> InToolkit);

	TSharedPtr<FSceneViewport> GetViewport() const { return Viewport; }

	TSharedPtr<SViewport> GetViewportWidget() const { return ViewportWidget; }

	/** Returns the texture width, in graph space  */
	FOptionalSize GetWidthGraphSpace() const;

	/** Returns the texture height, in graph space  */
	FOptionalSize GetHeightGraphSpace() const;

protected:
	//~ Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End of SWidget interface

private:
	/** Returns the padding of content, in graph space */
	FMargin GetPaddingGraphSpace() const;

	/** Returns the input texture currently in use */
	UTexture* GetInputTexture() const;

	/** Viewport client. */
	TSharedPtr<FDMXPixelMappingPreviewViewportClient> ViewportClient;

	/** Slate viewport for rendering and IO. */
	TSharedPtr<FSceneViewport> Viewport;

	/** Viewport widget. */
	TSharedPtr<SViewport> ViewportWidget;

	/** Weak pointer to the editor toolkit */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
};
