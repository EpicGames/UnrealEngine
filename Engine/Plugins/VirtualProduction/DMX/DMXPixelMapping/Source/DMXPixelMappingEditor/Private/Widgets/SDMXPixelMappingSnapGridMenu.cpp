// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingSnapGridMenu.h"

#include "DMXPixelMapping.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingSnapGridMenu"

namespace UE::DMX
{
	void SDMXPixelMappingSnapGridMenu::Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit)
	{
		WeakToolkit = InToolkit;

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
			.Padding(12.f)
			[
				SNew(SGridPanel)
				
				// Enable grid snapping
				+ SGridPanel::Slot(0, 0)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SnapGridEnabledLabel", "Enable Grid Snapping"))
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				]

				+ SGridPanel::Slot(1, 0)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(InArgs._IsChecked)
					.OnCheckStateChanged(InArgs._OnCheckStateChanged)
				]

				// Num Rows
				+ SGridPanel::Slot(0, 1)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SnapGridColumnsLabel", "Num Columns (X)"))
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.IsEnabled(this, &SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled)
				]

				+ SGridPanel::Slot(1, 1)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<int32>)
					.MinDesiredValueWidth(60.f)
					.AllowSpin(true)
					.MinValue(1)
					.MaxValue(32768)
					.MinSliderValue(1)
					.MaxSliderValue(512)
					.Value(this, &SDMXPixelMappingSnapGridMenu::GetColumns)
					.OnValueChanged(this, &SDMXPixelMappingSnapGridMenu::OnColumnsEdited)
					.IsEnabled(this, &SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled)
				]

				// Num Columns
				+ SGridPanel::Slot(0, 2)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SnapGridRowsLabel", "Num Rows (Y)"))
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.IsEnabled(this, &SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled)
				]

				+ SGridPanel::Slot(1, 2)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<int32>)
					.MinDesiredValueWidth(60.f)
					.AllowSpin(true)
					.MinValue(1)
					.MaxValue(32768)
					.MinSliderValue(1)
					.MaxSliderValue(512)
					.Value(this, &SDMXPixelMappingSnapGridMenu::GetRows)
					.OnValueChanged(this, &SDMXPixelMappingSnapGridMenu::OnRowsEdited)
					.IsEnabled(this, &SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled)
				]

				// Color picker
				+ SGridPanel::Slot(0, 3)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ColorPickerLabel", "Color"))
					.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
					.IsEnabled(this, &SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled)
				]

				+ SGridPanel::Slot(1, 3)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(4.f)
				[
					SNew(SBox)
					.MinDesiredWidth(60.f)
					[
						SNew(SColorBlock)
						.AlphaBackgroundBrush(FAppStyle::Get().GetBrush("ColorPicker.RoundedAlphaBackground"))
						.Color(this, &SDMXPixelMappingSnapGridMenu::GetColor)
						.ShowBackgroundForAlpha(true)
						.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Separate)
						.OnMouseButtonDown(this, &SDMXPixelMappingSnapGridMenu::OnColorBlockClicked)
						.Size(FVector2D(70.0f, 20.0f))
						.CornerRadius(FVector4(2.0f, 2.0f, 2.0f, 2.0f))
						.IsEnabled(this, &SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled)
					]
				]
			]
		];
	}


	TOptional<int32> SDMXPixelMappingSnapGridMenu::GetColumns() const
	{
		const UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->SnapGridColumns;
		}

		return 1;
	}

	void SDMXPixelMappingSnapGridMenu::OnColumnsEdited(int32 NewColumns)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			const FScopedTransaction SetSnapGridColumnsTransation(LOCTEXT("SetSnapGridColumnsTransation", "Set Pixel Mapping Grid X"));

			PixelMapping->PreEditChange(UDMXPixelMapping::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMapping, SnapGridColumns)));
			PixelMapping->SnapGridColumns = NewColumns;
			PixelMapping->PostEditChange();
		}
	}

	TOptional<int32> SDMXPixelMappingSnapGridMenu::GetRows() const
	{
		const UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->SnapGridRows;
		}

		return 1;
	}

	void SDMXPixelMappingSnapGridMenu::OnRowsEdited(int32 NewRows)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			const FScopedTransaction SetSnapGridRowTransation(LOCTEXT("SetSnapGridRowTransation", "Set Pixel Mapping Grid X"));

			PixelMapping->PreEditChange(UDMXPixelMapping::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDMXPixelMapping, SnapGridRows)));
			PixelMapping->SnapGridRows = NewRows;
			PixelMapping->PostEditChange();
		}
	}

	FLinearColor SDMXPixelMappingSnapGridMenu::GetColor() const
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->SnapGridColor;
		}

		return FLinearColor::Red;
	}

	FReply SDMXPixelMappingSnapGridMenu::OnColorBlockClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (!PixelMapping)
		{
			return FReply::Handled();
		}

		FColorPickerArgs PickerArgs;
		{
			PickerArgs.bUseAlpha = true;
			PickerArgs.bOnlyRefreshOnMouseUp = false;
			PickerArgs.bOnlyRefreshOnOk = false;
			PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
			PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SDMXPixelMappingSnapGridMenu::OnColorSelected);
			PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SDMXPixelMappingSnapGridMenu::OnColorPickerCancelled);
			PickerArgs.OnColorPickerWindowClosed = FOnWindowClosed::CreateSP(this, &SDMXPixelMappingSnapGridMenu::OnColorPickerWindowClosed);
			PickerArgs.InitialColor = PixelMapping->SnapGridColor;
			PickerArgs.ParentWidget = AsShared();
			PickerArgs.bOpenAsMenu = true;
			PickerArgs.bIsModal = false;
		}

		ColorPickerTransaction = MakeShared<FScopedTransaction>(LOCTEXT("ColorPickerTransaction", "Set Pixel Mapping Grid Color"));

		OpenColorPicker(PickerArgs);
		
		return FReply::Handled();
	}

	void SDMXPixelMappingSnapGridMenu::OnColorSelected(FLinearColor NewColor)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			PixelMapping->SnapGridColor = NewColor;
		}
	}

	void SDMXPixelMappingSnapGridMenu::OnColorPickerCancelled(FLinearColor OriginalColor)
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			PixelMapping->SnapGridColor = OriginalColor;
		}
	}

	void SDMXPixelMappingSnapGridMenu::OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window)
	{
		// Complete the transaction
		ColorPickerTransaction.Reset();
	}

	bool SDMXPixelMappingSnapGridMenu::IsGridSnappingEnabled() const
	{
		UDMXPixelMapping* PixelMapping = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetDMXPixelMapping() : nullptr;
		if (PixelMapping)
		{
			return PixelMapping->bGridSnappingEnabled;
		}

		return false;
	}
}

#undef LOCTEXT_NAMESPACE
