// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"
#include "SViewportToolBar.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SCompoundWidget.h"

class FDMXPixelMappingToolkit;
class FScopedTransaction;
class SWindow;


namespace UE::DMX
{
	class SDMXPixelMappingSnapGridMenu
		: public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SDMXPixelMappingSnapGridMenu)
			{}

			/** Called upon state change with the value of the next state */
			SLATE_EVENT(FOnCheckStateChanged, OnCheckStateChanged)

			/** Sets the current checked state of the checkbox */
			SLATE_ATTRIBUTE(ECheckBoxState, IsChecked)

		SLATE_END_ARGS()

	public:
		/** Constructs this widget */
		void Construct(const FArguments& InArgs, const TSharedRef<FDMXPixelMappingToolkit>& InToolkit);

	private:
		/** Returns the number of columns for grid snapping as text */
		TOptional<int32> GetColumns() const;

		/** Called when the number of grid snapping columns was edited */
		void OnColumnsEdited(int32 NewColumns);

		/** Returns the number of rows for grid snapping as text */
		TOptional<int32> GetRows() const;

		/** Called when the number of grid snapping rows was edited */
		void OnRowsEdited(int32 NewRows);

		/** Returns the current grid color */
		FLinearColor GetColor() const;

		/** Called when the grid color block was clicked */
		FReply OnColorBlockClicked(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

		/** Called when a color was selected in the color picker */
		void OnColorSelected(FLinearColor NewColor);

		/** Called when color picking was canceled */
		void OnColorPickerCancelled(FLinearColor OriginalColor);

		/** Called when the color picker window was closed */ 
		void OnColorPickerWindowClosed(const TSharedRef<SWindow>& Window);

		/** Returns true if grid snapping is enabled */
		bool IsGridSnappingEnabled() const;

		/** Transaction when selecting colors in its own menu */
		TSharedPtr<FScopedTransaction> ColorPickerTransaction;

		/** The toolkit that owns this widget */
		TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;

		// Slate args
		FOnCheckStateChanged OnCheckStateChanged;
		ECheckBoxState IsChecked;
	};
}