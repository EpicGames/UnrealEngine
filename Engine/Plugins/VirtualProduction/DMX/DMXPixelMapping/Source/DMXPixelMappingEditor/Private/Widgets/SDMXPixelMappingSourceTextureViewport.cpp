// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingSourceTextureViewport.h"

#include "Components/DMXPixelMappingRendererComponent.h"
#include "Engine/Texture.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SViewport.h"
#include "Viewports/DMXPixelMappingSceneViewport.h"
#include "Viewports/DMXPixelMappingSourceTextureViewportClient.h"
#include "Views/SDMXPixelMappingDesignerView.h"


void SDMXPixelMappingSourceTextureViewport::Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit)
{
	WeakToolkit = InToolkit;

	TSharedPtr<SHorizontalBox> HorizontalBox;

	ChildSlot
		[
			SNew(SBox)
			.Padding(this, &SDMXPixelMappingSourceTextureViewport::GetPaddingGraphSpace)
			.WidthOverride(this, &SDMXPixelMappingSourceTextureViewport::GetWidthGraphSpace)
			.HeightOverride(this, &SDMXPixelMappingSourceTextureViewport::GetHeightGraphSpace)
			[
				SAssignNew(ViewportWidget, SViewport)
				.EnableGammaCorrection(false)
				.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
				.ShowEffectWhenDisabled(false)
				.EnableBlending(true)
			]
		];

	ViewportClient = MakeShared<FDMXPixelMappingSourceTextureViewportClient>(InToolkit, SharedThis(this));
	Viewport = FDMXPixelMappingSceneViewport::Create(ViewportClient, ViewportWidget);
}

FOptionalSize SDMXPixelMappingSourceTextureViewport::GetWidthGraphSpace() const
{
	if (ViewportClient->IsDrawingVisibleRectOnly())
	{
		return ViewportClient->GetVisibleTextureBoxGraphSpace().Max.X;
	}
	else if (UTexture* InputTexture = GetInputTexture())
	{
		return InputTexture->GetSurfaceWidth();
	}
	return FOptionalSize();
}

FOptionalSize SDMXPixelMappingSourceTextureViewport::GetHeightGraphSpace() const
{
	if (ViewportClient->IsDrawingVisibleRectOnly())
	{
		return ViewportClient->GetVisibleTextureBoxGraphSpace().Max.Y;
	}
	else if (UTexture* InputTexture = GetInputTexture())
	{
		return InputTexture->GetSurfaceHeight();
	}
	return FOptionalSize();
}

void SDMXPixelMappingSourceTextureViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Viewport->Invalidate();
}

FMargin SDMXPixelMappingSourceTextureViewport::GetPaddingGraphSpace() const
{
	if (ViewportClient->IsDrawingVisibleRectOnly())
	{
		return FMargin(ViewportClient->GetVisibleTextureBoxGraphSpace().Min.X, ViewportClient->GetVisibleTextureBoxGraphSpace().Min.Y, 0.0, 0.0);
	}
	else
	{
		return 0.0;
	}
}

UTexture* SDMXPixelMappingSourceTextureViewport::GetInputTexture() const
{
	UDMXPixelMappingRendererComponent* RendererComponent = WeakToolkit.IsValid() ? WeakToolkit.Pin()->GetActiveRendererComponent() : nullptr;

	return RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
}
