// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FSceneViewport;
class SViewport;
class FDMXPixelMappingToolkit;
class FDMXPixelMappingSourceTextureViewportClient;
class UTexture;
class FTextureResource;
struct FOptionalSize;

class SDMXPixelMappingSourceTextureViewport
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingSourceTextureViewport) 
	{}
	
	SLATE_END_ARGS()

public:
	/** Constructs this widget */
	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit);

	/** Returns the viewport of this widget */
	TSharedPtr<FSceneViewport> GetViewport() const { return Viewport; }

	/** Returns the viewport widget of this viewport */
	TSharedPtr<SViewport> GetViewportWidget() const { return ViewportWidget; }

	/** Returns the texture width, in graph space  */
	FOptionalSize GetWidthGraphSpace() const;

	/** Returns the texture height, in graph space  */
	FOptionalSize GetHeightGraphSpace() const;

protected:
	// Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

private:
	/** Returns the padding of content, in graph space */
	FMargin GetPaddingGraphSpace() const;

	/** Returns the input texture currently in use */
	UTexture* GetInputTexture() const;

	/** Weak pointer to the editor toolkit */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;

	/** viewport client. */
	TSharedPtr<FDMXPixelMappingSourceTextureViewportClient> ViewportClient;

	/** Slate viewport for rendering and IO. */
	TSharedPtr<FSceneViewport> Viewport;

	/** Viewport widget. */
	TSharedPtr<SViewport> ViewportWidget;
};
