// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/CurveSequence.h"
#include "SNodePanel.h"
#include "Widgets/SCompoundWidget.h"

class FActiveTimerHandle;
class FDMXPixelMappingToolkit;
class FSlateWindowElementList;


/**
 * Most of the logic copied from a private class Engine/Source/Editor/UMGEditor/Private/Designer/SPaintSurface.h
 * Base class for Preview and Designer Grid
 */
class SDMXPixelMappingSurface 
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SDMXPixelMappingSurface )
		: _AllowContinousZoomInterpolation(false)
	{ }

		/** Slot for this designers content (optional) */
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ATTRIBUTE(bool, AllowContinousZoomInterpolation)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<FDMXPixelMappingToolkit>& InToolkit);

	// SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnTouchGesture(const FGeometry& MyGeometry, const FPointerEvent& GestureEvent) override;
	virtual FReply OnTouchEnded(const FGeometry& MyGeometry, const FPointerEvent& InTouchEvent) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual bool SupportsKeyboardFocus() const override { return true; }
	// End of Swidget interface

	/** Returns the view offset */
	FVector2D GetViewOffset() const;

	/** Gets the current zoom factor. */
	float GetZoomAmount() const;

	/** Returns the toolkit used with this surface */
	TSharedPtr<FDMXPixelMappingToolkit> GetToolkit() { return WeakToolkit.Pin(); }

	/** Zooms to fit content */
	void ZoomToFit(bool bInstantZoom);

protected:
	virtual void OnPaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	void PaintBackgroundAsLines(const FSlateBrush* BackgroundImage, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32& DrawLayerId) const;

	void ChangeZoomLevel(int32 ZoomLevelDelta, const FVector2D& WidgetSpaceZoomOrigin, bool bOverrideZoomLimiting);
	
	void PostChangedZoom();

	bool ScrollToLocation(const FGeometry& MyGeometry, FVector2D DesiredCenterPosition, const float InDeltaTime);

	bool ZoomToLocation(const FVector2D& CurrentSizeWithoutZoom, const FVector2D& DesiredSize, bool bDoneScrolling);

	FText GetZoomText() const;
	FSlateColor GetZoomTextColorAndOpacity() const;

	FSlateRect ComputeSensibleBounds() const;

	FVector2D GraphCoordToPanelCoord(const FVector2D& GraphSpaceCoordinate) const;
	FVector2D PanelCoordToGraphCoord(const FVector2D& PanelSpaceCoordinate) const;

protected:
	virtual FSlateRect ComputeAreaBounds() const;
	virtual float GetGridScaleAmount() const;
	virtual int32 GetGraphRulePeriod() const;
	virtual int32 GetGridSize() const = 0;

protected:
	/** The position within the graph at which the user is looking */
	FVector2D ViewOffset;

	/** The position in the grid to begin drawing at. */
	FVector2D GridOrigin;

	/** Should we render the grid lines? */
	bool bDrawGridLines;

	/** Previous Zoom Level */
	int32 PreviousZoomLevel;

	/** How zoomed in/out we are. e.g. 0.25f results in quarter-sized nodes. */
	int32 ZoomLevel;

	/** Are we panning the view at the moment? */
	bool bIsPanning = false;

	/** Are we zooming the view with trackpad at the moment? */
	bool bIsZooming = false;

	/** Allow continuous zoom interpolation? */
	TAttribute<bool> AllowContinousZoomInterpolation;

	/** Fade on zoom for graph */
	FCurveSequence ZoomLevelGraphFade;

	/** Curve that handles fading the 'Zoom +X' text */
	FCurveSequence ZoomLevelFade;

	// The interface for mapping ZoomLevel values to actual node scaling values
	TUniquePtr<FZoomLevelsContainer> ZoomLevels;

	bool bAllowContinousZoomInterpolation;

	bool bTeleportInsteadOfScrollingWhenZoomingToFit;

	FVector2D ZoomTargetTopLeft;
	FVector2D ZoomTargetBottomRight;
	FVector2D ZoomToFitPadding;

	/** The Y component of mouse drag (used when zooming) */
	float TotalMouseDelta;

	/** Offset in the panel the user started the LMB+RMB zoom from */
	FVector2D ZoomStartOffset;

	/**  */
	FVector2D ViewOffsetStart;

	/**  */
	FVector2D MouseDownPositionAbsolute;

	/** Cumulative magnify delta from trackpad gesture */
	float TotalGestureMagnify;

	/** Does the user need to press Control in order to over-zoom. */
	bool bRequireControlToOverZoom;

	/** If true, zooms to fit as soon as a texture is loaded */
	bool bInitialZoomToFit = true;

	/** Toolkit weak pointer */
	TWeakPtr<FDMXPixelMappingToolkit> WeakToolkit;
	
private:
	/** Active timer that handles deferred zooming until the target zoom is reached */
	EActiveTimerReturnType HandleZoomToFit(double InCurrentTime, float InDeltaTime);

	/** The handle to the active timer */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;

	// A flag noting if we have a pending zoom to extents operation to perform next tick.
	bool bDeferredZoomToExtents;
};
