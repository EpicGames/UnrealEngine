// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingTransformHandle.h"

#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMapping.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "Settings/DMXPixelMappingEditorSettings.h"
#include "Toolkits/DMXPixelMappingToolkit.h"
#include "Views/SDMXPixelMappingDesignerView.h"
#include "Widgets/Images/SImage.h"


#define LOCTEXT_NAMESPACE "SDMXPixelMappingTransformHandle"

void SDMXPixelMappingTransformHandle::Construct(const FArguments& InArgs, TSharedPtr<SDMXPixelMappingDesignerView> InDesignerView, EDMXPixelMappingTransformDirection InTransformDirection, TAttribute<FVector2D> InOffset)
{
	TransformDirection = InTransformDirection;
	DesignerViewWeakPtr = InDesignerView;
	Offset = InOffset;

	Action = EDMXPixelMappingTransformAction::None;

	DragDirection = ComputeDragDirection(InTransformDirection);
	DragOrigin = ComputeOrigin(InTransformDirection);

	SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SDMXPixelMappingTransformHandle::GetHandleVisibility));

	ChildSlot
	[
		SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("UMGEditor.TransformHandle"))
	];
}

EVisibility SDMXPixelMappingTransformHandle::GetHandleVisibility() const
{
	if (TransformDirection == EDMXPixelMappingTransformDirection::BottomRight)
	{
		return EVisibility::Visible;
	}

	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = DesignerViewWeakPtr.IsValid() ? DesignerViewWeakPtr.Pin()->GetToolkit() : nullptr;
	if (Toolkit.IsValid() && Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Resize)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

FReply SDMXPixelMappingTransformHandle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = DesignerViewWeakPtr.IsValid() ? DesignerViewWeakPtr.Pin()->GetToolkit() : nullptr;
	if (!Toolkit.IsValid() || MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	const FVector2D LocalSize = MyGeometry.GetLocalSize();
	const FVector2D LocalCursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());

	// Only handle dragging the edges of the widget
	constexpr double MouseThreshold = 10.0;
	if (LocalCursorPos.X < LocalSize.X - MouseThreshold ||
		LocalCursorPos.Y < LocalSize.Y - MouseThreshold)
	{
		return FReply::Unhandled();;
	}

	Action = ComputeActionAtLocation(MyGeometry, MouseEvent);

	const TSet<FDMXPixelMappingComponentReference> SelectedComponentReferences = Toolkit->GetSelectedComponents();
	if (const FDMXPixelMappingComponentReference* ComponentReference = SelectedComponentReferences.FindArbitraryElement())
	{
		UDMXPixelMappingBaseComponent* Component = ComponentReference->GetComponent();

		if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component))
		{
			InitialSize = OutputComponent->GetSize();
			InitialRotation = OutputComponent->GetRotation();
		}

		MouseDownPosition = MouseEvent.GetScreenSpacePosition();

		const FText TransactionText = Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Resize ?
			LOCTEXT("ResizePixelMappingComponent", "PixelMapping: Resize Component") :
			LOCTEXT("RotatePixelMappingComponent", "PixelMapping: Rotate Component");
	
		ScopedTransaction = MakeShareable<FScopedTransaction>(new FScopedTransaction(TransactionText));
		Component->Modify();

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SDMXPixelMappingTransformHandle::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	RequestApplyTransformHandle.Invalidate();

	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = DesignerViewWeakPtr.IsValid() ? DesignerViewWeakPtr.Pin()->GetToolkit() : nullptr;
	if (!Toolkit.IsValid() || !HasMouseCapture() || MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}
		
	Action = EDMXPixelMappingTransformAction::None;

	const TSet<FDMXPixelMappingComponentReference> SelectedComponentReferences = Toolkit->GetSelectedComponents();
	for (const FDMXPixelMappingComponentReference& ComponentReference : SelectedComponentReferences)
	{
		if (UDMXPixelMappingOutputComponent* EditedComponent = Cast<UDMXPixelMappingOutputComponent>(ComponentReference.GetComponent()))
		{
			const FVector2D& CursorPosition = MouseEvent.GetScreenSpacePosition();

			EditedComponent->Modify();
			if (Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Resize)
			{
				Resize(EditedComponent, CursorPosition, MyGeometry.Scale);
			}
			else if (Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Rotate)
			{
				Rotate(EditedComponent, CursorPosition, MyGeometry.Scale);
			}
			else
			{
				checkf(0, TEXT("Unhandled enum value"));
			}

			ScopedTransaction.Reset();

			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return FReply::Unhandled();
}

FReply SDMXPixelMappingTransformHandle::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	TSharedPtr<FDMXPixelMappingToolkit> Toolkit = DesignerViewWeakPtr.IsValid() ? DesignerViewWeakPtr.Pin()->GetToolkit() : nullptr;
	if (!Toolkit.IsValid() || Action == EDMXPixelMappingTransformAction::None)
	{
		return FReply::Unhandled();
	}

	if (RequestApplyTransformHandle.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSet<FDMXPixelMappingComponentReference> SelectedComponentReferences = Toolkit->GetSelectedComponents();
	for (const FDMXPixelMappingComponentReference& ComponentReference : SelectedComponentReferences)
	{
		UDMXPixelMappingOutputComponent* EditedComponent = Cast<UDMXPixelMappingOutputComponent>(ComponentReference.GetComponent());
		if (!EditedComponent)
		{
			continue;
		}

		const FVector2D& CursorPosition = MouseEvent.GetScreenSpacePosition();
		RequestApplyTransformHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda(
			[this, EditedComponent, CursorPosition, MyGeometry, Toolkit]()
			{					
				if (Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Resize)
				{
					Resize(EditedComponent, CursorPosition, MyGeometry.Scale);
				}
				else if (Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Rotate)
				{
					Rotate(EditedComponent, CursorPosition, MyGeometry.Scale);
				}
				else
				{
					checkf(0, TEXT("Unhandled enum value"));
				}

				RequestApplyTransformHandle.Invalidate();
			}));
	}

	return FReply::Unhandled();
}

FCursorReply SDMXPixelMappingTransformHandle::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = DesignerViewWeakPtr.Pin()->GetToolkit();
	if (!Toolkit.IsValid())
	{
		return FCursorReply::Unhandled();
	}

	if (Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Resize)
	{
		switch (TransformDirection)
		{
		case EDMXPixelMappingTransformDirection::BottomRight:
			return FCursorReply::Cursor(EMouseCursor::ResizeSouthEast);
		case EDMXPixelMappingTransformDirection::BottomCenter:
			return FCursorReply::Cursor(EMouseCursor::ResizeUpDown);
		case EDMXPixelMappingTransformDirection::CenterRight:
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
	}
	else if (Toolkit->GetTransformHandleMode() == UE::DMX::EDMXPixelMappingTransformHandleMode::Rotate)
	{
		return FCursorReply::Cursor(EMouseCursor::CardinalCross);
	}
	else
	{
		checkf(0, TEXT("Unhandled enum value"));
	}

	return FCursorReply::Unhandled();
}

void SDMXPixelMappingTransformHandle::Rotate(UDMXPixelMappingOutputComponent* OutputComponent, const FVector2D& CursorPosition, double WidgetScale)
{	
	if (!OutputComponent)
	{
		return;
	}

	// Position of the initially clicked location to the current mouse position in slate space. Remove zoom and scaling to get to graph space.
	const FVector2D MouseDownToCurorPosition = CursorPosition - MouseDownPosition;
	const FVector2D MouseDownToCursorPositionGraphSpace = MouseDownToCurorPosition / DesignerViewWeakPtr.Pin()->GetZoomAmount() / WidgetScale;
	
	const FVector2D InitialCenterToBottomRight = OutputComponent->GetSize().GetRotated(InitialRotation) / 2.0;
	const FVector2D NewCenterToBottomRight = InitialCenterToBottomRight + MouseDownToCursorPositionGraphSpace;

	const double DotProdcutNormalized = FVector2D::DotProduct(InitialCenterToBottomRight.GetSafeNormal(), NewCenterToBottomRight.GetSafeNormal());
	const double Sign = FVector2D::CrossProduct(InitialCenterToBottomRight, NewCenterToBottomRight) > 0.0 ? 1.0 : -1.0;
	const double NewAngle = Sign * FMath::UnwindRadians(FMath::Acos(DotProdcutNormalized));

	OutputComponent->Modify();
	OutputComponent->SetRotation(InitialRotation + FMath::RadiansToDegrees(NewAngle));
}

void SDMXPixelMappingTransformHandle::Resize(UDMXPixelMappingOutputComponent* OutputComponent, const FVector2D& CursorPosition, double WidgetScale)
{
	if (!OutputComponent)
	{
		return;
	}

	// Position of the initially clicked location to the current mouse position in slate space. Remove zoom and scaling to get to graph space.
	const FVector2D MouseDownToCurorPosition = CursorPosition - MouseDownPosition;
	const FVector2D MouseDownToCursorPositionGraphSpace = MouseDownToCurorPosition / DesignerViewWeakPtr.Pin()->GetZoomAmount() / WidgetScale;

	// Rotate to axis aligned, and apply the drag direction.
	const FVector2D ResizeTranslation = MouseDownToCursorPositionGraphSpace.GetRotated(-OutputComponent->GetRotation()) * DragDirection;
	const FVector2D RequestedSize = InitialSize + ResizeTranslation;

	const FVector2D NewSize = GetSnapSize(OutputComponent, RequestedSize);
	const FVector2D OldSize = OutputComponent->GetSize();

	OutputComponent->Modify();
	OutputComponent->SetSize(NewSize);
}

FVector2D SDMXPixelMappingTransformHandle::GetSnapSize(UDMXPixelMappingOutputComponent* OutputComponent, const FVector2D& RequestedSize) const
{
	const TSharedPtr<FDMXPixelMappingToolkit> Toolkit = DesignerViewWeakPtr.Pin()->GetToolkit();
	if (!Toolkit.IsValid() || !OutputComponent || !OutputComponent->GetRendererComponent())
	{
		return FVector2D::Clamp(RequestedSize, FVector2D(1.f, 1.f), RequestedSize);
	}
	UDMXPixelMappingRendererComponent* RendererComponent = OutputComponent->GetRendererComponent();

	const UDMXPixelMapping* PixelMapping = Toolkit->GetDMXPixelMapping();
	if (!PixelMapping || !PixelMapping->bGridSnappingEnabled)
	{
		return FVector2D::Clamp(RequestedSize, FVector2D(1.f, 1.f), RequestedSize);
	}

	const FVector2D CellSize = [PixelMapping, RendererComponent]()
		{
			const FVector2D TextureSize = RendererComponent->GetSize();
			return TextureSize / FVector2D(PixelMapping->SnapGridColumns, PixelMapping->SnapGridRows);
		}();

	// Grid snap bottom right
	const FVector2D RequestedBottomRight = OutputComponent->GetPositionRotated() + RequestedSize;
	const int32 BottomColumn = FMath::RoundHalfToZero(RequestedBottomRight.X / CellSize.X);
	const int32 RightRow = FMath::RoundHalfToZero(RequestedBottomRight.Y / CellSize.Y);

	FVector2D NewBottomRight = OutputComponent->GetPositionRotated() + RequestedSize;
	if (DragDirection.X > 0.f)
	{
		// Snap towards right
		NewBottomRight.X = BottomColumn * CellSize.X;
	}

	if (DragDirection.Y > 0.f)
	{
		// Snap towards bottom
		NewBottomRight.Y = RightRow * CellSize.Y;
	}

	const FVector2D SnapSize = NewBottomRight - OutputComponent->GetPositionRotated();
	FVector2D SnapSizeClamped;
	SnapSizeClamped.X = FMath::Max(SnapSize.X, 1.f);
	SnapSizeClamped.Y = FMath::Max(SnapSize.Y, 1.f);

	return SnapSizeClamped;
}

FVector2D SDMXPixelMappingTransformHandle::ComputeDragDirection(EDMXPixelMappingTransformDirection InTransformDirection) const
{
	switch ( InTransformDirection )
	{
	case EDMXPixelMappingTransformDirection::CenterRight:
		return FVector2D(1, 0);
	case EDMXPixelMappingTransformDirection::BottomCenter:
		return FVector2D(0, 1);
	case EDMXPixelMappingTransformDirection::BottomRight:
		return FVector2D(1, 1);
	}

	return FVector2D(0, 0);
}

FVector2D SDMXPixelMappingTransformHandle::ComputeOrigin(EDMXPixelMappingTransformDirection InTransformDirection) const
{
	FVector2D Size(10, 10);

	switch ( InTransformDirection )
	{
	case EDMXPixelMappingTransformDirection::CenterRight:
		return Size * FVector2D(0, 0.5);
	case EDMXPixelMappingTransformDirection::BottomCenter:
		return Size * FVector2D(0.5, 0);
	case EDMXPixelMappingTransformDirection::BottomRight:
		return Size * FVector2D(0, 0);
	}

	return FVector2D(0, 0);
}

EDMXPixelMappingTransformAction SDMXPixelMappingTransformHandle::ComputeActionAtLocation(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const
{
	FVector2D LocalPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	FVector2D GrabOriginOffset = LocalPosition - DragOrigin;
	if ( GrabOriginOffset.SizeSquared() < 36.f )
	{
		return EDMXPixelMappingTransformAction::Primary;
	}
	else
	{
		return EDMXPixelMappingTransformAction::Secondary;
	}
}

#undef LOCTEXT_NAMESPACE
