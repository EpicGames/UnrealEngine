// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Widgets/SCompoundWidget.h"

class FScopedTransaction;
class SDMXPixelMappingDesignerView;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingOutputComponent;

enum class EDMXPixelMappingTransformDirection : uint8
{
	CenterRight,
	BottomRight,
	BottomCenter,

	MAX
};

enum class EDMXPixelMappingTransformAction : uint8
{
	None,
	Primary,
	Secondary
};


/**
 * Most of the logic copied from a private class Engine/Source/Editor/UMGEditor/Private/Designer/STransformHandle.h
 * This is a small dot widget for control the size of the selected widget
 */
class SDMXPixelMappingTransformHandle
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDMXPixelMappingTransformHandle) 
		: _Offset(FVector2D(0, 0))
	{}
		SLATE_ATTRIBUTE(FVector2D, Offset)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SDMXPixelMappingDesignerView> InDesignerView, EDMXPixelMappingTransformDirection InTransformDirection, TAttribute<FVector2D> InOffset);

	// SWidget interface
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	// End SWidget interface

	EDMXPixelMappingTransformDirection GetTransformDirection() const { return TransformDirection; }
	FVector2D GetOffset() const { return Offset.Get(); }

private:
	EVisibility GetHandleVisibility() const;

	/** Rotate the component */
	void Rotate(UDMXPixelMappingOutputComponent* OutputComponent, const FVector2D& CursorPosition, double WidgetScale);

	/** Resizes the component */
	void Resize(UDMXPixelMappingOutputComponent* OutputComponent, const FVector2D& CursorPosition, double WidgetScale);

	/** Utiltity that returns the snap size of an output component. Note, this snaps to pixels if grid snapping is disabled. */
	FVector2D GetSnapSize(UDMXPixelMappingOutputComponent* OutputComponent, const FVector2D& RequestedSize) const;

	EDMXPixelMappingTransformAction ComputeActionAtLocation(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) const;

	FVector2D ComputeDragDirection(EDMXPixelMappingTransformDirection InTransformDirection) const;
	FVector2D ComputeOrigin(EDMXPixelMappingTransformDirection InTransformDirection) const;

	EDMXPixelMappingTransformDirection TransformDirection;
	EDMXPixelMappingTransformAction Action;

	FVector2D DragDirection = FVector2D::ZeroVector;
	FVector2D DragOrigin = FVector2D::ZeroVector;

	FVector2D MouseDownPosition = FVector2D::ZeroVector;
	FVector2D InitialSize = FVector2D::ZeroVector;
	double InitialRotation = 0.0;

	TSharedPtr<FScopedTransaction> ScopedTransaction;

	TWeakPtr<SDMXPixelMappingDesignerView> DesignerViewWeakPtr;

	TAttribute<FVector2D> Offset;

	/** Timer handle used when resize is requested */
	FTimerHandle RequestApplyTransformHandle;
};
