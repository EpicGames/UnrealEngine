// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SDMXPixelMappingZoomPan.h"

#include "Layout/ArrangedChildren.h"
#include "Layout/LayoutUtils.h"

#include "RHI.h"


void SDMXPixelMappingZoomPan::Construct(const FArguments& InArgs)
{
	bHasRelativeLayoutScale = true;

	ViewOffset = InArgs._ViewOffset;
	ZoomAmount = InArgs._ZoomAmount;

	ChildSlot
	[
		InArgs._Content.Widget
	];
}

void SDMXPixelMappingZoomPan::SetContent(const TSharedRef< SWidget >& InContent)
{
	ChildSlot
	[
		InContent
	];
}

void SDMXPixelMappingZoomPan::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
{
	const EVisibility ChildVisibility = ChildSlot.GetWidget()->GetVisibility();
	if ( ArrangedChildren.Accepts(ChildVisibility) )
	{
		const FMargin SlotPadding(ChildSlot.GetPadding());
		AlignmentArrangeResult XResult = AlignChild<Orient_Horizontal>(AllottedGeometry.Size.X, ChildSlot, SlotPadding, 1);
		AlignmentArrangeResult YResult = AlignChild<Orient_Vertical>(AllottedGeometry.Size.Y, ChildSlot, SlotPadding, 1);
		
		const FVector2D Size = ChildSlot.GetWidget()->GetDesiredSize().ClampAxes(0.f, GMaxTextureDimensions);
		ArrangedChildren.AddWidget( ChildVisibility, AllottedGeometry.MakeChild(
				ChildSlot.GetWidget(),
				FVector2D(XResult.Offset, YResult.Offset) - ViewOffset.Get(),
				ChildSlot.GetWidget()->GetDesiredSize(),
				ZoomAmount.Get()
		) );
	}
}

float SDMXPixelMappingZoomPan::GetRelativeLayoutScale(int32 ChildIndex, float LayoutScaleMultiplier) const
{
	return ZoomAmount.Get();
}
