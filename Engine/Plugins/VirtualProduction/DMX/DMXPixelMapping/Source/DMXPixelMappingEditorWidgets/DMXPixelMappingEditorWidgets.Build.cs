// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class DMXPixelMappingEditorWidgets : ModuleRules
{
	public DMXPixelMappingEditorWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange( new string[] {
			"Core",
			"CoreUObject",
			"Slate",
			"SlateCore",
		});

		PrivateDependencyModuleNames.AddRange( new string[] {
			"Core",
			"Slate",
			"SlateCore",
			
			"DMXProtocol",
			"DMXRuntime",
			"DMXPixelMappingCore",
		});
	}
}
