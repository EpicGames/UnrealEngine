// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingComponentWidget.h"

#include "SDMXPixelMappingComponentBox.h"
#include "SDMXPixelMappingComponentLabel.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS // The whole class and several child widgets are deprecated 5.1
FDMXPixelMappingComponentWidget::FDMXPixelMappingComponentWidget(TSharedPtr<SDMXPixelMappingComponentBox> InComponentBox, TSharedPtr<SDMXPixelMappingComponentLabel> InComponentLabel, bool bShowLabelAbove)
	: ComponentBox(InComponentBox)
	, ComponentLabel(InComponentLabel)
	, bLabelAbove(bShowLabelAbove)
{
	// Create a new component box if none was provided
	if (!InComponentBox.IsValid())
	{
		ComponentBox =
			SNew(SDMXPixelMappingComponentBox);
	}

	// Create a new component label if none was provided
	if (!InComponentLabel.IsValid())
	{
		ComponentLabel =
			SNew(SDMXPixelMappingComponentLabel)
			.bAlignAbove(bShowLabelAbove)
			.bScaleToSize(true);
	}
}

void FDMXPixelMappingComponentWidget::AddToCanvas(const TSharedRef<SConstraintCanvas>& Canvas, float ZOrder)
{
	RemoveFromCanvas();

	OuterCanvas = Canvas;
		
	OuterCanvas->AddSlot()
	.AutoSize(true)
	.Alignment(FVector2D::ZeroVector)
	.ZOrder(ZOrder)
	.Expose(ComponentSlot)
	[
		ComponentBox.ToSharedRef()
	];

	OuterCanvas->AddSlot()
	.AutoSize(true)
	.Expose(LabelSlot)
	.Alignment(FVector2D::ZeroVector)
	[
		ComponentLabel.ToSharedRef()
	];
}

FDMXPixelMappingComponentWidget::~FDMXPixelMappingComponentWidget()
{
	RemoveFromCanvas();
}

void FDMXPixelMappingComponentWidget::RemoveFromCanvas()
{
	if (OuterCanvas.IsValid() && ComponentSlot && LabelSlot)
	{
		OuterCanvas->RemoveSlot(ComponentSlot->GetWidget());
		OuterCanvas->RemoveSlot(LabelSlot->GetWidget());
	}

	ComponentSlot = nullptr;
	LabelSlot = nullptr;
	OuterCanvas.Reset();
}

void FDMXPixelMappingComponentWidget::SetZOrder(float ZOrder)
{
	if (OuterCanvas.IsValid())
	{
		ComponentSlot->SetZOrder(ZOrder);
	}
}

void FDMXPixelMappingComponentWidget::SetPosition(const FVector2D& LocalPosition)
{
	if (OuterCanvas.IsValid() && ComponentSlot && LabelSlot)
	{
		// In the middle of the top left pixel
		const FVector2D UIPosition = FVector2D(LocalPosition.X + 0.5f, LocalPosition.Y + 0.5f);

		ComponentSlot->SetOffset(FMargin(UIPosition.X, UIPosition.Y, 0.f, 0.f));
		LabelSlot->SetOffset(FMargin(UIPosition.X, UIPosition.Y, 0.f, 0.f));
	}
}

void FDMXPixelMappingComponentWidget::SetSize(const FVector2D& LocalSize)
{
	// Limit to >= (1, 1) 
	FVector2D Size = FVector2D(FMath::Max(LocalSize.X, 1.f), FMath::Max(LocalSize.Y, 1.f));

	ComponentBox->SetLocalSize(Size);
	ComponentLabel->SetWidth(Size.X);
}

FVector2D FDMXPixelMappingComponentWidget::GetLocalSize() const
{
	return ComponentBox->GetCachedGeometry().GetLocalSize();
}

void FDMXPixelMappingComponentWidget::SetLabelText(const FText& Text)
{
	ComponentLabel->SetText(Text);
}

void FDMXPixelMappingComponentWidget::SetColor(const FLinearColor& Color)
{
	ComponentBox->SetBorderColor(Color);
}

FLinearColor FDMXPixelMappingComponentWidget::GetColor() const
{
	return ComponentBox->GetBorderColor();
}

void FDMXPixelMappingComponentWidget::SetVisibility(EVisibility Visibility)
{
	ComponentBox->SetVisibility(Visibility);
	ComponentLabel->SetVisibility(Visibility);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
