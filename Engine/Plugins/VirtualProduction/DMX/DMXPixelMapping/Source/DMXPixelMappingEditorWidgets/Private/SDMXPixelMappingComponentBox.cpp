// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingComponentBox.h"

#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS // The whole class is deprecated 5.1
void SDMXPixelMappingComponentBox::Construct(const FArguments& InArgs)
{
	SetCanTick(false);

	IDText = InArgs._IDText;
	MaxIDTextSize = InArgs._MaxIDTextSize;

	BorderBrush.DrawAs = ESlateBrushDrawType::Border;
	BorderBrush.TintColor = FLinearColor(1.f, 0.f, 1.f);
	BorderBrush.Margin = FMargin(1.f);

	ChildSlot
	[
		SAssignNew(ComponentBox, SBox)
		[		
			SNew(SBorder)
			.BorderImage(&BorderBrush)
			[
				SAssignNew(IDTextBox, SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.WidthOverride(MaxIDTextSize.X)
				.HeightOverride(MaxIDTextSize.Y)
				[
					SNew(SScaleBox)
					.Stretch(EStretch::ScaleToFit)
					.StretchDirection(EStretchDirection::DownOnly)
					[
						SAssignNew(IDTextBlock, STextBlock)
						.Text(InArgs._IDText)
					]
				]
			]
		]
	];

	SetLocalSize(LocalSize);
}

void SDMXPixelMappingComponentBox::SetLocalSize(const FVector2D& NewLocalSize)
{
	LocalSize = NewLocalSize;
	
	ComponentBox->SetWidthOverride(NewLocalSize.X);
	ComponentBox->SetHeightOverride(NewLocalSize.Y);

	// Clamp the max ID text size to not exceed the size
	FVector2D LocalIDTextBoxSize = ComponentBox->GetCachedGeometry().GetLocalSize();
	
	FVector2D NewIDTextBoxSize;
	NewIDTextBoxSize.X = FMath::Min(MaxIDTextSize.X - 4.f, LocalSize.X);
	NewIDTextBoxSize.Y = FMath::Min(MaxIDTextSize.Y - 4.f, LocalSize.Y);
	
	NewIDTextBoxSize.X = FMath::Min(MaxIDTextSize.X - 4.f, LocalIDTextBoxSize.X);
	NewIDTextBoxSize.Y = FMath::Min(MaxIDTextSize.Y - 4.f, LocalIDTextBoxSize.Y);

	IDTextBox->SetWidthOverride(NewIDTextBoxSize.X);
	IDTextBox->SetHeightOverride(NewIDTextBoxSize.Y);
}

void SDMXPixelMappingComponentBox::SetIDText(const FText& NewIDText)
{
	IDTextBlock->SetText(NewIDText);
}

FVector2f SDMXPixelMappingComponentBox::GetLocalSize() const
{
	return GetCachedGeometry().GetLocalSize();
}

void SDMXPixelMappingComponentBox::SetBorderColor(const FLinearColor& Color)
{
	BorderBrush.TintColor = Color;
}

FLinearColor SDMXPixelMappingComponentBox::GetBorderColor() const
{
	return BorderBrush.TintColor.GetSpecifiedColor();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
