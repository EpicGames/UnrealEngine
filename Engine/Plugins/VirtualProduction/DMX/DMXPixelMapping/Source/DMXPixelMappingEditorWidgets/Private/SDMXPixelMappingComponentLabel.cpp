// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingComponentLabel.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Text/STextBlock.h"


#if WITH_EDITOR

PRAGMA_DISABLE_DEPRECATION_WARNINGS // The whole class is deprecated 5.1
void SDMXPixelMappingComponentLabel::Construct(const FArguments& InArgs)
{
	SetCanTick(false);

	LabelText = InArgs._LabelText;
	bAlignAbove = InArgs._bAlignAbove;

	TSharedPtr<SWidget> LabelWidget;
	if (InArgs._bScaleToSize)
	{
		LabelWidget =
			SNew(SScaleBox)
			.Stretch(EStretch::ScaleToFit)
			.StretchDirection(EStretchDirection::DownOnly)
			[
				SAssignNew(LabelTextBlock, STextBlock)
				.Text(InArgs._LabelText)
			];
	}
	else
	{
		LabelWidget =
			SNew(SBox)
			[
				SAssignNew(LabelTextBlock, STextBlock)
				.Text(InArgs._LabelText)
			];
	};

	ChildSlot
	[
		SAssignNew(Box, SBox)
		.Padding(FMargin(2.f, 1.f, 2.f, 1.f))
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Top)
		.Padding_Lambda([this]()
			{
				if (bAlignAbove)
				{
					float OffsetY = LabelTextBlock->GetCachedGeometry().GetLocalSize().Y;
					return FMargin(2.f, -OffsetY, 2.f, 2.f);
				}
				else
				{
					return FMargin(2.f, 1.f, 2.f, 2.f);
				}
			})
		[
			LabelWidget.ToSharedRef()
		]
	];
}

void SDMXPixelMappingComponentLabel::SetWidth(const float& Width)
{
	Box->SetWidthOverride(Width);
}

void SDMXPixelMappingComponentLabel::SetText(const FText& Text)
{
	check(LabelTextBlock.IsValid());

	LabelTextBlock->SetText(Text);
}

FVector2f SDMXPixelMappingComponentLabel::GetLocalSize() const
{
	return GetCachedGeometry().GetLocalSize();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITOR
