// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingEditorWidgets.h"

#include "DMXPixelMappingUtils.h"
#include "DMXRuntimeUtils.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SScaleBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingScreenComponent"

void SDMXPixelMappingScreenLayout::Construct(const FArguments& InArgs)
{
	bShowAddresses = InArgs._bShowAddresses;
	bShowUniverse = InArgs._bShowUniverse;
	LocalUniverse = InArgs._LocalUniverse;
	StartAddress = InArgs._StartAddress;
	NumXCells = InArgs._NumXCells;
	NumYCells = InArgs._NumYCells;
	Distribution = InArgs._Distribution;
	PixelFormat = InArgs._PixelFormat;
	Brush = InArgs._Brush;

	const bool bShouldAddChannels = FDMXPixelMappingUtils::CanFitCellIntoChannels(PixelFormat, StartAddress);

	if (bShouldAddChannels)
	{
		const uint32 DMXCellStep = FDMXPixelMappingUtils::GetNumChannelsPerCell(PixelFormat);
		const uint32 UniverseMaxChannels = FDMXPixelMappingUtils::GetUniverseMaxChannels(PixelFormat, StartAddress);

	// Prepare unsorted list
	uint32 UniverseChannel = StartAddress;
	uint32 UniverseIndex = 0;
	const int32 TotalPixels = NumXCells * NumYCells;

		for (int32 CellID = 0; CellID < TotalPixels; ++CellID)
		{
			if (UniverseChannel + (DMXCellStep - 1) > DMX_MAX_ADDRESS)
			{
				UniverseChannel = StartAddress;
				UniverseIndex++;
			}

			TPair<int32, int32> AddressUniversePair;
			AddressUniversePair.Key = UniverseChannel - 1 % UniverseMaxChannels + 1;
			AddressUniversePair.Value = LocalUniverse + UniverseIndex;
			UnorderedList.Add(AddressUniversePair);

			UniverseChannel += DMXCellStep;
		}

		FDMXRuntimeUtils::PixelMappingDistributionSort<TPair<int32, int32>>(Distribution, NumXCells, NumYCells, UnorderedList, SortedList);
	}

	SAssignNew(GridPanel, SUniformGridPanel);

	uint32 XYIndex = 0;
	for (int32 YIndex = 0; YIndex < NumYCells; ++YIndex)
	{
	for (int32 XIndex = 0; XIndex < NumXCells; ++XIndex)
	{
			GridPanel->AddSlot(XIndex, YIndex)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SImage)
						.Image(Brush)
					]
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFit)
							[
								SNew(SBox)
								.Padding(FMargin(8.f, 4.f))
								[
									SNew(STextBlock)
									.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateLambda([this, XYIndex, bShouldAddChannels]()
									{ 
										if (bShouldAddChannels)
										{
											FString String;
											const TPair<int32, int32>& AddressUniversePair = SortedList[XYIndex];

											if (bShowUniverse)
											{
												String = FString::FromInt(AddressUniversePair.Value);

												if (bShowAddresses)
												{
													String += TEXT(" : ");
												}
											}
											if (bShowAddresses)
											{
												String += FString::FromInt(AddressUniversePair.Key);
											}

											return FText::FromString(String);
										}
										else
										{
											return FText::GetEmpty();
										}
									})))
								]
							]
						]
					]

				];

			XYIndex++;
		}
	}

	ChildSlot
	[
		GridPanel.ToSharedRef()
	];
}

void SDMXPixelMappingSimpleScreenLayout::Construct(const FArguments& InArgs)
{
	LocalUniverse = InArgs._LocalUniverse;
	StartAddress = InArgs._StartAddress;
	Brush = InArgs._Brush;

	NumXCells = InArgs._NumXCells;
	NumYCells = InArgs._NumYCells;

	ChildSlot
		[
			SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SImage)
						.Image(Brush)
					]
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFit)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Padding(FMargin(8.f, 4.f))
								[
									SNew(STextBlock)
									.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Text(FText::Format(LOCTEXT("Num_Pixels", "{0} x {1} pixels"), NumXCells, NumYCells))
								]
								+ SVerticalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Padding(FMargin(8.f, 4.f))
								[
									SNew(STextBlock)
									.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Text(FText::Format(LOCTEXT("UniverseAndAddress", "Universe: {0}, Start Address: {1}"), LocalUniverse, StartAddress))
								]
							]
						]
					]
				]
		];
}

void SDMXPixelMappingCell::Construct(const FArguments& InArgs)
{
	Brush = InArgs._Brush;

	CellID = InArgs._CellID;

	ChildSlot
		[
			SNew(SBox)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SImage)
						.Image(Brush)
					]
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SNew(SScaleBox)
							.Stretch(EStretch::ScaleToFit)
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Padding(FMargin(8.f, 4.f))
								[
									SNew(STextBlock)
									.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Text(FText::Format(LOCTEXT("CellID", "{0}"), CellID))
								]
							]
						]
					]
				]
		];
}


#undef LOCTEXT_NAMESPACE
