// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDMXPixelMappingScreenComponentBox.h"

#include "SDMXPixelMappingEditorWidgets.h"

#include "Widgets/Layout/SBox.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS // The whole class and several child widgets are deprecated 5.1
const uint32 SDMXPixelMappingScreenComponentBox::MaxGridUICells = 40 * 40;

void SDMXPixelMappingScreenComponentBox::Construct(const FArguments& InArgs)
{
	BorderBrush.DrawAs = ESlateBrushDrawType::Border;
	BorderBrush.TintColor = FLinearColor(1.f, 0.f, 1.f);
	BorderBrush.Margin = FMargin(1.f);

	if ((InArgs._NumXCells * InArgs._NumYCells) > MaxGridUICells)
	{
		ChildSlot
		[
			SAssignNew(ComponentBox, SBox)
			[
				SNew(SDMXPixelMappingSimpleScreenLayout)
				.NumXCells(InArgs._NumXCells)
				.NumYCells(InArgs._NumYCells)
				.Brush(&BorderBrush)
				.LocalUniverse(InArgs._LocalUniverse)
				.StartAddress(InArgs._StartAddress)
			]
		];
	}
	else
	{
		ChildSlot
		[
			SAssignNew(ComponentBox, SBox)
			[
				SNew(SDMXPixelMappingScreenLayout)
				.NumXCells(InArgs._NumXCells)
				.NumYCells(InArgs._NumYCells)
				.Distribution(InArgs._Distribution)
				.PixelFormat(InArgs._PixelFormat)
				.Brush(&BorderBrush)
				.LocalUniverse(InArgs._LocalUniverse)
				.StartAddress(InArgs._StartAddress)
				.bShowAddresses(InArgs._bShowAddresses)
				.bShowUniverse(InArgs._bShowUniverse)
			]
		];		
	}
}

void SDMXPixelMappingScreenComponentBox::SetLocalSize(const FVector2D& NewLocalSize)
{
	LocalSize = NewLocalSize;
	
	ComponentBox->SetWidthOverride(NewLocalSize.X);
	ComponentBox->SetHeightOverride(NewLocalSize.Y);
}

FVector2f SDMXPixelMappingScreenComponentBox::GetLocalSize() const
{
	return GetCachedGeometry().GetLocalSize();
}

void SDMXPixelMappingScreenComponentBox::RebuildGrid(const FDMXPixelMappingScreenComponentGridParams& GridParams)
{
	if ((GridParams.NumXCells * GridParams.NumYCells) > MaxGridUICells)
	{
		ComponentBox->SetContent(
			SNew(SDMXPixelMappingSimpleScreenLayout)
			.NumXCells(GridParams.NumXCells)
			.NumYCells(GridParams.NumYCells)
			.Brush(&BorderBrush)
			.LocalUniverse(GridParams.LocalUniverse)
			.StartAddress(GridParams.StartAddress)
		);
	}
	else
	{
		ComponentBox->SetContent(
			SNew(SDMXPixelMappingScreenLayout)
			.NumXCells(GridParams.NumXCells)
			.NumYCells(GridParams.NumYCells)
			.Distribution(GridParams.Distribution)
			.PixelFormat(GridParams.PixelFormat)
			.Brush(&BorderBrush)
			.LocalUniverse(GridParams.LocalUniverse)
			.StartAddress(GridParams.StartAddress)
			.bShowAddresses(GridParams.bShowAddresses)
			.bShowUniverse(GridParams.bShowUniverse)
		);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
