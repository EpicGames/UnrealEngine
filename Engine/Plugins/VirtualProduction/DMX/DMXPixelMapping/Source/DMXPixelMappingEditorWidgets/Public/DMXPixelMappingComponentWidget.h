// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SConstraintCanvas.h"

class SDMXPixelMappingComponentBox;
class SDMXPixelMappingComponentLabel;

class SConstraintCanvas;


/**
 * Since pixel mapping designer logic relies on computed sizes, two widgets are needed, one as label text, one as the actual box that needs its real size.
 * This widget internally holds both, both are guaranteed to be valid, and adopt sizes and positions reasonably together.
 */
class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") FDMXPixelMappingComponentWidget;
class DMXPIXELMAPPINGEDITORWIDGETS_API FDMXPixelMappingComponentWidget
	: public TSharedFromThis<FDMXPixelMappingComponentWidget>
{
public:
	/** Constructs the class, optionally with custom widgets */
	FDMXPixelMappingComponentWidget(TSharedPtr<SDMXPixelMappingComponentBox> InComponentBox = nullptr, TSharedPtr<SDMXPixelMappingComponentLabel> InComponentLabel = nullptr, bool bShowLabelAbove = false);

	/** Destructor */
	virtual ~FDMXPixelMappingComponentWidget();

	/** Adds the widgets to a canvas. If it already resides in a canvas it is removed from that. */
	void AddToCanvas(const TSharedRef<SConstraintCanvas>& Canvas, float ZOrder);

	/** Adds the widgets to a canvas. Needs to be assigned to a canvas already. */
	void RemoveFromCanvas();

	/** Sets the position in a canvas. Needs to be assigned to a canvas already. */
	void SetZOrder(float ZOrder);

	/** Sets the position in the canvas it was added to. */
	void SetPosition(const FVector2D& LocalPosition);

	/** Sets the size on both widgets */
	void SetSize(const FVector2D& LocalSize);

	/** Sets the size of both widgets */
	FVector2D GetLocalSize() const;

	/** Sets the label text */
	void SetLabelText(const FText& Text);

	/** Sets if the ID text of is visible */
	void SetIDVisibility(bool bVisible);

	/** Sets the color of the widget */
	void SetColor(const FLinearColor& Color);

	/** Retuns the color of the widget */
	FLinearColor GetColor() const;

	/** Sets the visibility of the widget */
	void SetVisibility(EVisibility Visibility);

	const TSharedPtr<SDMXPixelMappingComponentBox>& GetComponentBox() const { return ComponentBox; }
	const TSharedPtr<SDMXPixelMappingComponentLabel>& GetComponentLabel() const { return ComponentLabel; }

protected:
	/** When added to a canvas, the canvas it was added to */
	TSharedPtr<SConstraintCanvas> OuterCanvas;

	/** The canvas slot of the component widget */
	SConstraintCanvas::FSlot* ComponentSlot = nullptr;

	/** The canvas slot of the label widget */
	SConstraintCanvas::FSlot* LabelSlot = nullptr;

	/** The component widget */
	TSharedPtr<SDMXPixelMappingComponentBox> ComponentBox;

	/** The label widget */
	TSharedPtr<SDMXPixelMappingComponentLabel> ComponentLabel;

	/** If true, the label is drawn above the box, else inside */
	bool bLabelAbove = false;
};
