// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/SlateBrush.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"


class SBox;
class STextBlock;


/** A label component label in pixelmapping designer. Ment to be used with DMXPixelMapingComponentWidgetWrapper */
class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") SDMXPixelMappingComponentBox;
class DMXPIXELMAPPINGEDITORWIDGETS_API SDMXPixelMappingComponentBox
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingComponentBox)
		:_MaxIDTextSize(FVector2D(18.f, 18.f))
	{}
		/** Text shown in the middle of the widget. Always scales to the widget */
		SLATE_ATTRIBUTE(FText, IDText)

		/** The max size of the ID Text (the size of the Box that surrounds it) */
		SLATE_ARGUMENT(FVector2D, MaxIDTextSize)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Sets the size of the widget */
	virtual void SetLocalSize(const FVector2D& NewSize);

	/** Sets the IDText */
	virtual void SetIDText(const FText& NewIDText);

	/** Returns the local size */
	virtual FVector2f GetLocalSize() const;

	/** Sets the color of the border */
	virtual void SetBorderColor(const FLinearColor& Color);

	/** Retuns the color of the widget */
	virtual FLinearColor GetBorderColor() const;

protected:
	/** The currently drawn local size */
	FVector2D LocalSize = FVector2D(1.f, 1.f);

	/** The box that is shown */
	TSharedPtr<SBox> ComponentBox;

	/** The box that is shown */
	TSharedPtr<SBox> IDTextBox;

	/** The box that is shown */
	TSharedPtr<STextBlock> IDTextBlock;

	/** The brush used for the border */
	FSlateBrush BorderBrush;

	// Slate arguments
	TAttribute<FText> IDText;
	FVector2D MaxIDTextSize;
};