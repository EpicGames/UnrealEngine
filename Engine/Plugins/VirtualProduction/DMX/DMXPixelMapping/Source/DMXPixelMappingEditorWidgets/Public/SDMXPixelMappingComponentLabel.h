// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SBox;
class STextBlock;

/** A box in pixel mapping designer. Ment to be used with DMXPixelMapingComponentWidgetWrapper */
class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") SDMXPixelMappingComponentLabel;
class DMXPIXELMAPPINGEDITORWIDGETS_API SDMXPixelMappingComponentLabel
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingComponentLabel)
		: _bAlignAbove(false)
		, _bScaleToSize(true)
	{}

		/** Label shown on top of the widget */
		SLATE_ATTRIBUTE(FText, LabelText)

		/** If true, aligns above the box */
		SLATE_ARGUMENT(bool, bAlignAbove)

		/** If true, scales to its size */
		SLATE_ARGUMENT(bool, bScaleToSize)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Sets the width of the widget */
	virtual void SetWidth(const float& Width);

	/** Sets the label text */
	virtual void SetText(const FText& Text);

	/** Returns the local size */
	virtual FVector2f GetLocalSize() const;

protected:
	/** The box that defines the size */
	TSharedPtr<SBox> Box;

	/** Text box that shows the ID text */
	TSharedPtr<STextBlock> LabelTextBlock;

	// Slate arguments
	TAttribute<FText> LabelText;	
	bool bAlignAbove;
};
