// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateBrush.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Library/DMXEntityFixtureType.h"
#include "DMXPixelMappingTypes.h"

class STextBlock;
class SUniformGridPanel;

class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") SDMXPixelMappingScreenLayout;
class DMXPIXELMAPPINGEDITORWIDGETS_API SDMXPixelMappingScreenLayout 
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingScreenLayout) 
		: _NumXCells(1)
		, _NumYCells(1)
		, _Distribution(EDMXPixelMappingDistribution::TopLeftToRight)
		, _PixelFormat(EDMXCellFormat::PF_RGB)
		, _bShowAddresses(false)
		, _bShowUniverse(false)
		, _LocalUniverse(1)
		, _StartAddress(1)
	{}
		
		SLATE_ARGUMENT(int32, NumXCells)

		SLATE_ARGUMENT(int32, NumYCells)

		SLATE_ARGUMENT(EDMXPixelMappingDistribution, Distribution)

		SLATE_ARGUMENT(EDMXCellFormat, PixelFormat)
		
		SLATE_ARGUMENT(bool, bShowAddresses)

		SLATE_ARGUMENT(bool, bShowUniverse)

		SLATE_ARGUMENT(int32, LocalUniverse)

		SLATE_ARGUMENT(int32, StartAddress)

		SLATE_ATTRIBUTE(const FSlateBrush*, Brush)

	SLATE_END_ARGS()

	//~ Begin SCompoundWidget interface
	void Construct(const FArguments& InArgs);
	//~ Begin SCompoundWidget interface

private:
	int32 NumXCells;

	int32 NumYCells;

	EDMXPixelMappingDistribution Distribution;

	EDMXCellFormat PixelFormat;

	bool bShowAddresses;

	bool bShowUniverse;

	int32 LocalUniverse;

	int32 StartAddress;

	TAttribute<const FSlateBrush*> Brush;

	TSharedPtr<STextBlock> InfoTextBlock;

	TSharedPtr<SUniformGridPanel> GridPanel;

	TArray<TPair<int32, int32>> UnorderedList;

	TArray<TPair<int32, int32>> SortedList;
};

class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") SDMXPixelMappingSimpleScreenLayout;
class DMXPIXELMAPPINGEDITORWIDGETS_API SDMXPixelMappingSimpleScreenLayout
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingSimpleScreenLayout)
		: _NumXCells(1)
		, _NumYCells(1)
		, _LocalUniverse(1)
		, _StartAddress(1)
	{}

		SLATE_ARGUMENT(int32, NumXCells)

		SLATE_ARGUMENT(int32, NumYCells)

		SLATE_ARGUMENT(int32, LocalUniverse)

		SLATE_ARGUMENT(int32, StartAddress)

		SLATE_ATTRIBUTE(const FSlateBrush*, Brush)

	SLATE_END_ARGS()

	//~ Begin SCompoundWidget interface
	void Construct(const FArguments& InArgs);
	//~ Begin SCompoundWidget interface

private:
	int32 LocalUniverse;

	int32 StartAddress;

	int32 NumXCells;

	int32 NumYCells;

	TAttribute<const FSlateBrush*> Brush;
};

class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") SDMXPixelMappingCell;
class DMXPIXELMAPPINGEDITORWIDGETS_API SDMXPixelMappingCell
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingCell)
		: _CellID(1)
	{}

		SLATE_ARGUMENT(int32, CellID)

		SLATE_ATTRIBUTE(const FSlateBrush*, Brush)

	SLATE_END_ARGS()

	//~ Begin SCompoundWidget interface
	void Construct(const FArguments& InArgs);
	//~ Begin SCompoundWidget interface

public:
	TAttribute<const FSlateBrush*> Brush;

private:
	int32 CellID;
};

