// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SDMXPixelMappingComponentBox.h"

#include "DMXPixelMappingTypes.h"
#include "Library/DMXEntityFixtureType.h"

#include "CoreMinimal.h"

class SBox;


/** Params to update a SDMXPixelMappingScreenComponentBox */
struct UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") FDMXPixelMappingScreenComponentGridParams;
struct DMXPIXELMAPPINGEDITORWIDGETS_API FDMXPixelMappingScreenComponentGridParams
{
	int32 NumXCells = 1;
	int32 NumYCells = 1;
	EDMXPixelMappingDistribution Distribution = EDMXPixelMappingDistribution::TopLeftToRight;
	EDMXCellFormat PixelFormat = EDMXCellFormat::PF_RGB;
	bool bShowAddresses = false;
	bool bShowUniverse = false;
	int32 LocalUniverse = 1;
	int32 StartAddress = 1;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/** A label component label in pixelmapping designer. Ment to be used with DMXPixelMapingComponentWidgetWrapper */
class UE_DEPRECATED(5.1, "Pixel Mapping Editor Widgets are no longer supported and to be implemented per view. See SDMXPixelMappingOutputComponent for an example.") SDMXPixelMappingScreenComponentBox;
class DMXPIXELMAPPINGEDITORWIDGETS_API SDMXPixelMappingScreenComponentBox
	: public SDMXPixelMappingComponentBox
{
public:
	SLATE_BEGIN_ARGS(SDMXPixelMappingScreenComponentBox)
		: _NumXCells(1)
		, _NumYCells(1)
		, _Distribution(EDMXPixelMappingDistribution::TopLeftToRight)
		, _PixelFormat(EDMXCellFormat::PF_RGB)
		, _bShowAddresses(false)
		, _bShowUniverse(false)
		, _LocalUniverse(1)
		, _StartAddress(1)
	{}

		/** The size of the widget */
		SLATE_ATTRIBUTE(FVector2D, Size)

		/** The brush the border uses */
		SLATE_ATTRIBUTE(FLinearColor, Color)

		SLATE_ARGUMENT(int32, NumXCells)

		SLATE_ARGUMENT(int32, NumYCells)

		SLATE_ARGUMENT(EDMXPixelMappingDistribution, Distribution)

		SLATE_ARGUMENT(EDMXCellFormat, PixelFormat)

		SLATE_ARGUMENT(bool, bShowAddresses)

		SLATE_ARGUMENT(bool, bShowUniverse)

		SLATE_ARGUMENT(int32, LocalUniverse)

		SLATE_ARGUMENT(int32, StartAddress)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Sets the size of the widget */
	virtual void SetLocalSize(const FVector2D& NewSize) override;

	/** Returns the local size */
	virtual FVector2f GetLocalSize() const override;

	/** Rebuilds the grid */
	void RebuildGrid(const FDMXPixelMappingScreenComponentGridParams& GridParams);

private:
	/** The max grid cells shown before the widget shows as single box */
	static const uint32 MaxGridUICells;

	/** Can't set text, it's not in use in this class */
	using SDMXPixelMappingComponentBox::SetIDText;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
