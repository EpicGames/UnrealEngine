// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatform.h"
#include "Math/NumericLimits.h"
#include "Templates/Atomic.h"
#include "Templates/UniquePtr.h"


namespace UE::DMX::Internal
{
	/**************************************************
	 * 
	 * DMX Dev Note: 
	 * 
	 * Implementation and comments are an exact copy of Engine/Source/Runtime/Experimental/Chaos/Public/Framework/TripleBufferedData.h. 
	 * Copied Jan 2024 (~UE5.4), to avoid a dependency on experimental code.
	 * 
	 **************************************************/

	/**
	 * A lock free paradigm for sharing data between threads.
	 *
	 * Called a triple buffer because at any point in time, there may be 3 
	 * buffers available: 1 owned by the producing thread, 1 owned by the 
	 * consuming thread, and 1 waiting in an atomic variable. The third storage
	 * location enables transfer of ownership such that if you have a copy of 
	 * the data, you own it without worry of contention.
	 *
	 * Producer thread:
	 * \code
	 * struct AnimXf
	 * {
	 *     TArray<FTransform> Xf;
	 *     TArray<FVector> Velocity;
	 * };
	 *
	 * struct MyProducer
	 * {
	 *     MyConsumer Consumer; // Owns the triple buffer.
	 *     AnimXf* Buffer = nullptr;
	 *
	 *     // This function is called repeatedly at some interval by the producing thread.
	 *     void Produce()
	 *     {
	 *         // Get a new buffer if we need one.
	 *         if(!Buffer) 
	 *             Buffer = Consumer.AnimXfTripleBuffer.ExchangeProducerBuffer();
	 *         // This class now has exclusive ownership of the memory pointed to by Buffer.
	 *         Buffer->Xf = ...;
	 *         Buffer->Velocity = ...;
	 *         // Push the new values to the consumer, and get a new buffer for next time.
	 *         Buffer = Consumer.AnimXfTripleBuffer.ExchangeProducerBuffer();
	 *     }
	 * };
	 * \endcode
	 *
	 * Consumer thread:
	 * \code
	 * struct MyConsumer
	 * {
	 *     // In this example the consumer owns the triple buffer, but that's 
	 *     // not a requirement.
	 *     TDMXPixelMappingTripleBufferedData<AnimXf> AnimXfTripleBuffer;
	 *     AnimXf* Buffer = nullptr;
	 *
	 *     // This function is called repeatedly at some interval by the consuming thread.
	 *     void Consume()
	 *     {
	 *         // Get a new view of the data, which can be null or old.
	 *         Buffer = AnimXfTripleBuffer.ExchangeAnimXfConsumerBuffer();
	 *         // This class now has exclusive ownership of the memory pointed to by Buffer.
	 *         if(Buffer)
	 *         {
	 *             ... = Buffer->Xf;
	 *             ... = Buffer->Velocity;
	 *         }
	 *     }
	 * };
	 * \endcode
	 */
	template <class DataType>
	class TDMXPixelMappingTripleBufferedData
	{
	public:
		TDMXPixelMappingTripleBufferedData()
			: ProducerThreadBuffer(&Buffers[0])
			, ConsumerThreadBuffer(&Buffers[1])
			, Interchange(&Buffers[2])
			, Counter(0)
			, LastId(0)
		{}

		/**
		 * Get a new buffer for the producing thread to write to, while at
		 * the same time making the previous buffer available to the consumer.
		 *
		 * This function will manufacture new instances of \c DataType, if it 
		 * needs to.  Thus, the returned pointer will never be null.
		 *
		 * The \c DataType value is returned by shared pointer to simplify 
		 * ownership semantics, particularly with respect to cleanup.  It is 
		 * important to not mistake that for exclusive ownership of the memory,
		 * and retain the return value of this function too long.  For instance:
		 * \code
		 * TDMXPixelMappingTripleBufferedData<int> IntBuffer;
		 * ...
		 * int* MyCopy = IntBuffer.ExchangeProducerBuffer();
		 * (*MyCopy) = 1; // Ok
		 * IntBuffer.ExchangeProducerBuffer(); // Mem holding "1" now available to consumer thread.
		 * (*MyCopy) = 2; // Race condition!
		 * \endcode
		 *
		 * Instead, always reuse the same shared pointer, or dereference the 
		 * returned shared pointer if you're not worried about copying the data.
		 * \code
		 * TDMXPixelMappingTripleBufferedData<int> IntBuffer;
		 * ...
		 * int* MyCopy = IntBuffer.ExchangeProducerBuffer();
		 * (*MyCopy) = 1; // Ok
		 * MyCopy = IntBuffer.ExchangeProducerBuffer(); // Mem holding "1" now available to consumer thread.
		 * (*MyCopy) = 2; // Ok
		 * \endcode
		 */
		DataType* ExchangeProducerBuffer()
		{
			ProducerThreadBuffer = Interchange.Exchange(ProducerThreadBuffer);
			if (!ProducerThreadBuffer->Value)
			{
				ProducerThreadBuffer->Value = TUniquePtr<DataType>(new DataType());
			}
			ProducerThreadBuffer->Id = ++Counter; // First value is 1, will overflow.
			return ProducerThreadBuffer->Value.Get();
		}

		/**
		 * Get an updated buffer for the consuming thread to read from.
		 *
		 * The returned pointer may be null if the producer thread hasn't done 
		 * an update since the last time this function was called.
		 *
		 * The same capacity for a race condition exists with this functions
		 * return value as with \c ExchangeProducerBuffer().  
		 * \code
		 * TDMXPixelMappingTripleBufferedData<int> IntBuffer;
		 * ...
		 * int* MyCopy = IntBuffer.ExchangeConsumerBuffer();
		 * int X = *MyCopy; // Ok
		 * IntBuffer.ExchangeConsumerBuffer(); // Mem holding X returned to producer thread.
		 * int Y = *MyCopy; // Race condition!
		 * \endcode
		 *
		 * Always refresh the same variable.
		 * \code
		 * TDMXPixelMappingTripleBufferedData<int> IntBuffer;
		 * ...
		 * int* MyCopy = IntBuffer.ExchangeConsumerBuffer();
		 * int X = *MyCopy; // Ok
		 * MyCopy = IntBuffer.ExchangeConsumerBuffer(); // Mem holding X returned to producer thread.
		 * int Y = *MyCopy; // Ok
		 * \endcode
		 */
		DataType* ExchangeConsumerBuffer()
		{
			ConsumerThreadBuffer = Interchange.Exchange(ConsumerThreadBuffer);
			if (ConsumerThreadBuffer->Id <= LastId && 
				LastId - ConsumerThreadBuffer->Id < TNumericLimits<uint32>::Max()) // Detect overflows
			{
				return nullptr;
			}
			LastId = ConsumerThreadBuffer->Id;
			return ConsumerThreadBuffer->Value.Get();
		}

	private:
		// Non-copyable and non-movable, due to TAtomic member.
		TDMXPixelMappingTripleBufferedData(TDMXPixelMappingTripleBufferedData&& Other) = delete;
		TDMXPixelMappingTripleBufferedData(const TDMXPixelMappingTripleBufferedData&) = delete;
		TDMXPixelMappingTripleBufferedData& operator=(TDMXPixelMappingTripleBufferedData&&) = delete;
		TDMXPixelMappingTripleBufferedData& operator=(const TDMXPixelMappingTripleBufferedData&) = delete;

		struct DataTypeWrapper
		{
			DataTypeWrapper()
			  : Value(nullptr)
			  , Id(0)
			{}
			DataTypeWrapper(const DataTypeWrapper&) = delete;
			DataTypeWrapper(DataTypeWrapper&& Other)
				: Value(MoveTemp(Other.Value))
				, Id(MoveTemp(Other.Id))
			{}

			void operator=(const DataTypeWrapper& Other) = delete;

			// Depending on the use case, it may be useful to store DataType 
			// instances via shared pointer so that the thread that doesn't
			// own the TripleBufferedData instance, can retain ownership of 
			// this memory after this class goes out of scope.
			TUniquePtr<DataType> Value;
			uint64 Id;
		};

		DataTypeWrapper Buffers[3] = { DataTypeWrapper(), DataTypeWrapper(), DataTypeWrapper() };
		DataTypeWrapper* ProducerThreadBuffer;
		DataTypeWrapper* ConsumerThreadBuffer;
		TAtomic<DataTypeWrapper*> Interchange;
		uint64 Counter; // Only accessed by producer thread
		uint64 LastId; // Only accessed by consumer thread
	};
} // namespace UE::DMX::Internal
