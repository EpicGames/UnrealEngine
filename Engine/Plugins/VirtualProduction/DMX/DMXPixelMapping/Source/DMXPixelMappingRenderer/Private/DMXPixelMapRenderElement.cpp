// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRenderElement.h"


namespace UE::DMXPixelMapping::Rendering
{
	void FPixelMapRenderElement::SetParameters(FPixelMapRenderElementParameters InParameters)
	{
		const FScopeLock LockParameters(&AccessParametersMutex);

		Parameters = InParameters;
	}

	FPixelMapRenderElementParameters FPixelMapRenderElement::GetParameters() const
	{
		const FScopeLock LockParameters(&AccessParametersMutex);

		return Parameters;
	}

	void FPixelMapRenderElement::SetColor(const FLinearColor& InColor)
	{
		if (!ProducerColorPtr)
		{
			ProducerColorPtr = ColorTipleBuffer.ExchangeProducerBuffer();
		}

		// This call now has exclusive ownership of the memory pointed to by Buffer.
		*ProducerColorPtr = InColor;

		// Push the new values to the consumer, and get a new buffer for next time.
		ProducerColorPtr = ColorTipleBuffer.ExchangeProducerBuffer();
	}

	FLinearColor FPixelMapRenderElement::GetColor() const
	{
		// Get a new view of the data, which can be null or old.
		const FLinearColor* ConsumerColorPtr = ColorTipleBuffer.ExchangeConsumerBuffer();
		if (ConsumerColorPtr)
		{
			ColorGameThread = *ConsumerColorPtr;
		}
		
		return ColorGameThread;
	}
}
