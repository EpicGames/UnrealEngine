// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingApplyFilterMaterialProxy.h"

#include "CanvasTypes.h"
#include "CommonRenderResources.h"
#include "DMXStats.h"
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "Engine/TextureRenderTarget2D.h"
#include "GlobalShader.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialRenderProxy.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "RHI.h"
#include "RHIStaticStates.h"
#include "ScreenRendering.h"
#include "TextureResource.h"
#include "RendererInterface.h"

#define LOCTEXT_NAMESPACE "DMXPixelMappingApplyFilterMaterialProxy"

namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	DECLARE_CYCLE_STAT(TEXT("PixelMapping ApplyFilterMaterial"), STAT_PreprocessApplyFilterMaterial, STATGROUP_DMX);

	FDMXPixelMappingPreprocessMaterialCanvas::FDMXPixelMappingPreprocessMaterialCanvas()
	{
		Canvas = NewObject<UCanvas>();
	}

	void FDMXPixelMappingPreprocessMaterialCanvas::DrawMaterialToRenderTarget(UTextureRenderTarget2D* TextureRenderTarget, UMaterialInterface* Material)
	{
		if (!Material || !TextureRenderTarget || !TextureRenderTarget->GetResource())
		{
			return;
		}

		// This is a user-facing function, so we'd rather make sure that shaders are ready by the time we render, in order to ensure we don't draw with a fallback material
		Material->EnsureIsComplete();
		FTextureRenderTargetResource* RenderTargetResource = TextureRenderTarget->GameThread_GetRenderTargetResource();
		if (!RenderTargetResource)
		{
			return;
		}

		FCanvas RenderCanvas(
			RenderTargetResource,
			nullptr,
			GEngine->GetWorld(),
			GMaxRHIFeatureLevel);

		Canvas->Init(TextureRenderTarget->SizeX, TextureRenderTarget->SizeY, nullptr, &RenderCanvas);
		{
			ENQUEUE_RENDER_COMMAND(FlushDeferredResourceUpdateCommand)(
				[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
				{
					RenderTargetResource->FlushDeferredResourceUpdate(RHICmdList);
				});

			Canvas->K2_DrawMaterial(Material, FVector2D(0, 0), FVector2D(TextureRenderTarget->SizeX, TextureRenderTarget->SizeY), FVector2D(0, 0));

			RenderCanvas.Flush_GameThread();
			Canvas->Canvas = nullptr;

			// UpdateResourceImmediate must be called here to ensure mips are generated.
			TextureRenderTarget->UpdateResourceImmediate(false);

			ENQUEUE_RENDER_COMMAND(ResetSceneTextureExtentHistory)(
				[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
				{
					RenderTargetResource->ResetSceneTextureExtentsHistory();
				});
		}
	}

	void FDMXPixelMappingPreprocessMaterialCanvas::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(Canvas);
	}

	FDMXPixelMappingApplyFilterMaterialProxy::FDMXPixelMappingApplyFilterMaterialProxy(EPixelFormat InFormat)
		: Format(InFormat)
	{}

	void FDMXPixelMappingApplyFilterMaterialProxy::Render(UTexture* InInputTexture, const UDMXPixelMappingPreprocessRenderer& InPreprocessRenderer)
	{
		SCOPE_CYCLE_COUNTER(STAT_PreprocessApplyFilterMaterial);

		MaterialInstanceDynamic = InPreprocessRenderer.GetFilterMID();

		WeakInputTexture = InInputTexture;
		UpdateRenderTargets(InPreprocessRenderer.GetNumDownsamplePasses(), InPreprocessRenderer.GetDesiredOutputSize2D());

		if (!CanRender())
		{
			return;
		}

		UTexture* InputTexture = WeakInputTexture.Get();
		check(InputTexture);
		if (MaterialInstanceDynamic && DownsampleRenderTargets.IsEmpty() && ensureMsgf(OutputRenderTarget, TEXT("Cannot apply pixel mapping filter material. Missing output render target.")))
		{
			// Render without downsampling
			MaterialInstanceDynamic->SetTextureParameterValue(InPreprocessRenderer.GetTextureParameterName(), InputTexture);
			MaterialInstanceDynamic->SetScalarParameterValue(InPreprocessRenderer.GetBlurDistanceParameterName(), InPreprocessRenderer.GetBlurDistance());
			DrawMaterialCanvas.DrawMaterialToRenderTarget(OutputRenderTarget, MaterialInstanceDynamic);
		}
		else
		{
			const bool bApplyFilterMaterialEachDownsamplePass = InPreprocessRenderer.ShouldApplyFilterMaterialEachDownsamplePass();

			// Render with downsampling. 
			// Note, if the last downsample render target is of minimal size, further renderer targets are not created.
			// Hence use the number of downsample render targets instead of InParams.NumDownsamplePasses.
			const int32 NumDownsamplePasses = DownsampleRenderTargets.Num();
			for (int32 DownsamplePass = 0; DownsamplePass < NumDownsamplePasses; DownsamplePass++)
			{
				UTexture* SourceTexture = DownsamplePass == 0 ? InputTexture : DownsampleRenderTargets[DownsamplePass - 1];
				UTextureRenderTarget2D* DownsampleRenderTarget = DownsampleRenderTargets[DownsamplePass].Get();

				const bool bApplyFilterMaterial = MaterialInstanceDynamic && (bApplyFilterMaterialEachDownsamplePass || DownsampleRenderTarget == DownsampleRenderTargets.Last());
				if (bApplyFilterMaterial)
				{
					MaterialInstanceDynamic->SetTextureParameterValue(InPreprocessRenderer.GetTextureParameterName(), SourceTexture);
					MaterialInstanceDynamic->SetScalarParameterValue(InPreprocessRenderer.GetBlurDistanceParameterName(), InPreprocessRenderer.GetBlurDistance());
					DrawMaterialCanvas.DrawMaterialToRenderTarget(DownsampleRenderTarget, MaterialInstanceDynamic);
				}
				else
				{
					RenderTextureToTarget(SourceTexture, DownsampleRenderTarget);
				}
			}

			// Scale to output size
			UTexture* LastSourceTexture = DownsampleRenderTargets.IsEmpty() ? InputTexture : DownsampleRenderTargets.Last();
			RenderTextureToTarget(LastSourceTexture, OutputRenderTarget);
		}
	}

	UTexture* FDMXPixelMappingApplyFilterMaterialProxy::GetRenderedTexture() const
	{
		return CanRender() ? OutputRenderTarget : WeakInputTexture.Get();
	}

	void FDMXPixelMappingApplyFilterMaterialProxy::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(MaterialInstanceDynamic);
		Collector.AddReferencedObjects(DownsampleRenderTargets);
		Collector.AddReferencedObject(OutputRenderTarget);
	}

	bool FDMXPixelMappingApplyFilterMaterialProxy::CanRender() const
	{
		return WeakInputTexture.IsValid() && OutputRenderTarget && FApp::CanEverRender();
	}

	void FDMXPixelMappingApplyFilterMaterialProxy::UpdateRenderTargets(int32 NumDownsamplePasses, const TOptional<FVector2D>& OptionalOutputSize)
	{
		if (!WeakInputTexture.IsValid())
		{
			OutputRenderTarget = nullptr;
			return;
		}
		UTexture* InputTexture = WeakInputTexture.Get();
		check(InputTexture);
		
		const bool bResize = 
			OptionalOutputSize.IsSet() &&
			InputTexture->GetSurfaceWidth() != OptionalOutputSize.GetValue().X &&
			InputTexture->GetSurfaceHeight() != OptionalOutputSize.GetValue().Y;

		const bool bNeedsAnyRenderTargets = NumDownsamplePasses > 0 || MaterialInstanceDynamic || bResize;
		if (!bNeedsAnyRenderTargets)
		{
			OutputRenderTarget = nullptr;
			return;
		}

		const int32 NumExpectedDownsampleRenderTargets = OptionalOutputSize.IsSet() ? NumDownsamplePasses : NumDownsamplePasses - 1;
		const bool bInvalidRenderTargets = 
			!OutputRenderTarget || 
			NumExpectedDownsampleRenderTargets != DownsampleRenderTargets.Num() ||
			(OptionalOutputSize.IsSet() && OutputRenderTarget->GetSurfaceWidth() != OptionalOutputSize.GetValue().X) ||
			(OptionalOutputSize.IsSet() && OutputRenderTarget->GetSurfaceHeight() != OptionalOutputSize.GetValue().Y);

		if (bInvalidRenderTargets)
		{
			FlushRenderingCommands();
			
			const FVector2D InputSize = FVector2D(InputTexture->GetSurfaceWidth(), InputTexture->GetSurfaceHeight());

			const float NumSteps = NumDownsamplePasses;
			FScopedSlowTask Task(NumSteps, LOCTEXT("UpdateRenderTargets", "Updating Render Targets..."));
			Task.MakeDialogDelayed(.5f);

			// Create downsample render targets
			DownsampleRenderTargets.Reset();
			FVector2D DownsampleSize = InputSize;
			for (int32 DownsamplePass = 0; DownsamplePass < NumDownsamplePasses; DownsamplePass++)
			{
				Task.EnterProgressFrame();

				DownsampleSize /= 2.0;
				if (DownsampleSize.X <= 1.0 || DownsampleSize.Y <= 1.0)
				{
					break;
				}

				constexpr bool bForceLinearGamma = false;

				UTextureRenderTarget2D* DownsampleRenderTarget = NewObject<UTextureRenderTarget2D>();
				DownsampleRenderTarget->ClearColor = FLinearColor::Black;
				DownsampleRenderTarget->InitCustomFormat(DownsampleSize.X, DownsampleSize.Y, Format, bForceLinearGamma);
				DownsampleRenderTarget->UpdateResourceImmediate();
				DownsampleRenderTargets.Add(DownsampleRenderTarget);
			}

			UTextureRenderTarget2D* ScaleRenderTarget = nullptr;
			if (OptionalOutputSize.IsSet())
			{
				constexpr bool bForceLinearGamma = false;

				ScaleRenderTarget = NewObject<UTextureRenderTarget2D>();
				ScaleRenderTarget->ClearColor = FLinearColor::Black;
				ScaleRenderTarget->InitCustomFormat(OptionalOutputSize.GetValue().X, OptionalOutputSize.GetValue().Y, Format, bForceLinearGamma);
				ScaleRenderTarget->UpdateResourceImmediate();
			}

			if (ScaleRenderTarget)
			{
				OutputRenderTarget = ScaleRenderTarget;
			}
			else if (!DownsampleRenderTargets.IsEmpty())
			{
				OutputRenderTarget = DownsampleRenderTargets.Pop();
			}
			else
			{
				OutputRenderTarget = nullptr;
			}
		}
	}

	void FDMXPixelMappingApplyFilterMaterialProxy::RenderTextureToTarget(UTexture* Texture, UTextureRenderTarget2D* RenderTarget) const
	{
		IRendererModule* RendererModule = FModuleManager::GetModulePtr<IRendererModule>("Renderer");
		ENQUEUE_RENDER_COMMAND(PixelMappingPreprocessDownsamplePass)(
			[RendererModule, Texture, RenderTarget, this](FRHICommandListImmediate& RHICmdList)
			{
				if (!Texture || !Texture->GetResource())
				{
					return;
				}

				const FTextureRHIRef SourceTextureRHI = Texture->GetResource()->GetTexture2DRHI();
				const FTextureRHIRef TargetTextureRHI = RenderTarget->GetResource()->GetTexture2DRHI();

				FRHIRenderPassInfo RPInfo(TargetTextureRHI, MakeRenderTargetActions(ERenderTargetLoadAction::ENoAction, ERenderTargetStoreAction::EStore));

				RHICmdList.Transition(FRHITransitionInfo(TargetTextureRHI, ERHIAccess::Unknown, ERHIAccess::RTV));
				RHICmdList.BeginRenderPass(RPInfo, TEXT("PixelMappingFilterProxy_RenderToTarget"));
				{
					RHICmdList.SetViewport(0, 0, 0.0f, (float)TargetTextureRHI->GetSizeX(), (float)TargetTextureRHI->GetSizeY(), 1.0f);

					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

					FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
					TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
					TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;

					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Bilinear>::GetRHI(), SourceTextureRHI);

					RendererModule->DrawRectangle(RHICmdList,
						0, 0,									// Dest X, Y
						(float)TargetTextureRHI->GetSizeX(),	// Dest Width
						(float)TargetTextureRHI->GetSizeY(),	// Dest Height
						0, 0,									// Source U, V
						1, 1,									// Source USize, VSize
						TargetTextureRHI->GetSizeXY(),			// Target buffer size
						FIntPoint(1, 1),						// Source texture size
						VertexShader,
						EDRF_Default);
				}
				RHICmdList.EndRenderPass();
			});
	}
}

#undef LOCTEXT_NAMESPACE
