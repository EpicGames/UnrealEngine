// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingPreprocessRenderer.h"

#include "UObject/GCObject.h"
#include "UObject/ObjectPtr.h"

class UCanvas;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;
class UTextureRenderTarget2D;


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	/** Canvas implementation for pixelmapping */
	class FDMXPixelMappingPreprocessMaterialCanvas
		: public FGCObject
	{
	public:
		/** Constructor */
		FDMXPixelMappingPreprocessMaterialCanvas();

		/** Draws a material to a render target */
		void DrawMaterialToRenderTarget(UTextureRenderTarget2D* TextureRenderTarget, UMaterialInterface* Material);

	protected:
		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("DMXPixelMapping::Rendering::FDMXPixelMappingPreprocessMaterialCanvas");
		}
		//~ End FGCObject interface

	private:
		/** Canvas used to draw the material */
		TObjectPtr<UCanvas> Canvas;
	};

	/** Proxy to filter the texture used in Pixel Mapping */
	class FDMXPixelMappingApplyFilterMaterialProxy
		: public IPreprocessApplyFilterMaterialProxy
		, public FGCObject
	{
	public:
		FDMXPixelMappingApplyFilterMaterialProxy(EPixelFormat InFormat);

		//~ Begin IPreprocessApplyFilterMaterialProxy interface
		virtual void Render(UTexture* InInputTexture, const UDMXPixelMappingPreprocessRenderer& InPreprocessRenderer) override;
		virtual UTexture* GetRenderedTexture() const override;
		//~ End IPreprocessApplyFilterMaterialProxy interface

	protected:
		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("FDMXPixelMappingApplyFilterMaterialProxy");
		}
		//~ End FGCObject interface

	private:
		/** Returns true if rendering is needed */
		bool CanRender() const;

		/** Updates render targets */
		void UpdateRenderTargets(int32 NumDownsamplePasses, const TOptional<FVector2D>& OptionalOutputSize);

		/** Renders the Input Texture to the Output render target */
		void RenderTextureToTarget(UTexture* Texture, UTextureRenderTarget2D* RenderTarget) const;

		/** The pixel format his proxy should use */
		EPixelFormat Format = PF_Unknown;

		/** Weak ref to the input texture */
		TWeakObjectPtr<UTexture> WeakInputTexture;

		/** The dynamic material instance to be applied to the input texture */
		TObjectPtr<UMaterialInstanceDynamic> MaterialInstanceDynamic;

		/** Downsample render targets */
		TArray<TObjectPtr<UTextureRenderTarget2D>> DownsampleRenderTargets;

		/** The target to which the output is rendered. Either the last downscale render target or the scale render target */
		TObjectPtr<UTextureRenderTarget2D> OutputRenderTarget;

		/** Canvas used to draw materials */
		FDMXPixelMappingPreprocessMaterialCanvas DrawMaterialCanvas;
	};
}
