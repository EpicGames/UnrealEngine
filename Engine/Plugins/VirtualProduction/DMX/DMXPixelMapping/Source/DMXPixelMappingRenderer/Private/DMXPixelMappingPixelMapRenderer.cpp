// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingPixelMapRenderer.h"

#include "DMXPixelMappingRenderElement.h"
#include "DMXPixelMappingRenderPixelMapProxy.h"
#include "Engine/TextureRenderTarget2D.h"
#include "RenderingThread.h"


const FIntPoint UDMXPixelMappingPixelMapRenderer::MaxPixelMapSize = FIntPoint(4096, 4096);

UDMXPixelMappingPixelMapRenderer::UDMXPixelMappingPixelMapRenderer()
{
	using namespace UE::DMXPixelMapping::Rendering::Private;
	RendererPixelMapProxy = MakeShared<FDMXPixelMappingRenderPixelMapProxy>();
}

void UDMXPixelMappingPixelMapRenderer::SetElements(const TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>>& InElements, EPixelFormat InFormat)
{
	check(IsInGameThread());

	if (RenderElements == InElements)
	{
		return;
	}

	if (InElements.IsEmpty())
	{
		PixelMapRenderTarget = nullptr;
		return;
	}

	if (!ensureMsgf(InElements.Num() < MaxPixelMapSize.X * MaxPixelMapSize.Y, TEXT("PixelMapping contains more pixels than supported by engine.")))
	{
		return;
	}

	const FIntPoint DesiredSize
	{
		FMath::Min(InElements.Num(), MaxPixelMapSize.X),
		InElements.Num() / MaxPixelMapSize.X + 1
	};
	const FVector2D ValidSize
	{
		FMath::Max(1.0, DesiredSize.X),
		FMath::Max(1.0, DesiredSize.Y)
	};

	// Create a new render target if needed
	constexpr bool bForceLinearGamma = false;
	if (!PixelMapRenderTarget)
	{
		const FName TargetName = MakeUniqueObjectName(this, UTextureRenderTarget2D::StaticClass(), TEXT("PixelMapRenderTarget"));
		PixelMapRenderTarget = NewObject<UTextureRenderTarget2D>(this, TargetName);
		PixelMapRenderTarget->ClearColor = FLinearColor::Black;

		const FVector2D InitialSize{ 1.0, 1.0 };
		PixelMapRenderTarget->InitCustomFormat(ValidSize.X, ValidSize.Y, InFormat, bForceLinearGamma);
	}
	else if (PixelMapRenderTarget->GetFormat() != InFormat)
	{
		PixelMapRenderTarget->InitCustomFormat(ValidSize.X, ValidSize.Y, InFormat, bForceLinearGamma);
	}
	else if (ValidSize.X != PixelMapRenderTarget->GetSurfaceWidth() || ValidSize.Y != PixelMapRenderTarget->GetSurfaceHeight())
	{
		PixelMapRenderTarget->ResizeTarget(ValidSize.X, ValidSize.Y);
	}

	RenderElements = InElements;
}

void UDMXPixelMappingPixelMapRenderer::SetElements(const TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>>& InElements)
{
	// DERPECATED 5.4 - Use EPixelFormat::PF_B8G8R8A8 as it was used up to 5.4
	SetElements(InElements, EPixelFormat::PF_B8G8R8A8);
}

void UDMXPixelMappingPixelMapRenderer::Render(UTexture* InputTexture, float Brightness)
{
	RendererPixelMapProxy->Render(InputTexture, PixelMapRenderTarget, RenderElements, Brightness);
}
