// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingPreprocessRenderer.h"

#include "DMXPixelMappingApplyFilterMaterialProxy.h"
#include "DMXPixelMappingRenderInputMaterialProxy.h"
#include "DMXPixelMappingRenderInputTextureProxy.h"
#include "DMXPixelMappingRenderInputUserWidgetProxy.h"
#include "DMXStats.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/Package.h"


DECLARE_CYCLE_STAT(TEXT("PixelMapping PreprocessInputTexture"), STAT_DMXPixelMappingPreprocessInputTexture, STATGROUP_DMX);

void UDMXPixelMappingPreprocessRenderer::SetInputTexture(UTexture* InTexture, EPixelFormat InFormat)
{
	using namespace UE::DMXPixelMapping::Rendering::Preprocess::Private;
	RenderInputProxy = MakeShared<FDMXPixelMappingRenderInputTextureProxy>(InTexture);
	ApplyMaterialProxy = MakeShared<FDMXPixelMappingApplyFilterMaterialProxy>(InFormat);

	bShowInputSize = false;
}

void UDMXPixelMappingPreprocessRenderer::SetInputMaterial(UMaterialInterface* InMaterial, EPixelFormat InFormat)
{
	using namespace UE::DMXPixelMapping::Rendering::Preprocess::Private;
	RenderInputProxy = MakeShared<FDMXPixelMappingRenderInputMaterialProxy>(InMaterial, InputSize, InFormat);
	ApplyMaterialProxy = MakeShared<FDMXPixelMappingApplyFilterMaterialProxy>(InFormat);

	bShowInputSize = true;
}

void UDMXPixelMappingPreprocessRenderer::SetInputUserWidget(UUserWidget* InUserWidget, EPixelFormat InFormat)
{
	using namespace UE::DMXPixelMapping::Rendering::Preprocess::Private;
	RenderInputProxy = MakeShared<FDMXPixelMappingRenderInputUserWidgetProxy>(InUserWidget, InputSize, InFormat);
	ApplyMaterialProxy = MakeShared<FDMXPixelMappingApplyFilterMaterialProxy>(InFormat);

	bShowInputSize = true;
}

void UDMXPixelMappingPreprocessRenderer::SetInputTexture(UTexture* InTexture)
{
	// DERPECATED 5.4 - Use EPixelFormat::PF_B8G8R8A8 as it was used up to 5.4
	SetInputTexture(InTexture, EPixelFormat::PF_B8G8R8A8);
}

void UDMXPixelMappingPreprocessRenderer::SetInputMaterial(UMaterialInterface* InMaterial)
{
	// DERPECATED 5.4 - Use EPixelFormat::PF_B8G8R8A8 as it was used up to 5.4
	SetInputMaterial(InMaterial, EPixelFormat::PF_B8G8R8A8);
}

void UDMXPixelMappingPreprocessRenderer::SetInputUserWidget(UUserWidget* InUserWidget)
{
	// DERPECATED 5.4 - Use EPixelFormat::PF_B8G8R8A8 as it was used up to 5.4
	SetInputUserWidget(InUserWidget, EPixelFormat::PF_B8G8R8A8);
}

void UDMXPixelMappingPreprocessRenderer::ClearInput()
{
	RenderInputProxy.Reset();
	ApplyMaterialProxy.Reset();
}

void UDMXPixelMappingPreprocessRenderer::Render()
{
	SCOPE_CYCLE_COUNTER(STAT_DMXPixelMappingPreprocessInputTexture);

	if (RenderInputProxy.IsValid())
	{
		RenderInputProxy->Render();
		ApplyMaterialProxy->Render(RenderInputProxy->GetRenderedTexture(), *this);
	}
}

UTexture* UDMXPixelMappingPreprocessRenderer::GetRenderedTexture() const
{
	return ApplyMaterialProxy->GetRenderedTexture();
}

TOptional<FVector2D> UDMXPixelMappingPreprocessRenderer::GetDesiredOutputSize2D() const
{
	if (OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::SameAsInput)
	{
		return RenderInputProxy->GetSize2D();
	}
	else if (OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::Downsampled)
	{
		return TOptional<FVector2D>();
	}
	else if (OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::CustomSize)
	{
		return TOptional<FVector2D>(CustomOutputSize);
	}

	return TOptional<FVector2D>();
}

FVector2D UDMXPixelMappingPreprocessRenderer::GetResultingSize2D() const
{
	if (OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::SameAsInput)
	{
		return RenderInputProxy->GetSize2D();
	}
	else if (OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::Downsampled)
	{
		const FVector2D DownsampledSize = RenderInputProxy->GetSize2D() / FMath::Pow(2.0, NumDownSamplePasses);
		return FVector2D(FMath::Max(1.0, DownsampledSize.X), FMath::Max(1.0, DownsampledSize.Y));
	}
	else if (OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::CustomSize)
	{
		return CustomOutputSize;
	}

	checkf(0, TEXT("Unhandled output size mode in DMXPixelMappingRenderInputTextureProxy"));
	return FVector2D::ZeroVector;
}

void UDMXPixelMappingPreprocessRenderer::PostLoad()
{
	Super::PostLoad();

	if (FilterMaterial)
	{
		UPackage* TransientPackage = GetTransientPackage();
		FilterMID = FilterMaterial ? UMaterialInstanceDynamic::Create(FilterMaterial, TransientPackage) : nullptr;
	}
}

#if WITH_EDITOR
void UDMXPixelMappingPreprocessRenderer::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingPreprocessRenderer, FilterMaterial))
	{
		if (FilterMaterial)
		{
			UPackage* TransientPackage = GetTransientPackage();
			FilterMID = FilterMaterial ? UMaterialInstanceDynamic::Create(FilterMaterial, TransientPackage) : nullptr;
		}
		else
		{
			FilterMID = nullptr;
		}
	}
}
#endif // WITH_EDITOR
