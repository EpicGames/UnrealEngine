// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRenderInputMaterialProxy.h"

#include "DMXPixelMappingPreprocessRenderer.h"
#include "DMXStats.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "SlateMaterialBrush.h"
#include "Slate/WidgetRenderer.h"
#include "Widgets/Images/SImage.h"


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	DECLARE_CYCLE_STAT(TEXT("PixelMapping RenderInputMaterial"), STAT_DMXPixelMappingPreprocessRenderInputMaterial, STATGROUP_DMX);

	FDMXPixelMappingRenderInputMaterialProxy::FDMXPixelMappingRenderInputMaterialProxy(UMaterialInterface* InMaterial, const FVector2D& InInputSize, EPixelFormat InFormat)
		: WeakMaterial(InMaterial)
	{
		if (InMaterial)
		{
			InMaterial->EnsureIsComplete();
		}

		constexpr bool bForceLinearGamma = false;

		IntermediateRenderTarget = NewObject<UTextureRenderTarget2D>();
		IntermediateRenderTarget->ClearColor = FLinearColor::Black;
		IntermediateRenderTarget->InitCustomFormat(InInputSize.X, InInputSize.Y, InFormat, bForceLinearGamma);
		IntermediateRenderTarget->UpdateResourceImmediate();

		UIMaterialBrush = MakeShared<FSlateMaterialBrush>(FVector2D(1.f));

		const bool bUseGammaCorrection = false;
		MaterialRenderer = MakeShared<FWidgetRenderer>(bUseGammaCorrection);
	}

	void FDMXPixelMappingRenderInputMaterialProxy::Render()
	{
		SCOPE_CYCLE_COUNTER(STAT_DMXPixelMappingPreprocessRenderInputMaterial);

		UMaterial* Material = WeakMaterial.IsValid() ? WeakMaterial->GetMaterial() : nullptr;
		if (Material && Material->IsUIMaterial())
		{
			check(IntermediateRenderTarget);
			FVector2D TextureSize = FVector2D(IntermediateRenderTarget->SizeX, IntermediateRenderTarget->SizeY);

			UIMaterialBrush->ImageSize = TextureSize;
			UIMaterialBrush->SetMaterial(Material);

			TSharedRef<SWidget> Widget =
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(UIMaterialBrush.Get())
				];

			static const float DeltaTime = 0.f; 
			MaterialRenderer->DrawWidget(IntermediateRenderTarget, Widget, TextureSize, DeltaTime);

			// Reset material after drawing
			UIMaterialBrush->SetMaterial(nullptr);
		}
	}

	UTexture* FDMXPixelMappingRenderInputMaterialProxy::GetRenderedTexture() const
	{
		return IntermediateRenderTarget;
	}

	FVector2D FDMXPixelMappingRenderInputMaterialProxy::GetSize2D() const
	{
		check(IntermediateRenderTarget);
		return FVector2D(IntermediateRenderTarget->GetSurfaceWidth(), IntermediateRenderTarget->GetSurfaceHeight());
	}

	void FDMXPixelMappingRenderInputMaterialProxy::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(IntermediateRenderTarget);
	}
}
