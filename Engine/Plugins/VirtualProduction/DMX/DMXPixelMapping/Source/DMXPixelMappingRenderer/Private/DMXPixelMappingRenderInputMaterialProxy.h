// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingPreprocessRenderer.h"

#include "UObject/GCObject.h"

struct FSlateMaterialBrush;
class FWidgetRenderer;
class UMaterialInterface;
class UTextureRenderTarget2D;


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	/** Proxy responsible for the input material */
	class FDMXPixelMappingRenderInputMaterialProxy
		: public IPreprocessRenderInputProxy
		, public FGCObject
	{
	public:
		FDMXPixelMappingRenderInputMaterialProxy(UMaterialInterface* InMaterial, const FVector2D& InInputSize, EPixelFormat Format);
		
		//~ Begin IPreprocessRenderInputProxy interface
		virtual void Render() override;
		virtual UTexture* GetRenderedTexture() const override;
		virtual FVector2D GetSize2D() const override;
		//~ End IPreprocessRenderInputProxy interface

	protected:
		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("DMXPixelMapping::Rendering::Private::FDMXPixelMappingRenderInputMaterialProxy");
		}
		//~ End FGCObject interface

	private:
		/** Material renderer */
		TSharedPtr<FWidgetRenderer> MaterialRenderer;

		/** Brush for Material widget renderer */
		TSharedPtr<FSlateMaterialBrush> UIMaterialBrush;

		/** The Input object */
		TWeakObjectPtr<UMaterialInterface> WeakMaterial;

		/** Intermediate render target for materials and user widgets */
		TObjectPtr<UTextureRenderTarget2D> IntermediateRenderTarget;
	};
}
