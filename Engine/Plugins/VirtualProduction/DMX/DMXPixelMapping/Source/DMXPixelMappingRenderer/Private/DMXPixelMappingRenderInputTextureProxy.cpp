// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRenderInputTextureProxy.h"

#include "Engine/Texture.h"


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	FDMXPixelMappingRenderInputTextureProxy::FDMXPixelMappingRenderInputTextureProxy(UTexture* InTexture)
		:  WeakInputTexture(InTexture)
	{
		if (InTexture)
		{
			InTexture->WaitForStreaming();
		}
	}

	void FDMXPixelMappingRenderInputTextureProxy::Render()
	{
		// No rendering required
	}

	UTexture* FDMXPixelMappingRenderInputTextureProxy::GetRenderedTexture() const
	{
		return WeakInputTexture.Get();
	}

	FVector2D FDMXPixelMappingRenderInputTextureProxy::GetSize2D() const
	{
		return WeakInputTexture.IsValid() ?
			FVector2D(WeakInputTexture->GetSurfaceWidth(), WeakInputTexture->GetSurfaceHeight()) :
			FVector2D{ 0.0, 0.0 };
	}
}
