// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingPreprocessRenderer.h"

#include "UObject/WeakObjectPtr.h"

class UDMXPixelMappingPreprocessRenderer;
class UTexture;


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	/** Proxy responsible for the input texture */
	class FDMXPixelMappingRenderInputTextureProxy
		: public IPreprocessRenderInputProxy
	{
	public:
		FDMXPixelMappingRenderInputTextureProxy(UTexture* InTexture);

		//~ Begin IPreprocessRenderInputProxy interface
		virtual void Render() override;
		virtual UTexture* GetRenderedTexture() const override;
		virtual FVector2D GetSize2D() const override;
		//~ End IPreprocessRenderInputProxy interface

	private:
		/** The Input object */
		TWeakObjectPtr<UTexture> WeakInputTexture;
	};
}
