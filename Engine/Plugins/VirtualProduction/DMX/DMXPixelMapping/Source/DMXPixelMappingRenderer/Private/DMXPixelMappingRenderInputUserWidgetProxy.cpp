// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRenderInputUserWidgetProxy.h"

#include "Blueprint/UserWidget.h"
#include "DMXStats.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Framework/Application/SlateApplication.h"
#include "Slate/WidgetRenderer.h"


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	DECLARE_CYCLE_STAT(TEXT("PixelMapping RenderInputUserWidget"), STAT_PreprocessRenderInputUserWidget, STATGROUP_DMX);

	FDMXPixelMappingRenderInputUserWidgetProxy::FDMXPixelMappingRenderInputUserWidgetProxy(UUserWidget* InUserWidget, const FVector2D& InInputSize, EPixelFormat InFormat)
		: WeakUserWidget(InUserWidget)
	{
		constexpr bool bForceLinearGamma = false;

		IntermediateRenderTarget = NewObject<UTextureRenderTarget2D>();
		IntermediateRenderTarget->ClearColor = FLinearColor::Black;
		IntermediateRenderTarget->InitCustomFormat(InInputSize.X, InInputSize.Y, InFormat, bForceLinearGamma);
		IntermediateRenderTarget->UpdateResourceImmediate();

		const bool bUseGammaCorrection = false;
		UMGRenderer = MakeShared<FWidgetRenderer>(bUseGammaCorrection);
	}

	void FDMXPixelMappingRenderInputUserWidgetProxy::Render()
	{
		SCOPE_CYCLE_COUNTER(STAT_PreprocessRenderInputUserWidget);

		if (UUserWidget* UserWidget = WeakUserWidget.Get())
		{
			check(IntermediateRenderTarget);
			FVector2D TextureSize = FVector2D(IntermediateRenderTarget->SizeX, IntermediateRenderTarget->SizeY);

			static const float DeltaTime = 0.f;
			UMGRenderer->DrawWidget(IntermediateRenderTarget, UserWidget->TakeWidget(), TextureSize, DeltaTime);
		}
	}

	UTexture* FDMXPixelMappingRenderInputUserWidgetProxy::GetRenderedTexture() const
	{
		return IntermediateRenderTarget;
	}

	FVector2D FDMXPixelMappingRenderInputUserWidgetProxy::GetSize2D() const
	{
		check(IntermediateRenderTarget);
		return FVector2D(IntermediateRenderTarget->GetSurfaceWidth(), IntermediateRenderTarget->GetSurfaceHeight());
	}

	void FDMXPixelMappingRenderInputUserWidgetProxy::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(IntermediateRenderTarget);
	}
}
