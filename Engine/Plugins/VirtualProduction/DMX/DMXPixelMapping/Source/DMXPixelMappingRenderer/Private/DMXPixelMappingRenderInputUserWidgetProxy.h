// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingPreprocessRenderer.h"

#include "UObject/GCObject.h"

class FWidgetRenderer;
class UUserWidget;
class UTextureRenderTarget2D;


namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	/** Proxy responsible for the input user widget */
	class FDMXPixelMappingRenderInputUserWidgetProxy
		: public IPreprocessRenderInputProxy
		, public FGCObject
	{
	public:
		FDMXPixelMappingRenderInputUserWidgetProxy(UUserWidget* InUserWidget, const FVector2D& InInputSize, EPixelFormat InFormat);

		//~ Begin IPreprocessRenderInputProxy interface
		virtual void Render() override;
		virtual UTexture* GetRenderedTexture() const override;
		virtual FVector2D GetSize2D() const override;
		//~ End IPreprocessRenderInputProxy interface

	protected:
		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override
		{
			return TEXT("DMXPixelMapping::Rendering::Private::FDMXPixelMappingRenderInputUserWidgetProxy");
		}
		//~ End FGCObject interface

	private:
		/** UMG widget renderer */
		TSharedPtr<FWidgetRenderer> UMGRenderer;

		/** The Input object */
		TWeakObjectPtr<UUserWidget> WeakUserWidget;

		/** Intermediate render target for materials and user widgets */
		TObjectPtr<UTextureRenderTarget2D> IntermediateRenderTarget;
	};
}
