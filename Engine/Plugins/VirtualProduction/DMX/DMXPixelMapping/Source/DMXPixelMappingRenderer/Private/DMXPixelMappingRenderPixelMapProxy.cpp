// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRenderPixelMapProxy.h"

#include "CommonRenderResources.h"
#include "DMXPixelMappingRenderElement.h"
#include "Engine/TextureRenderTarget2D.h"
#include "PixelShaderUtils.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderingThread.h"
#include "RHIResources.h"
#include "TextureResource.h"


namespace UE::DMXPixelMapping::Rendering::Private
{
	static constexpr TCHAR RenderPassName[] = TEXT("RenderPixelMapping");
	static constexpr TCHAR RenderPassHint[] = TEXT("Render Pixel Mapping");
};

DECLARE_GPU_STAT_NAMED(DMXPixelMappingShadersStat, UE::DMXPixelMapping::Rendering::Private::RenderPassHint);

class FDMXPixelBlendingQualityDimension : SHADER_PERMUTATION_ENUM_CLASS("PIXELBLENDING_QUALITY", EDMXPixelBlendingQuality);
class FDMXVertexUVDimension : SHADER_PERMUTATION_BOOL("VERTEX_UV_STATIC_CALCULATION");

/**
 * Pixel Mapping downsampling vertex shader
 */
class FDMXPixelMappingRendererVS
	: public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDMXPixelMappingRendererVS);
	SHADER_USE_PARAMETER_STRUCT(FDMXPixelMappingRendererVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FDMXPixelBlendingQualityDimension, FDMXVertexUVDimension>;
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, DrawRectanglePosScaleBias)
		SHADER_PARAMETER(FVector4f, DrawRectangleInvTargetSizeAndTextureSize)
		SHADER_PARAMETER(FVector4f, DrawRectangleUVScaleBias)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }
};

/**
 * Pixel Mapping downsampling pixel shader
 */
class FDMXPixelMappingRendererPS
	: public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FDMXPixelMappingRendererPS);
	SHADER_USE_PARAMETER_STRUCT(FDMXPixelMappingRendererPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FDMXPixelBlendingQualityDimension, FDMXVertexUVDimension>;
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)

		SHADER_PARAMETER(float, Brightness)
		SHADER_PARAMETER(FVector2f, UVTopLeftRotated)
		SHADER_PARAMETER(FVector2f, UVTopRightRotated)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }
};

IMPLEMENT_GLOBAL_SHADER(FDMXPixelMappingRendererVS, "/Plugin/DMXPixelMapping/Private/DMXPixelMapping.usf", "DMXPixelMappingVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FDMXPixelMappingRendererPS, "/Plugin/DMXPixelMapping/Private/DMXPixelMapping.usf", "DMXPixelMappingPS", SF_Pixel);



namespace UE::DMXPixelMapping::Rendering::Private
{
	void FDMXPixelMappingRenderPixelMapProxy::Render(
		UTexture* InInputTexture,
		UTextureRenderTarget2D* InRenderTarget,
		const TArray<TSharedRef<FPixelMapRenderElement>>& InElements,
		float Brightness
	)
	{
		check(IsInGameThread());

		// Ensure lifetime
		InputTexture = InInputTexture;
		RenderTarget = InRenderTarget;

		// Actual rendering
		ENQUEUE_RENDER_COMMAND(RenderTargetResource)(
			[SharedThis = StaticCastSharedRef<FDMXPixelMappingRenderPixelMapProxy>(AsShared()), this, Elements = InElements, Brightness]
			(FRHICommandListImmediate& RHICmdList)
			{
				RHI_BREADCRUMB_EVENT_STAT(RHICmdList, DMXPixelMappingShadersStat, "RenderPixelMapping");

				const FTextureResource* InputTextureResource = InputTexture ? InputTexture->GetResource() : nullptr;
				const FTextureRenderTargetResource* RenderTargetResource = SharedThis->RenderTarget ? SharedThis->RenderTarget->GetRenderTargetResource() : nullptr;

				if (!InputTextureResource || !RenderTargetResource)
				{
					return;
				}

				const FIntPoint PixelSize(1);
				const FIntPoint TextureSize(1);

				const FIntPoint InputTextureSize = FIntPoint(InputTextureResource->GetSizeX(), InputTextureResource->GetSizeY());
				const FIntPoint OutputTextureSize = FIntPoint(RenderTargetResource->GetSizeX(), RenderTargetResource->GetSizeY());

				const FTextureRHIRef InputTextureRHI = InputTextureResource->TextureRHI;

				RHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->TextureRHI, ERHIAccess::SRVMask, ERHIAccess::RTV));

				FRHIRenderPassInfo RpInfo(RenderTargetResource->TextureRHI, ERenderTargetActions::Load_Store);
				RHICmdList.BeginRenderPass(RpInfo, RenderPassName);
				{
					RHICmdList.SetViewport(0.f, 0.f, 0.f, OutputTextureSize.X, OutputTextureSize.Y, 1.f);

					for (int32 PixelIndex = 0; PixelIndex < Elements.Num(); PixelIndex++)
					{
						const TSharedRef<FPixelMapRenderElement>& Element = Elements[PixelIndex];

						const FVector2D& UV = Element->GetParameters().UV;
						const FVector2D& UVSize = Element->GetParameters().UVSize;
						const FVector2D& UVTopLeftRotated = Element->GetParameters().UVTopLeftRotated;
						const FVector2D& UVTopRightRotated = Element->GetParameters().UVTopRightRotated;

						const int32 RenderTargetPoisitionX = PixelIndex % OutputTextureSize.X;
						const int32 RenderTargetPositionY = PixelIndex / OutputTextureSize.X;

						// Create shader permutations
						FDMXPixelMappingRendererPS::FPermutationDomain PermutationVector;
						PermutationVector.Set<FDMXPixelBlendingQualityDimension>(Element->GetParameters().CellBlendingQuality);
						PermutationVector.Set<FDMXVertexUVDimension>(Element->GetParameters().bStaticCalculateUV);

						// Create shaders
						FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
						TShaderMapRef<FDMXPixelMappingRendererVS> VertexShader(ShaderMap, PermutationVector);
						TShaderMapRef<FDMXPixelMappingRendererPS> PixelShader(ShaderMap, PermutationVector);

						// Setup graphics pipeline
						FGraphicsPipelineStateInitializer GraphicsPSOInit;
						RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
						GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
						GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
						GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
						GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
						GraphicsPSOInit.PrimitiveType = PT_TriangleList;
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						// Set vertex shader buffer
						FDMXPixelMappingRendererVS::FParameters VSParameters;
						VSParameters.DrawRectanglePosScaleBias = FVector4f(PixelSize.X, PixelSize.Y, RenderTargetPoisitionX, RenderTargetPositionY);
						VSParameters.DrawRectangleUVScaleBias = FVector4f(UVSize.X, UVSize.Y, UV.X, UV.Y);
						VSParameters.DrawRectangleInvTargetSizeAndTextureSize = FVector4f(
							1.0 / OutputTextureSize.X, 1.0 / OutputTextureSize.Y,
							1.0 / TextureSize.X, 1.0 / TextureSize.Y);
						SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters);

						// Set pixel shader buffer
						FDMXPixelMappingRendererPS::FParameters PSParameters;
						PSParameters.InputTexture = InputTextureRHI;
						PSParameters.Brightness = Brightness;
						PSParameters.InputSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
						PSParameters.UVTopLeftRotated = FVector2f(UVTopLeftRotated);
						PSParameters.UVTopRightRotated = FVector2f(UVTopRightRotated);
						SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSParameters);

						// Draw a two triangle on the entire viewport.
						FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);
					}
				}
				RHICmdList.EndRenderPass();

				// Copy texture from GPU to CPU
				{
					// Read the contents of a texture to an output CPU buffer
					TArray<FLinearColor> ColorArray;
					const FIntRect Rect(0, 0, OutputTextureSize.X, OutputTextureSize.Y);

					// Read surface data. 
					// Use min/max compression mode here - Either the input was tone mapped earlier, or we want to preserve dynamic range.
					RHICmdList.ReadSurfaceData(RenderTargetResource->TextureRHI, Rect, ColorArray, ERangeCompressionMode::RCM_MinMax);

					// Write data to the game thread
					if (!ensureMsgf(ColorArray.Num() >= Elements.Num(), TEXT("Pixel maparray is smaller than the number of elements that should have been rendered. Failed to Pixel Map")))
					{
						return;
					}

					for (int32 PixelIndex = 0; PixelIndex < Elements.Num(); PixelIndex++)
					{
						Elements[PixelIndex]->SetColor(ColorArray[PixelIndex]);
					}
				}
			});
	}

	void FDMXPixelMappingRenderPixelMapProxy::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(InputTexture);
		Collector.AddReferencedObject(RenderTarget);
	}
}
