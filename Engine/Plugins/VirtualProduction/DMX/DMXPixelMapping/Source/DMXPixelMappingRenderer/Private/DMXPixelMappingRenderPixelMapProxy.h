// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingPixelMapRenderer.h"
#include "UObject/GCObject.h"

class UTexture;
class UTextureRenderTarget2D;
namespace UE::DMXPixelMapping::Rendering { class FPixelMapRenderElement; }


namespace UE::DMXPixelMapping::Rendering::Private
{
	/** Interface for the object responsible to render the input texture/material/umg widget. */
	class FDMXPixelMappingRenderPixelMapProxy
		: public IRenderPixelMapProxy
		, public FGCObject
	{
	public:
		/**
		 * Pixelmapping specific, downsample and draw input texture to destination texture.
		 *
		 * @param InInputTexture				The input texture to pixel map
		 * @param InRenderTarget				The render target that is pixel mapped to
		 * @param InElements					Elements to be rendered
		 * @param InBrightness					Global brightness
		 * @param InCallback					Callback for reading  the pixels from GPU to CPU
		 */
		virtual void Render(
			UTexture* InInputTexture,
			UTextureRenderTarget2D* InRenderTarget,
			const TArray<TSharedRef<FPixelMapRenderElement>>& InElements,
			float InBrightness
		) override;

	protected:
		//~ Begin FGCObject interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FDMXPixelMappingRenderPixelMapProxy"); }
		//~ End FGCObject interface

	private:
		/** Strong ref to the current input texture */
		TObjectPtr<UTexture> InputTexture;

		/** Strong ref to the current render target */
		TObjectPtr<UTextureRenderTarget2D> RenderTarget;
	};
}
