// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRenderer.h"

#include "Blueprint/UserWidget.h"
#include "CanvasTypes.h"
#include "ClearQuad.h"
#include "CommonRenderResources.h"
#include "Engine/Canvas.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Materials/Material.h"
#include "Modules/ModuleManager.h"
#include "PixelShaderUtils.h"
#include "RenderingThread.h"
#include "Rendering/SlateRenderer.h"
#include "ScreenRendering.h"
#include "TextureResource.h"
#include "SlateMaterialBrush.h"
#include "Slate/WidgetRenderer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"


namespace DMXPixelMappingRenderer
{
	static constexpr TCHAR RenderPassName[] = TEXT("RenderPixelMapping");
	static constexpr TCHAR RenderPassHint[] = TEXT("Render Pixel Mapping");
};

#if WITH_EDITOR
namespace DMXPixelMappingRenderer
{
	static constexpr TCHAR RenderPreviewPassName[] = TEXT("PixelMappingPreview");
	static constexpr TCHAR RenderPreviewPassHint[] = TEXT("Pixel Mapping Preview");
};

DECLARE_GPU_STAT_NAMED(DMXPixelMappingPreviewStat, DMXPixelMappingRenderer::RenderPreviewPassHint);
#endif // WITH_EDITOR

class UE_DEPRECATED(5.3, "Only here to keep support of deprecated FDMXPixelMappingRenderer::DownsampleRender.") FDEPRECATED_DMXPixelBlendingQualityDimension;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FDEPRECATED_DMXPixelBlendingQualityDimension : SHADER_PERMUTATION_ENUM_CLASS("PIXELBLENDING_QUALITY", EDMXPixelShaderBlendingQuality);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.3, "Only here to keep support of deprecated FDMXPixelMappingRenderer::DownsampleRender.") FDEPRECATED_DMXVertexUVDimension;
class FDEPRECATED_DMXVertexUVDimension : SHADER_PERMUTATION_BOOL("VERTEX_UV_STATIC_CALCULATION");

/**
 * Pixel Mapping downsampling vertex shader
 */
class UE_DEPRECATED(5.3, "Only here to keep support of deprecated FDMXPixelMappingRenderer::DownsampleRender.") FDEPRECATED_DMXPixelMappingRendererVS;
class FDEPRECATED_DMXPixelMappingRendererVS
	: public FGlobalShader
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

public:
	DECLARE_GLOBAL_SHADER(FDEPRECATED_DMXPixelMappingRendererVS);
	SHADER_USE_PARAMETER_STRUCT(FDEPRECATED_DMXPixelMappingRendererVS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FDEPRECATED_DMXPixelBlendingQualityDimension, FDEPRECATED_DMXVertexUVDimension>;
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector4f, DrawRectanglePosScaleBias)
		SHADER_PARAMETER(FVector4f, DrawRectangleInvTargetSizeAndTextureSize)
		SHADER_PARAMETER(FVector4f, DrawRectangleUVScaleBias)
	END_SHADER_PARAMETER_STRUCT()

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }
};

/**
 * Pixel Mapping downsampling pixel shader
 */
class UE_DEPRECATED(5.3, "Only here to keep support of deprecated FDMXPixelMappingRenderer::DownsampleRender.") FDEPRECATED_DMXPixelMappingRendererPS;
class FDEPRECATED_DMXPixelMappingRendererPS
	: public FGlobalShader
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

public:
	DECLARE_GLOBAL_SHADER(FDEPRECATED_DMXPixelMappingRendererPS);
	SHADER_USE_PARAMETER_STRUCT(FDEPRECATED_DMXPixelMappingRendererPS, FGlobalShader);

	using FPermutationDomain = TShaderPermutationDomain<FDEPRECATED_DMXPixelBlendingQualityDimension, FDEPRECATED_DMXVertexUVDimension>;
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_TEXTURE(Texture2D, InputTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InputSampler)

		SHADER_PARAMETER(float, Brightness)
		SHADER_PARAMETER(FVector2f, UVTopLeftRotated)
		SHADER_PARAMETER(FVector2f, UVTopRightRotated)
	END_SHADER_PARAMETER_STRUCT()

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters) { return true; }
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
IMPLEMENT_GLOBAL_SHADER(FDEPRECATED_DMXPixelMappingRendererVS, "/Plugin/DMXPixelMapping/Private/DMXPixelMapping.usf", "DMXPixelMappingVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FDEPRECATED_DMXPixelMappingRendererPS, "/Plugin/DMXPixelMapping/Private/DMXPixelMapping.usf", "DMXPixelMappingPS", SF_Pixel);
PRAGMA_ENABLE_DEPRECATION_WARNINGS


PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDMXPixelMappingRenderer::FDMXPixelMappingRenderer()
{
	static const FName RendererModuleName("Renderer");
	RendererModule = FModuleManager::GetModulePtr<IRendererModule>(RendererModuleName);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FDMXPixelMappingRenderer::DownsampleRender(
	const FTextureResource* InputTexture,
	const FTextureResource* DstTexture,
	const FTextureRenderTargetResource* DstTextureTargetResource,
	const TArray<FDMXPixelMappingDownsamplePixelParamsV2>& InDownsamplePixelPass,
	DownsampleReadCallback InCallback
) const
{
	// DEPRECATED 5.3

	check(IsInGameThread());
	
	ENQUEUE_RENDER_COMMAND(DownsampleRenderRDG)(
		[this, InputTexture, DstTexture, DstTextureTargetResource, InCallback, DownsamplePixelPass = InDownsamplePixelPass]
		(FRHICommandListImmediate& RHICmdList)
		{
			FRHITexture* RenderTargetRef = DstTextureTargetResource->TextureRHI;
			FRHITexture* DstTextureRef = DstTexture->TextureRHI;
			FRHITexture* ResolveRenderTarget = DstTextureTargetResource->GetRenderTargetTexture();

			if (!RenderTargetRef || !DstTextureRef || !ResolveRenderTarget)
			{
				ensure(false);
				return;
			}

			const FIntPoint PixelSize(1);
			const FIntPoint TextureSize(1);

			const FIntPoint InputTextureSize = FIntPoint(InputTexture->GetSizeX(), InputTexture->GetSizeY());
			const FIntPoint OutputTextureSize = FIntPoint(DstTexture->GetSizeX(), DstTexture->GetSizeY());

			const FTextureRHIRef InputTextureRHI = InputTexture->TextureRHI;

			RHICmdList.Transition(FRHITransitionInfo(RenderTargetRef, ERHIAccess::SRVMask, ERHIAccess::RTV));

			FRHIRenderPassInfo RpInfo(RenderTargetRef, ERenderTargetActions::Load_Store);
			RHICmdList.BeginRenderPass(RpInfo, DMXPixelMappingRenderer::RenderPassName);
			{
				RHICmdList.SetViewport(0.f, 0.f, 0.f, OutputTextureSize.X, OutputTextureSize.Y, 1.f);

				for (const FDMXPixelMappingDownsamplePixelParamsV2& PixelParam : DownsamplePixelPass)
				{
					// Create shader permutations
					FDEPRECATED_DMXPixelMappingRendererPS::FPermutationDomain PermutationVector;
					PermutationVector.Set<FDEPRECATED_DMXPixelBlendingQualityDimension>(static_cast<EDMXPixelShaderBlendingQuality>(PixelParam.CellBlendingQuality));
					PermutationVector.Set<FDEPRECATED_DMXVertexUVDimension>(PixelParam.bStaticCalculateUV);

					// Create shaders
					FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
					TShaderMapRef<FDEPRECATED_DMXPixelMappingRendererVS> VertexShader(ShaderMap, PermutationVector);
					TShaderMapRef<FDEPRECATED_DMXPixelMappingRendererPS> PixelShader(ShaderMap, PermutationVector);

					// Setup graphics pipeline
					FGraphicsPipelineStateInitializer GraphicsPSOInit;
					RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
					GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
					GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
					GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Never>::GetRHI();
					GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
					GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
					GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
					GraphicsPSOInit.PrimitiveType = PT_TriangleList;
					SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

					// Set vertex shader buffer
					FDEPRECATED_DMXPixelMappingRendererVS::FParameters VSParameters;
					VSParameters.DrawRectanglePosScaleBias = FVector4f(PixelSize.X, PixelSize.Y, PixelParam.Position.X, PixelParam.Position.Y);
					VSParameters.DrawRectangleUVScaleBias = FVector4f(PixelParam.UVSize.X, PixelParam.UVSize.Y, PixelParam.UV.X, PixelParam.UV.Y);
					VSParameters.DrawRectangleInvTargetSizeAndTextureSize = FVector4f(
						1.f / OutputTextureSize.X, 1.f / OutputTextureSize.Y,
						1.f / TextureSize.X, 1.f / TextureSize.Y);
					SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), VSParameters);
					
					// Set pixel shader buffer
					FDEPRECATED_DMXPixelMappingRendererPS::FParameters PSParameters;
					PSParameters.InputTexture = InputTextureRHI;
					PSParameters.Brightness = Brightness;
					PSParameters.InputSampler = TStaticSamplerState<SF_Trilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
					SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PSParameters);

					// Draw a two triangle on the entire viewport.
					FPixelShaderUtils::DrawFullscreenQuad(RHICmdList, 1);

				}
			}
			RHICmdList.EndRenderPass();

			// Copy texture from GPU to CPU
			{
				TransitionAndCopyTexture(RHICmdList, ResolveRenderTarget, RenderTargetRef, {});

				// Read the contents of a texture to an output CPU buffer
				TArray<FLinearColor> ColorArray;
				const FIntRect Rect(0, 0, OutputTextureSize.X, OutputTextureSize.Y);

				// Read surface without flush rendering thread
				RHICmdList.ReadSurfaceData(ResolveRenderTarget, Rect, ColorArray, FReadSurfaceDataFlags());

				// Fire the callback after drawing and copying texture to CPU buffer
				InCallback(MoveTemp(ColorArray), Rect);
			}
		});
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FDMXPixelMappingRenderer::RenderMaterial(UTextureRenderTarget2D* InRenderTarget, UMaterialInterface* InMaterialInterface) const
{	
	// DEPRECATED 5.3
	if (InMaterialInterface == nullptr)
	{
		return;
	}

	if (InRenderTarget == nullptr)
	{
		return;
	}

	FVector2D TextureSize = FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY);
	UMaterial* Material = InMaterialInterface->GetMaterial();

	if (Material != nullptr && Material->IsUIMaterial())
	{
		UIMaterialBrush->ImageSize = TextureSize;
		UIMaterialBrush->SetMaterial(InMaterialInterface);

		TSharedRef<SWidget> Widget =
			SNew(SOverlay)
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(UIMaterialBrush.Get())
			];

		static const float DeltaTime = 0.f;
		MaterialWidgetRenderer->DrawWidget(InRenderTarget, Widget, TextureSize, DeltaTime);
		
		// Reset material after drawing
		UIMaterialBrush->SetMaterial(nullptr);
	}
}

void FDMXPixelMappingRenderer::RenderWidget(UTextureRenderTarget2D* InRenderTarget, UUserWidget* InUserWidget) const
{
	// DEPRECATED 5.3
	if (InUserWidget == nullptr)
	{
		return;
	}

	if (InRenderTarget == nullptr)
	{
		return;
	}

	FVector2D TextureSize = FVector2D(InRenderTarget->SizeX, InRenderTarget->SizeY);
	static const float DeltaTime = 0.f;

	UMGRenderer->DrawWidget(InRenderTarget, InUserWidget->TakeWidget(), TextureSize, DeltaTime);
}


#if WITH_EDITOR
void FDMXPixelMappingRenderer::RenderPreview(const FTextureResource* TextureResource, const FTextureResource* DownsampleResource, TArray<FDMXPixelMappingDownsamplePixelPreviewParam>&& InPixelPreviewParamSet) const
{
	check(IsInGameThread());

	ENQUEUE_RENDER_COMMAND(DMXPixelMapping_CopyToPreveiewTexture)(
	[this, TextureResource, DownsampleResource, PixelPreviewParamSet = MoveTemp(InPixelPreviewParamSet)]
	(FRHICommandListImmediate& RHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT(RHICmdList, DMXPixelMappingPreviewStat, "PixelMappingPreview");

		const FTextureRHIRef DownsampleTextureRef = DownsampleResource->TextureRHI;
		const FTextureRHIRef RenderTargetRef = TextureResource->TextureRHI;

		if (!DownsampleTextureRef.IsValid() || !RenderTargetRef.IsValid())
		{
			ensure(false);
			return;
		}

		const FIntPoint OutputTextureSize = FIntPoint(TextureResource->GetSizeX(), TextureResource->GetSizeY());

		// Clear preview texture
		{
			FRHIRenderPassInfo RPInfo(RenderTargetRef, ERenderTargetActions::DontLoad_Store);
			RHICmdList.Transition(FRHITransitionInfo(RenderTargetRef, ERHIAccess::Unknown, ERHIAccess::RTV));
			RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearCanvas"));
			RHICmdList.SetViewport(0.f, 0.f, 0.f, OutputTextureSize.X, OutputTextureSize.Y, 1.f);
			DrawClearQuad(RHICmdList, FLinearColor::Black);
			RHICmdList.EndRenderPass();
		}

		// Render Preview
		{
			FRHIRenderPassInfo RPInfo(RenderTargetRef, ERenderTargetActions::Load_Store);
			RHICmdList.BeginRenderPass(RPInfo, DMXPixelMappingRenderer::RenderPreviewPassName);
			{
				RHICmdList.SetViewport(0.f, 0.f, 0.f, OutputTextureSize.X, OutputTextureSize.Y, 1.f);

				// Create shaders
				FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
				TShaderMapRef<FScreenVS> VertexShader(ShaderMap);
				TShaderMapRef<FScreenPS> PixelShader(ShaderMap);

				// Setup graphics pipeline
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

				SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), DownsampleTextureRef);

				const float DownsampleSizeX = DownsampleResource->GetSizeX();
				const float DownsampleSizeY = DownsampleResource->GetSizeY();

				constexpr float PixelSizeX = 1.f;
				constexpr float PixelSizeY = 1.f;

				const float SizeU = PixelSizeX / DownsampleSizeX;
				const float SizeV = PixelSizeY / DownsampleSizeY;

				// Draw preview downsampled rectangles
				for (const FDMXPixelMappingDownsamplePixelPreviewParam& PixelPreviewParam : PixelPreviewParamSet)
				{
					const float U = static_cast<float>(PixelPreviewParam.DownsamplePosition.X) / DownsampleSizeX;
					const float V = static_cast<float>(PixelPreviewParam.DownsamplePosition.Y) / DownsampleSizeY;

					RendererModule->DrawRectangle(
						RHICmdList,
						PixelPreviewParam.ScreenPixelPosition.X, PixelPreviewParam.ScreenPixelPosition.Y,	// Dest X, Y
						PixelPreviewParam.ScreenPixelSize.X, PixelPreviewParam.ScreenPixelSize.Y,		// Dest Width, Height
						U, V,																				// Source U, V
						SizeU, SizeV,																		// Source USize, VSize
						FIntPoint(TextureResource->GetSizeX(), TextureResource->GetSizeY()),				// Target buffer size
						FIntPoint(PixelSizeX, PixelSizeY),													// Source texture size
						VertexShader);
				}
			}
			RHICmdList.EndRenderPass();
		}
	});
}
#endif // WITH_EDITOR

void FDMXPixelMappingRenderer::RenderTextureToRectangle(const FTextureResource* InTextureResource, const FTextureRHIRef InRenderTargetTexture, FVector2D InSize, bool bSRGBSource) const
{
	check(IsInGameThread());

	struct FRenderContext
	{
		const FTextureResource* TextureResource = nullptr;
		const FTextureRHIRef Texture2DRHI = nullptr;
		FVector2D ViewportSize;
		bool bSRGBSource;
	};

	if (!InTextureResource)
	{
		return;
	}

	FRenderContext RenderContext
	{
		InTextureResource,
		InRenderTargetTexture,
		InSize,
		bSRGBSource
	};

	ENQUEUE_RENDER_COMMAND(DMXPixelMapping_CopyToPreveiewTexture)([this, RenderContext]
	(FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.Transition(FRHITransitionInfo(RenderContext.TextureResource->TextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));

		FRHIRenderPassInfo RPInfo(RenderContext.Texture2DRHI, ERenderTargetActions::Load_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("DMXPixelMapping_CopyToPreveiewTexture"));
		{
			RHICmdList.SetViewport(0.f, 0.f, 0.f, RenderContext.ViewportSize.X, RenderContext.ViewportSize.Y, 1.f);

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			auto ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FScreenVS> VertexShader(ShaderMap);

			GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleList;
			if (RenderContext.bSRGBSource)
			{
				TShaderMapRef<FScreenPSsRGBSource> PixelShader(ShaderMap);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), RenderContext.TextureResource->TextureRHI);
			}
			else
			{
				TShaderMapRef<FScreenPS> PixelShader(ShaderMap);
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParametersLegacyPS(RHICmdList, PixelShader, TStaticSamplerState<SF_Point>::GetRHI(), RenderContext.TextureResource->TextureRHI);
			}

			RendererModule->DrawRectangle(
				RHICmdList,
				0, 0,// X, Y Position in screen pixels of the top left corner of the quad
				RenderContext.ViewportSize.X, RenderContext.ViewportSize.Y,	// SizeX, SizeY	Size in screen pixels of the quad
				0, 0,		// U, V	Position in texels of the top left corner of the quad's UV's
				1, 1,		// SizeU, SizeV	Size in texels of the quad's UV's
				FIntPoint(RenderContext.ViewportSize.X, RenderContext.ViewportSize.Y), // TargetSizeX, TargetSizeY Size in screen pixels of the target surface
				FIntPoint(1, 1), // TextureSize Size in texels of the source texture
				VertexShader, // VertexShader The vertex shader used for rendering
				EDRF_Default); // Flags see EDrawRectangleFlags
		}
		RHICmdList.EndRenderPass();

		RHICmdList.Transition(FRHITransitionInfo(RenderContext.TextureResource->TextureRHI, ERHIAccess::SRVMask, ERHIAccess::RTV));
	});
}
