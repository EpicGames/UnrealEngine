// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDMXPixelMappingRenderer.h"

#include "DMXPixelMappingPreprocessRenderer.h"

class IRendererModule;
class FWidgetRenderer;

struct FSlateMaterialBrush;


/** DEPRECATED 5.2 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FDMXPixelMappingRenderer
	: public IDMXPixelMappingRenderer
{
public:
	/** Default constructor */
	FDMXPixelMappingRenderer();

	//~ Begin IDMXPixelMappingRenderer implementation
	virtual void DownsampleRender(
		const FTextureResource* InputTexture,
		const FTextureResource* DstTexture,
		const FTextureRenderTargetResource* DstTextureTargetResource,
		const TArray<FDMXPixelMappingDownsamplePixelParamsV2>& InDownsamplePixelPass,
		DownsampleReadCallback InCallback
	) const override;

	virtual void RenderMaterial(UTextureRenderTarget2D* InRenderTarget, UMaterialInterface* InMaterialInterface) const override;
	virtual void RenderWidget(UTextureRenderTarget2D* InRenderTarget, UUserWidget* InUserWidget) const override;

	virtual void RenderTextureToRectangle(const FTextureResource* InTextureResource, const FTextureRHIRef InRenderTargetTexture, FVector2D InSize, bool bSRGBSource) const override;

#if WITH_EDITOR
	virtual void RenderPreview(const FTextureResource* TextureResource, const FTextureResource* DownsampleResource, TArray<FDMXPixelMappingDownsamplePixelPreviewParam>&& InPixelPreviewParamSet) const override;
#endif // WITH_EDITOR
	//~ End IDMXPixelMappingRenderer implementation

private:
	/** Brush for Material widget renderer */
	TSharedPtr<FSlateMaterialBrush> UIMaterialBrush;

	/** Material widget renderer */
	TSharedPtr<FWidgetRenderer> MaterialWidgetRenderer;

	/** UMG widget renderer */
	TSharedPtr<FWidgetRenderer> UMGRenderer;

	/** The public interface of the renderer module. */
	IRendererModule* RendererModule;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
