// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRendererModule.h"
#include "DMXPixelMappingRenderer.h"

#include "Interfaces/IPluginManager.h"
#include "ShaderCore.h"
#include "Misc/Paths.h"

// Load shaders here
#define DMX_PIXEL_MAPPING_SHADERS_MAP TEXT("/Plugin/DMXPixelMapping")

void FDMXPixelMappingRendererModule::StartupModule()
{
	if (!AllShaderSourceDirectoryMappings().Contains(DMX_PIXEL_MAPPING_SHADERS_MAP))
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("DMXPixelMapping"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(DMX_PIXEL_MAPPING_SHADERS_MAP, PluginShaderDir);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<IDMXPixelMappingRenderer> FDMXPixelMappingRendererModule::CreateRenderer() const
{
	return MakeShared<FDMXPixelMappingRenderer>();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

IMPLEMENT_MODULE(FDMXPixelMappingRendererModule, DMXPixelMappingRenderer);
