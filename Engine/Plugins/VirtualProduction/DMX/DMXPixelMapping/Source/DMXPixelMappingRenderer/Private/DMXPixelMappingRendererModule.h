// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDMXPixelMappingRendererModule.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class IDMXPixelMappingRenderer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class FDMXPixelMappingRendererModule 
	: public IDMXPixelMappingRendererModule
{
public:

	//~ Begin IModuleInterface implementation
	virtual void StartupModule() override;
	//~ End IModuleInterface implementation

	//~ Begin IDMXPixelMappingRendererModule implementation
	UE_DEPRECATED(5.3, "IDMXPixelMappingRenderer was replaced with UDMXPixelMappigPixelMapRenderer")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TSharedPtr<IDMXPixelMappingRenderer> CreateRenderer() const override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//~ End IDMXPixelMappingRendererModule implementation
};
