// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelFormat.h"
#include "UObject/Object.h"

#include "DMXPixelMappingPixelMapRenderer.generated.h"

class FTextureResource;
class FTextureRenderTargetResource;
class UTextureRenderTarget2D;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;
class UUserWidget;
namespace UE::DMXPixelMapping::Rendering { class FPixelMapRenderElement; }


namespace UE::DMXPixelMapping::Rendering::Private
{
	/** Interface for the object responsible to render the pixel map */
	class IRenderPixelMapProxy
		: public TSharedFromThis<IRenderPixelMapProxy>
	{
	public:
		virtual ~IRenderPixelMapProxy() {}

		/**
		 * Pixelmapping specific, downsample and draw input texture to destination texture.
		 *
		 * @param InputTexture					The input texture to render
		 * @param RenderTarget					The render target to render to
		 * @param InElements						The elements that are pixel mapped
		 * @param Brightness						Global brightness of the rendered colors
		 */
		virtual void Render(
			UTexture* InInputTexture,
			UTextureRenderTarget2D* InRenderTarget,
			const TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>>& InElements,
			float InBrightness
		) = 0;
	};
}


/** Maps pixel map render elements to pixels */
UCLASS()
class DMXPIXELMAPPINGRENDERER_API UDMXPixelMappingPixelMapRenderer
	: public UObject
{
	GENERATED_BODY()

public:
	UDMXPixelMappingPixelMapRenderer();

	/** Sets the elements to render */
	void SetElements(const TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>>& InElements, EPixelFormat InFormat);

	/** DEPRECATED 5.4 - Sets the elements to render */
	UE_DEPRECATED(5.4, "It is now required to specify a pixel format, see related overload.")
	void SetElements(const TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>>& InElements);

	/** Renders the current elements */
	void Render(UTexture* InputTexture, float Brightness);

private:
	static const FIntPoint MaxPixelMapSize;

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> PixelMapRenderTarget;

	/** The elements to render */
	TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>> RenderElements;

	/** Proxy that carries out the actual rendering */
	TSharedPtr<UE::DMXPixelMapping::Rendering::Private::IRenderPixelMapProxy> RendererPixelMapProxy;
};
