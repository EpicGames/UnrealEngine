// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PixelFormat.h"
#include "UObject/Object.h"

#include "DMXPixelMappingPreprocessRenderer.generated.h"

class IDMXPixelMappingRenderer;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;
class UUserWidget;


UENUM(BlueprintType)
enum class EDMXPixelMappingRenderingPreprocessorSizeMode : uint8
{
	SameAsInput,
	Downsampled,
	CustomSize	
};

namespace UE::DMXPixelMapping::Rendering::Preprocess::Private
{
	/** Interface for the object responsible to render the input texture/material/umg widget. */
	class IPreprocessRenderInputProxy
		: public TSharedFromThis<IPreprocessRenderInputProxy>
	{
	public:
		virtual ~IPreprocessRenderInputProxy() {}

		/** Renders the input texture, optionally using a Filter material */
		virtual void Render() = 0;

		/** Returns the currently rendered texture, or nullptr if no texture was rendered yet */
		virtual UTexture* GetRenderedTexture() const = 0;

		/** Returns the size of the input object */
		virtual FVector2D GetSize2D() const = 0;
	};

	/** Interface for the object responsible to apply the material to the input texture. */
	class IPreprocessApplyFilterMaterialProxy
		: public TSharedFromThis<IPreprocessApplyFilterMaterialProxy>
	{
	public:
		virtual ~IPreprocessApplyFilterMaterialProxy() {}

		/** Renders the input texture, optionally using a Filter material */
		virtual void Render(UTexture* InInputTexture, const class UDMXPixelMappingPreprocessRenderer& InPreprocessRenderer) = 0;

		/** Returns the preprocessed texture */
		virtual UTexture* GetRenderedTexture() const = 0;
	};
}

/** Renders the texture that is used to pixel map */
UCLASS(BlueprintType)
class DMXPIXELMAPPINGRENDERER_API UDMXPixelMappingPreprocessRenderer
	: public UObject
{
	GENERATED_BODY()

public:
	/** Sets the input texture that is used when rendering */
	void SetInputTexture(UTexture* InTexture, EPixelFormat InFormat);

	/** Sets the input texture that is used when rendering */
	void SetInputMaterial(UMaterialInterface* InMaterial, EPixelFormat InFormat);

	/** Sets the input texture that is used when rendering */
	void SetInputUserWidget(UUserWidget* InUserWidget, EPixelFormat InFormat);

	/** DEPRECATED 5.4 - Sets the input texture that is used when rendering */
	UE_DEPRECATED(5.4, "It is now required to specify a pixel format, see related overload.")
	void SetInputTexture(UTexture* InTexture);

	/** DEPRECATED 5.4 - Sets the input texture that is used when rendering */
	UE_DEPRECATED(5.4, "It is now required to specify a pixel format, see related overload.")
	void SetInputMaterial(UMaterialInterface* InMaterial);

	/** DEPRECATED 5.4 - Sets the input texture that is used when rendering */
	UE_DEPRECATED(5.4, "It is now required to specify a pixel format, see related overload.")
	void SetInputUserWidget(UUserWidget* InUserWidget);

	/** Clears input */
	void ClearInput();

	/** Renders the current input */
	void Render();

	/** Returns the rendered input texture, or nullptr if no texture was rendered yet */
	UTexture* GetRenderedTexture() const;

	/** Returns the resulting size of preprocessing, even if rendering did not occur yet */
	FVector2D GetResultingSize2D() const;

	/** Returns the Filter material, or nullptr if no Filter material is set */
	const TObjectPtr<UMaterialInstanceDynamic>& GetFilterMID() const { return FilterMID; }

	/** Returns true, if the Filter material should be applied each downsample pass */
	bool ShouldApplyFilterMaterialEachDownsamplePass() const { return bApplyFilterMaterialEachDownsamplePass; }

	/** Returns the number of time the texture should be downsampled by factor 2 */
	int32 GetNumDownsamplePasses() const { return NumDownSamplePasses; }

	/** Returns the desired output size, or an unset optional vector 2D if the resulting size should be used */
	TOptional<FVector2D> GetDesiredOutputSize2D() const;

	/** Returns the blur distance parameter */
	float GetBlurDistance() const { return BlurDistance; }

	/** Returns the texture parameter name */
	FName GetTextureParameterName() const { return TextureParameterName; }

	/** Returns the blur distance parameter name */
	FName GetBlurDistanceParameterName() const { return BlurDistanceParamterName; }

	// Property name getters
	static FName GetOutputSizeModeMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingPreprocessRenderer, OutputSizeMode); }
	static FName GetCustomOutputSizeMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingPreprocessRenderer, CustomOutputSize); }
	static FName GetFilterMaterialMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingPreprocessRenderer, FilterMaterial); }
	static FName GetBlurDistanceMemberNameChecked() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingPreprocessRenderer, BlurDistance); }

protected:
	//~ Begin UObject interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject interface

private:
	/** Size of the input texture (Material and User Widget only) */
	UPROPERTY(EditAnywhere, Category = "Render Settings", Meta = (EditCondition = "bShowInputSize", EditConditionHides, HideEditConditionToggle, ClampMin = "1", ClampMax = "8192", UIMin = "1", UIMax = "4096", AllowPrivateAccess = true))
	FVector2D InputSize{ 1024.f, 1024.f };

	/** Number of times the pixelmapping input is downsampled */
	UPROPERTY(EditAnywhere, Category = "Filtering", Meta = (ClampMin = "0", ClampMax = "256", UIMin = "0", UIMax = "16", AllowPrivateAccess = true))
	int32 NumDownSamplePasses = 0;

	/** Defines how the texture is resized after filtering */
	UPROPERTY(EditAnywhere, Category = "Filtering", Meta = (AllowPrivateAccess = true))
	EDMXPixelMappingRenderingPreprocessorSizeMode OutputSizeMode = EDMXPixelMappingRenderingPreprocessorSizeMode::SameAsInput;

	/** Size of the rendered texture */
	UPROPERTY(EditAnywhere, Category = "Filtering", Meta = (DisplayName = "Custom Size", ClampMin = "1", ClampMax = 2048, UIMin = 1, UIMax = 2048, EditCondition = "OutputSizeMode == EDMXPixelMappingRenderingPreprocessorSizeMode::CustomSize", EditConditionHides, AllowPrivateAccess = true))
	FIntPoint CustomOutputSize { 256, 256 };
	
	/** Filter material applied to the rendered input */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Filtering", Meta = (AllowPrivateAccess = true))
	TObjectPtr<UMaterialInterface> FilterMaterial;

	/** Actual material instance dynamic applied to the rendered input */
	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> FilterMID;

	/** If true, the filter material is applied each downsample pass, otherwise only once after the last pass */
	UPROPERTY(EditAnywhere, Category = "Filtering", Meta = (AllowPrivateAccess = true, DisplayName = "Apply Material each pass"))
	bool bApplyFilterMaterialEachDownsamplePass = true;

	/** Blur distance applied, only applicable if the filter matierial has a "BlurDistance" parameter */
	UPROPERTY(EditAnywhere, Category = "Filtering", Meta = (UIMin = "0", UIMax = "1", SliderExponent = 2.0, AllowPrivateAccess = true))
	float BlurDistance = .02f;

	/** The texture parameter name in the Filter Material */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Filtering", Meta = (AllowPrivateAccess = true))
	FName TextureParameterName = "Texture";

	/** The blur distance parameter name in the Filter Material */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Filtering", Meta = (AllowPrivateAccess = true))
	FName BlurDistanceParamterName = "BlurDistance";

	/** If true, shows the input size property */
	UPROPERTY(Transient)
	bool bShowInputSize = false;

	/** Proxy to render the input */
	TSharedPtr<UE::DMXPixelMapping::Rendering::Preprocess::Private::IPreprocessRenderInputProxy> RenderInputProxy;

	/** Proxy to render the input */
	TSharedPtr<UE::DMXPixelMapping::Rendering::Preprocess::Private::IPreprocessApplyFilterMaterialProxy> ApplyMaterialProxy;
};
