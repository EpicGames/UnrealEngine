// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingTripleBufferedData.h"
#include "HAL/CriticalSection.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"

#include "DMXPixelMappingRenderElement.generated.h"


/** Enum that defines the quality of how pixels are rendered */
UENUM()
enum class EDMXPixelBlendingQuality : uint8
{
	/** 1 sample */
	Low,

	/** 5 samples ( 2 x 2 with center) */
	Medium,

	/** 9 samples ( 3 x 3 ) */
	High,

	/** Max, for shaders only */
	MAX UMETA(Hidden)
};

namespace UE::DMXPixelMapping::Rendering
{
	struct DMXPIXELMAPPINGRENDERER_API FPixelMapRenderElementParameters
	{
		/** Position in texels of the center of the quad's UV's */
		FVector2D UV;

		/** Size in texels of UV. May match UVSize */
		FVector2D UVSize;

		/** Position in texels of the top left corner of the quad's UV's, in rotated space */
		FVector2D UVTopLeftRotated;

		/** Position in texels of the top right corner of the quad's UV's, in rotate space */
		FVector2D UVTopRightRotated;

		/** Rotation in degrees. Useful to paint these elements in the preview view. */
		double Rotation;

		/** DERPECATED 5.4 - Size in texels of the quad's total UV space */
		FVector2D UVCellSize_DEPRECATED;

		/** The quality of color samples in the pixel shader(number of samples) */
		EDMXPixelBlendingQuality CellBlendingQuality;

		/** Calculates the UV point to sample purely on the UV position/size. Works best for renderers which represent a single pixel */
		bool bStaticCalculateUV = true;
	};

	/** An element rendered by the pixel map renderer */
	class DMXPIXELMAPPINGRENDERER_API FPixelMapRenderElement
		: public TSharedFromThis<FPixelMapRenderElement>
	{
	public:
		FPixelMapRenderElement(FPixelMapRenderElementParameters InParameters)
			: Parameters(InParameters)
		{}

		virtual ~FPixelMapRenderElement() {};

		/** Sets the parameters, thread-safe */
		void SetParameters(FPixelMapRenderElementParameters InParameters);

		/** Gets the parameters, thread-safe */
		FPixelMapRenderElementParameters GetParameters() const;

		/** Sets the pixel color, thread safe. This is called from the renderer to set the rendered color. */
		void SetColor(const FLinearColor& Color);

		/** Gets the pixel color, thread safe */
		FLinearColor GetColor() const;

	private:
		/** The current color, readable from the game thread */
		mutable FLinearColor ColorGameThread;

		/** Triple buffered color data, useful to copy data to game thread without locking. */
		mutable UE::DMX::Internal::TDMXPixelMappingTripleBufferedData<FLinearColor> ColorTipleBuffer;

		/** Pointer to the color data that is currently safe to write for the producer */
		FLinearColor* ProducerColorPtr = nullptr;

		/** The current parameters */
		FPixelMapRenderElementParameters Parameters;

		/** Mutex acccess to parameters */
		mutable FCriticalSection AccessParametersMutex;
	};
}
