// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHI.h"
#include "RHIResources.h"

enum class EDMXPixelBlendingQuality : uint8;

class FTextureResource;
class FTextureRenderTargetResource;
class UTextureRenderTarget2D;
class UMaterialInstanceDynamic;
class UMaterialInterface;
class UTexture;
class UUserWidget;


/**
 * Used in shader permutation for determining number of samples to use in texture blending.
 * If adding to this you must also adjust the public facing option: 'EPixelBlendingQuality' under the runtime module's DMXPixelMappingOutputComponent.h
 */
enum class UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer instead.") EDMXPixelShaderBlendingQuality : uint8;
enum class EDMXPixelShaderBlendingQuality : uint8
{
	Low,
	Medium,
	High,

	MAX
};


/**
 * Downsample pixel preview rendering params.
 * Using for pixel rendering setting in preview
 */
struct UE_DEPRECATED(5.3, "Deprecated in favor of FPixelMapRenderElement.") FDMXPixelMappingDownsamplePixelParam;
struct FDMXPixelMappingDownsamplePixelPreviewParam
{
	/** Position in screen pixels of the top left corner of the quad */
	FVector2D ScreenPixelPosition;

	/** Size in screen pixels of the quad */
	FVector2D ScreenPixelSize;

	/** Downsample pixel position in screen pixels of the quad */
	FIntPoint DownsamplePosition;
};

/**
 * Downsample pixel rendering params
 * Using for pixel rendering in downsample rendering pipeline
 */
struct UE_DEPRECATED(5.2, "Deprecated in favor of FDMXPixelMappingDownsamplePixelParamsV2. To apply color spaces, all color values are now computed at all times.") FDMXPixelMappingDownsamplePixelParam;
struct FDMXPixelMappingDownsamplePixelParam
{
	/** RGBA pixel multiplication */
	FVector4 PixelFactor;

	/** RGBA pixel flag for inversion */
	FIntVector4 InvertPixel;

	/** Position in screen pixels of the top left corner of the quad */
	FIntPoint Position;

	/** Position in texels of the top left corner of the quad's UV's */
	FVector2D UV;

	/** Size in texels of the quad's total UV space */
	FVector2D UVSize;

	/** Size in texels of UV.May match UVSize */
	FVector2D UVCellSize;

	/** The quality of color samples in the pixel shader(number of samples) */
	EDMXPixelBlendingQuality CellBlendingQuality;

	/** Calculates the UV point to sample purely on the UV position/size. Works best for renderers which represent a single pixel */
	bool bStaticCalculateUV;
};

struct UE_DEPRECATED(5.3, "Deprecated in favor of FPixelMapRenderElement.") FDMXPixelMappingDownsamplePixelParamsV2;
struct FDMXPixelMappingDownsamplePixelParamsV2
{
	/** Position in screen pixels of the top left corner of the quad */
	FIntPoint Position;

	/** Position in texels of the top left corner of the quad's UV's */
	FVector2D UV;

	/** Size in texels of the quad's total UV space */
	FVector2D UVSize;

	/** Size in texels of UV.May match UVSize */
	FVector2D UVCellSize;

	/** The quality of color samples in the pixel shader(number of samples) */
	EDMXPixelBlendingQuality CellBlendingQuality;

	/** Calculates the UV point to sample purely on the UV position/size. Works best for renderers which represent a single pixel */
	bool bStaticCalculateUV;
};


/**
 * The public interface of the Pixel Mapping renderer instance interface.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class IDMXPixelMappingRenderer 
	: public TSharedFromThis<IDMXPixelMappingRenderer>
{
public:
	using DownsampleReadCallback = TFunction<void(TArray<FLinearColor>&&, FIntRect)>;

public:
	/** Virtual destructor */
	virtual ~IDMXPixelMappingRenderer() = default;

	/**
	 * Pixelmapping specific, downsample and draw input texture to destination texture.
	 *
	 * @param InputTexture					Rendering resource of input texture
	 * @param DstTexture					Rendering resource of RenderTarget texture
	 * @param InDownsamplePixelPass			Pixels rendering params
	 * @param InCallback					Callback for reading  the pixels from GPU to CPU			
	 */
	UE_DEPRECATED(5.3, "Please use UDMXDMXPixelMappingPixelMapRenderer instead.")
	virtual void DownsampleRender(
		const FTextureResource* InputTexture,
		const FTextureResource* DstTexture,
		const FTextureRenderTargetResource* DstTextureTargetResource,
		const TArray<FDMXPixelMappingDownsamplePixelParamsV2>& InDownsamplePixelPass,
		DownsampleReadCallback InCallback
	) const = 0;

	/**
	 * Render material into the RenderTarget2D
	 *
	 * @param InRenderTarget				2D render target texture resource
	 * @param InMaterialInterface			Material to use
	 */
	UE_DEPRECATED(5.3, "Please use FWidgetRenderer::DrawWidget or similar engine methods instead.")
	virtual void RenderMaterial(UTextureRenderTarget2D* InRenderTarget, UMaterialInterface* InMaterialInterface) const = 0;

	/**
	 * Render material into the RenderTarget2D
	 *
	 * @param InRenderTarget				2D render target texture resource
	 * @param InUserWidget					UMG widget to use
	 */
	UE_DEPRECATED(5.3, "Please use FWidgetRenderer::DrawWidget or similar engine methods instead.")
	virtual void RenderWidget(UTextureRenderTarget2D* InRenderTarget, UUserWidget* InUserWidget) const  = 0;

	/**
	 * Rendering input texture to render target
	 *
	 * @param InTextureResource				Input texture resource
	 * @param InRenderTargetTexture			RenderTarget
	 * @param InSize						Rendering size
	 * @param bSRGBSource					If the source texture is sRGB
	 */
	UE_DEPRECATED(5.3, "Removed without replacement. Please use core engine methods to draw texture to rectangle.")
	virtual void RenderTextureToRectangle(const FTextureResource* InTextureResource, const FTextureRHIRef InRenderTargetTexture, FVector2D InSize, bool bSRGBSource) const = 0;

#if WITH_EDITOR
	/**
	 * Render preview with one or multiple downsampled textures
	 *
	 * @param TextureResource				Rendering resource of RenderTarget texture
	 * @param DownsampleResource			Rendering resource of RenderTarget texture
	 * @param InPixelPreviewParamSet		Pixels rendering params
	 */
	 UE_DEPRECATED(5.3, "Pixel mapping no longer has a dedicated preview renderer. Instead draw pixel map elements to a view port (see FDMXPixelMappingPreviewViewportClient.")
	 virtual void RenderPreview(const FTextureResource* TextureResource, const FTextureResource* DownsampleResource, TArray<FDMXPixelMappingDownsamplePixelPreviewParam>&& InPixelPreviewParamSet) const = 0;
#endif // WITH_EDITOR

	/**
	* Sets the brigthness of the renderer
	*/
	UE_DEPRECATED(5.3, "IDMXPixelMappingRenderer and all its members are deprecated.")
	void SetBrightness(const float InBrightness) { Brightness = InBrightness; }

protected:
	/** Brightness multiplier for the renderer */
	float Brightness = 1.0f;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
