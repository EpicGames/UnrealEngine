// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class IDMXPixelMappingRenderer;
PRAGMA_ENABLE_DEPRECATION_WARNINGS


/**
 * Public module interface for the DMX Renderer module
 */
class IDMXPixelMappingRendererModule 
	: public IModuleInterface
{
public:
	static constexpr auto ModuleName = TEXT("DMXPixelMappingRenderer");

public:

	/** Virtual destructor */
	virtual ~IDMXPixelMappingRendererModule() {}

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IDMXPixelMappingRendererModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IDMXPixelMappingRendererModule>(IDMXPixelMappingRendererModule::ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IDMXPixelMappingRendererModule::ModuleName);
	}

	/**
	 * Get Renderer instance
	 * There is no limit to the amount of creation of renderers
	 *
	 * @return IDMXPixelMappingRenderer instance.
	 */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TSharedPtr<IDMXPixelMappingRenderer> CreateRenderer() const = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
