// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DMXPixelMappingRuntime : ModuleRules
{
	public DMXPixelMappingRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"DMXRuntime",
				"DMXPixelMappingRenderer",
				"RHI",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"UMG",
				"Slate",
				"SlateCore",
				"DMXProtocol",
				"DMXRuntime",
				"DMXPixelMappingCore",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",
					"DMXPixelMappingEditorWidgets",
				}
			);
		}
	}
}
