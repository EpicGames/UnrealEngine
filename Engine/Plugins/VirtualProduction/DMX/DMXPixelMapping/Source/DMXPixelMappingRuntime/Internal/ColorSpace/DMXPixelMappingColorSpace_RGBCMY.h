// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "ColorSpace/DMXPixelMappingColorSpace.h"
#include "ColorManagement/ColorSpace.h"
#include "Templates/UniquePtr.h"

#include "DMXPixelMappingColorSpace_RGBCMY.generated.h"

/** The color space of the DMX output */
UENUM(BlueprintType)
enum class EDMXPixelMappingOutputColorSpace_RGBCMY : uint8
{
	sRGB UMETA(DisplayName = "sRGB / Rec709"),
	Rec2020,
	P3DCI,
	P3D65,
	Plasa UMETA(DisplayName = "PLASA RGB - ANSI E1.54")
};

/** The gamma mode of the DMX output */
UENUM(BlueprintType)
enum class EDMXPixelMappingGamma_RGBCMY : uint8
{
	Linear,
	AsOutputColorSpace,
	Custom
};

/** Defines how the luminance channel (typically Dimmer) is generated */
UENUM(BlueprintType)
enum class EDMXPixelMappingLuminanceType_RGBCMY : uint8
{
	FromColor,
	Constant,
	FromAlpha,
	None
};

UCLASS(meta = (DisplayName = "RGB / CMY"))
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingColorSpace_RGBCMY
	: public UDMXPixelMappingColorSpace
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXPixelMappingColorSpace_RGBCMY();

	//~ Begin DMXPixelMappingColorSpace interface
	virtual void SetRGBA(const FLinearColor& InColor) override;
	//~ End DMXPixelMappingColorSpace interface

	/** The color space to use */
	UPROPERTY(EditAnywhere, Category = "Color Space", Meta = (DisplayName = "Output Color Space"))
	EDMXPixelMappingOutputColorSpace_RGBCMY PixelMappingOutputColorSpace = EDMXPixelMappingOutputColorSpace_RGBCMY::sRGB;

	/** Applies Gamma correction to the output DMX Values */
	UPROPERTY(EditAnywhere, Category = "Color Space")
	EDMXPixelMappingGamma_RGBCMY OutputGamma = EDMXPixelMappingGamma_RGBCMY::AsOutputColorSpace;

	/** The custom Gamma, whereas OutColor = Pow(InColor, 1 / CustomGamma) */
	UPROPERTY(EditAnywhere, Category = "Color Space", Meta = (UIMin = 0.0, UIMax = 4.0, EditCondition = "OutputGamma == EDMXPixelMappingGamma_RGBCMY::Custom", EditConditionHides))
	float CustomGamma = 2.2f;

	/** If set, converts Red to Cyan */
	UPROPERTY(EditAnywhere, Category = "RGB")
	bool bSendCyan = false;

	/** If set, converts Green to Magenta */
	UPROPERTY(EditAnywhere, Category = "RGB")
	bool bSendMagenta = false;

	/** If set, converts Blue to Yellow */
	UPROPERTY(EditAnywhere, Category = "RGB")
	bool bSendYellow = false;

	/** Attribute sent for Red */
	UPROPERTY(EditAnywhere, Category = "RGB")
	FDMXAttributeName RedAttribute;

	/** Attribute sent for Green */
	UPROPERTY(EditAnywhere, Category = "RGB")
	FDMXAttributeName GreenAttribute;

	/** Attribute sent for Blue */
	UPROPERTY(EditAnywhere, Category = "RGB")
	FDMXAttributeName BlueAttribute;

	/** Adds a Dimmer Attribute */
	UPROPERTY(EditAnywhere, Category = "Luminance")
	EDMXPixelMappingLuminanceType_RGBCMY LuminanceType = EDMXPixelMappingLuminanceType_RGBCMY::FromColor;

	/** Attribute sent for the generated Luminance */
	UPROPERTY(EditAnywhere, Category = "Luminance", Meta = (EditConditionHides, EditCondition = "LuminanceType != EDMXPixelMappingLuminanceType_RGBCMY::None"))
	FDMXAttributeName LuminanceAttribute;

	/** Luminance used when Luminance Type is set to 'Set Value' */
	UPROPERTY(EditAnywhere, Category = "Luminance", Meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = 0.0, UIMax = 1.0, EditConditionHides, EditCondition = "LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::Constant"))
	float Luminance = 1.f;

	/** Min Luminance used when Luminance Type is set to 'From White' or 'From Alpha' */
	UPROPERTY(EditAnywhere, Category = "Luminance", Meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = 0.0, UIMax = 1.0, EditConditionHides, EditCondition = "LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromColor || LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromAlpha"))
	float MinLuminance = 0.f;

	/** Max Luminance used when Luminance Type is set to 'From White' or 'From Alpha' */
	UPROPERTY(EditAnywhere, Category = "Luminance", Meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = 0.0, UIMax = 1.0, EditConditionHides, EditCondition = "LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromColor || LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromAlpha"))
	float MaxLuminance = 1.f;

protected:
	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

private:
	/** Updates the ColorSpace and ColorSpaceTransform members */
	void UpdateColorSpaceAndTransform();

	/** Applies gamma correction according to the color space to the Color. */
	void EncodeGammaCorrection(EDMXPixelMappingOutputColorSpace_RGBCMY InPixelMappingOutputColorSpaceEnum, FLinearColor& InOutColor);

	/** Applies a custom gamma correction to the Color. OutColor = Pow(InColor, CustomGamma) */
	void EncodeGammaCorrection(float CustomGamma, FLinearColor& InOutColor);

	/** Encodes Rec2020 in a linear color */
	float EncodeRec2020(float InValue) const;

	/** Gets the EColorSpace enum given the current Output Color Space Enum */
	UE::Color::EColorSpace ConvertToOutputColorSpaceEnum(EDMXPixelMappingOutputColorSpace_RGBCMY InPixelMappingOutputColorSpaceEnum) const;

	/** The input color space instance */
	UE::Color::FColorSpace InputColorSpace;

	/** The output color space instance */
	UE::Color::FColorSpace OutputColorSpace;

	/** The color space transform instance */
	TUniquePtr<UE::Color::FColorSpaceTransform> ColorSpaceTransform;
};
