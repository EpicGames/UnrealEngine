// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "ColorSpace/DMXPixelMappingColorSpace.h"

#include "ColorManagement/ColorSpace.h"

#include "DMXPixelMappingColorSpace_xyY.generated.h"


UCLASS(meta = (DisplayName = "CIE 1931 xyY"))
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingColorSpace_xyY
	: public UDMXPixelMappingColorSpace
{
	GENERATED_BODY()

public:
	/** Constructor */
	UDMXPixelMappingColorSpace_xyY();

	//~ Begin DMXPixelMappingColorSpace interface
	virtual void SetRGBA(const FLinearColor& InColor) override;
	//~ End DMXPixelMappingColorSpace interface

	/**
	 * Output gamma of the Y component in xyY space whereas Y = Pow(Y, 1 / CustomGamma).
	 * CIE 1931 xyY is linear gamma so typically no gamma should be applied.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Color Space")
	float CustomGamma = 1.f;

	/** Attribute sent for x */
	UPROPERTY(EditAnywhere, Category = "XY", Meta = (DisplayName = "x Attribute"))
	FDMXAttributeName XAttribute;

	/** Attribute sent for y */
	UPROPERTY(EditAnywhere, Category = "XY", Meta = (DisplayName = "y Attribute"))
	FDMXAttributeName YAttribute;

	/** 
	 * Sets the range of the xyY color space. 
	 * A value of 1.0 corresponds to CIE 1931 xyY. 
	 * Many hardware lights use a more narrow range, typically 0.8. 
	 */
	UPROPERTY(EditAnywhere, Category = "XY", Meta = (ClampMin = 0.1, ClampMax = 2.0, UIMin = 0.1, UIMax = 1.0, DisplayName = "Color Space Range"))
	float ColorSpaceRange = .8f;

	/** Attribute sent for Y */
	UPROPERTY(EditAnywhere, Category = "Luminance")
	FDMXAttributeName LuminanceAttribute;

	/** Min Luminance */
	UPROPERTY(EditAnywhere, Category = "Luminance", Meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = 0.0, UIMax = 1.0, EditConditionHides, EditCondition = "LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromWhite || LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromAlpha"))
	float MinLuminance = 0.f;

	/** Max Luminance */
	UPROPERTY(EditAnywhere, Category = "Luminance", Meta = (ClampMin = 0.0, ClampMax = 1.0, UIMin = 0.0, UIMax = 1.0, EditConditionHides, EditCondition = "LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromWhite || LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromAlpha"))
	float MaxLuminance = 1.f;

protected:
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

private:
	/** Cached sRGB color space, to avoid instantiating on each conversion */
	UE::Color::FColorSpace SRGBColorSpace;
};
