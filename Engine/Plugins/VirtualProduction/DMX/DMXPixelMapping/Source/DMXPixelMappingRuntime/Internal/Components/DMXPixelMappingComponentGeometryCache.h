// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DMXPixelMappingComponentGeometryCache.generated.h"

class UDMXPixelMappingOutputComponent;


/** 
 * Caches pixel mapping component geometry. 
 * Useful for performance and to propagnonate changes incrementally to children.
 * 
 * The outer îs expected to be a DMXPixelMappingOutputComponent or a derived class.
 */
USTRUCT()
struct FDMXPixelMappingComponentGeometryCache
{
	GENERATED_BODY()

public:
	/** Initializes the geometry cache */
	void Initialize(UDMXPixelMappingOutputComponent* InOwner, const FVector2D& InPosition, const FVector2D& InSize, double InRotation);

	/** Sets the absoulte position, returns the translation between new and previous position. */
	FVector2D SetPositionAbsolute(const FVector2D& NewPosition);
	const FVector2D& GetPositionAbsolute() const { return Position; }

	/** Sets the absoulte position, rotated. Returns the translation between new and previous position. */
	FVector2D SetPositionRotatedAbsolute(const FVector2D& NewPositionRotated);
	FVector2D GetPositionRotatedAbsolute() const;

	/** 
	 * Sets the absoulte size, outputs the delta size and delta position. 
	 * Delta position occurs if the rectangle is rotated, as its pivot is in the center, not the top left edge.
	 * To keep it in rotated position, the position needs be updated.
	 */
	void SetSizeAbsolute(const FVector2D& InNewSize, FVector2D& OutDeltaSize, FVector2D& OutDeltaPosition);
	
	/** Sets the absolute size without outputting deltas */
	void SetSizeAbsolute(const FVector2D& InNewSize);
	const FVector2D& GetSizeAbsolute() const { return Size; }

	/** Sets the absoulte rotation, returns the delta to the previous rotation. */
	double SetRotationAbsolute(double NewRotation);
	double GetRotationAbsolute() const { return Rotation;  }

	/** Returns rotation as sin cos */
	void GetSinCos(double& OutSin, double& OutCos) const;

	/** Returns the edges of the rectangle, absolute, rotated, clockwise order */
	void GetEdgesAbsolute(FVector2D& A, FVector2D& B, FVector2D& C, FVector2D& D) const;

	/** Propagonates position changes to children. */
	void PropagonatePositionChangesToChildren(const FVector2D& Translation);

	/** Propagonates size changes to children. */
	void PropagonateSizeChangesToChildren(const FVector2D& DeltaSize, const FVector2D& DeltaPosition);

	/** Propagonates rotation changes to children. */
	void PropagonateRotationChangesToChildren(double DeltaRotation);

private:
	/** Position before it was changed */
	UPROPERTY(Transient)
	FVector2D Position = FVector2D::ZeroVector;

	/** Size before it was changed */
	UPROPERTY(Transient)
	FVector2D Size = FVector2D::ZeroVector;

	/** Rotation before it was changed */
	UPROPERTY(Transient)
	double Rotation = 0.f;

	/** Cached Sine of Rotation. Preferable over non-reflected FQuat2D etc. for undo/redo purpose. */
	UPROPERTY(Transient)
	double Sin = 0.f;

	/** Cached Cosine of Rotation. Preferable over non-reflected FQuat2D etc. for undo/redo purpose. */ 
	UPROPERTY(Transient)
	double Cos = 1.f;

	/** The owner of this cache */
	TWeakObjectPtr<UDMXPixelMappingOutputComponent> WeakOwner;
};
