// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "DMXPixelMappingActor.generated.h"

class UDMXPixelMapping;
class USceneComponent;
enum class EDMXPixelMappingResetDMXMode : uint8;

/** Actor class for DMX Pixel Mapping */
UCLASS(NotBlueprintable)
class DMXPIXELMAPPINGRUNTIME_API ADMXPixelMappingActor
	: public AActor
{
	GENERATED_BODY()

public:
	/** Constructor */
	ADMXPixelMappingActor();

	/** Sets the Pixel Mapping used in this actor */
	void SetPixelMapping(UDMXPixelMapping* InPixelMapping);

	/** Starts sending DMX */
	UFUNCTION(BlueprintCallable, Category = "DMX Pixel Mapping")
	void StartSendingDMX();

	/** Stop sending DMX. */
	UFUNCTION(BlueprintCallable, Category = "DMX Pixel Mapping")
	void StopSendingDMX();

	/** Pause sending DMX */
	UFUNCTION(BlueprintCallable, Category = "DMX Pixel Mapping")
	void PauseSendingDMX();

	/** Returns true if the asset is playing DMX */
	UFUNCTION(BlueprintPure, Category = "DMX Pixel Mapping")
	bool IsSendingDMX() const;

	/** Sets how the pixel mapping asset resets the channels it sends to when Stop Sending DMX is called */
	UFUNCTION(BlueprintCallable, Category = "DMX Pixel Mapping")
	void SetStopMode(EDMXPixelMappingResetDMXMode ResetMode);

protected:
	//~ Begin AActor interface
	virtual void PostLoad() override;
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End AActor interface

private:
#if WITH_EDITOR
	/** Applies the play in editor state. Enables send when auto activate is on, send DMX in editor is enabled and not currently playing */
	void ApplySendDMXInEditorState();
#endif // WITH_EDITOR

	/** The pixel mapping used in this actor */
	UPROPERTY(VisibleAnywhere, Category = "DMX Pixel Mapping")
	TObjectPtr<UDMXPixelMapping> PixelMapping;

	/**  
	 * True if the pixel mapping actor auto activates.
	 * If Send DMX in Editor is disabled, starts to send DMX on begin play.
	 * If Send DMX in Editor is enabled, it will start to send DMX when the level is loaded in editor.
	 */
	UPROPERTY(EditAnywhere, Category = "DMX Pixel Mapping")
	bool bAutoActivate = true;

#if WITH_EDITORONLY_DATA
	/** True if the pixel mapping should send DMX data in editor */
	UPROPERTY(EditAnywhere, Category = "DMX Pixel Mapping", Meta = (DisplayName = "Send DMX in Editor"))
	bool bSendDMXInEditor = false;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** True while the actor plays in a world */
	bool bIsPlayInWorld = false;
#endif // WITH_EDITOR

	/** Scene component to make the actor easily visible in Editor */
	UPROPERTY(VisibleAnywhere, Category = "Actor", AdvancedDisplay, Meta = (AllowPrivateAccess = true))
	TObjectPtr<USceneComponent> RootSceneComponent;
};
