// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"


// Custom serialization version for changes to DMX Pixel Mapping Objects in the Main Stream
struct DMXPIXELMAPPINGRUNTIME_API FDMXPixelMappingMainStreamObjectVersion
{
	enum Type
	{
		// Roughly corresponds to 4.26
		BeforeCustomVersionWasAdded = 0,

		// Update DMXPixelMappingMatrixComponent to no longer use a custom MatrixFixturePatchRef, but the default EntityFixturePatchRef one.
		ChangePixelMappingMatrixComponentToFixturePatchReference,

		// Update DMXPixelMappingBaseComponent to store the parent as a weak ptr
		UseWeakPtrForPixelMappingComponentParent,

		// Update DMXPixelMappingRendererComponent to lock those components that use a texture in designer to trigger the edit condition of the size property
		LockRendererComponentsThatUseTextureInDesigner,

		// Update DMXPixelMappingFixtureGroupItemComponent and DMXPixelMappingMatrixCellComponent to use DMXPixelMappingColorSpace
		UseDMXPixelMappingColorSpace,

		// The renderer component now holds the layout rect, so its children can be layout out relative to the current texture size
		RendererComponentHoldsLayoutRect,

		// 5.5: Color spaces now apply a gamma correction
		DMXOutputAppliesGammaCorrection,

		// 5.6: Each pixel mapping holds its own reset DMX mode
		PerPixelMappingResetDMXMode,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FDMXPixelMappingMainStreamObjectVersion() {}
};
