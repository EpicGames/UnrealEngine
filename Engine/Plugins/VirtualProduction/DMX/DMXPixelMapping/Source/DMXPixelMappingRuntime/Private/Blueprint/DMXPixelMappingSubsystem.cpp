// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/DMXPixelMappingSubsystem.h"
#include "DMXPixelMapping.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "Components/DMXPixelMappingOutputComponent.h"
#include "Components/DMXPixelMappingOutputDMXComponent.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"

#include "Engine/Engine.h"


/*static*/ UDMXPixelMappingSubsystem* UDMXPixelMappingSubsystem::GetDMXPixelMappingSubsystem_Pure()
{
	return GEngine->GetEngineSubsystem<UDMXPixelMappingSubsystem>();
}

/*static*/ UDMXPixelMappingSubsystem* UDMXPixelMappingSubsystem::GetDMXPixelMappingSubsystem_Callable()
{
	return GetDMXPixelMappingSubsystem_Pure();
}

UDMXPixelMapping* UDMXPixelMappingSubsystem::GetDMXPixelMapping(UDMXPixelMapping* InPixelMapping)
{
	return InPixelMapping;
}

UDMXPixelMappingRendererComponent* UDMXPixelMappingSubsystem::GetRendererComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName)
{
	UDMXPixelMappingRootComponent* RootComponent = InDMXPixelMapping ? InDMXPixelMapping->GetRootComponent() : nullptr;
	if (RootComponent)
	{
		// Use the specialized method here that is faster than the generic UDMXPixelMappingBaseComponent::FindComponentOfClass
		return RootComponent->FindRendererComponentByName(InComponentName);
	}
	return nullptr;
}

UDMXPixelMappingOutputDMXComponent* UDMXPixelMappingSubsystem::GetOutputDMXComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName)
{
	if (InDMXPixelMapping != nullptr)
	{
		return InDMXPixelMapping->FindComponentOfClass<UDMXPixelMappingOutputDMXComponent>(InComponentName);
	}
	return nullptr;
}

UDMXPixelMappingFixtureGroupComponent* UDMXPixelMappingSubsystem::GetFixtureGroupComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName)
{
	if (InDMXPixelMapping != nullptr)
	{
		return InDMXPixelMapping->FindComponentOfClass<UDMXPixelMappingFixtureGroupComponent>(InComponentName);
	}
	return nullptr;
}

UDMXPixelMappingMatrixComponent* UDMXPixelMappingSubsystem::GetMatrixComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName)
{
	if (InDMXPixelMapping != nullptr)
	{
		return InDMXPixelMapping->FindComponentOfClass<UDMXPixelMappingMatrixComponent>(InComponentName);
	}
	return nullptr;
}
