// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorSpace/DMXPixelMappingColorSpace.h"

#include "ColorManagement/ColorSpace.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"


void UDMXPixelMappingColorSpace::ResetToBlack()
{
	for (TTuple<FDMXAttributeName, float>& AttributeNameToValuePair : AttributeNameToValueMap)
	{
		AttributeNameToValuePair.Value = 0.f;
	}
}

#if WITH_EDITOR
void UDMXPixelMappingColorSpace::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnPostEditChangeProperty.Broadcast(PropertyChangedEvent);
}
#endif // WITH_EDITOR

void UDMXPixelMappingColorSpace::SetAttributeValue(const FDMXAttributeName& AttributeName, float Value)
{
	AttributeNameToValueMap.FindOrAdd(AttributeName) = Value;
}

void UDMXPixelMappingColorSpace::ClearCachedAttributeValues()
{
	AttributeNameToValueMap.Reset();
}
