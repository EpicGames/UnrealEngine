// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorSpace/DMXPixelMappingColorSpace_RGBCMY.h"

#include "ColorManagement/TransferFunctions.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"

UDMXPixelMappingColorSpace_RGBCMY::UDMXPixelMappingColorSpace_RGBCMY()
	: RedAttribute("Red")
	, GreenAttribute("Green")
	, BlueAttribute("Blue")
	, LuminanceAttribute("Dimmer")
{}

void UDMXPixelMappingColorSpace_RGBCMY::SetRGBA(const FLinearColor& InColor)
{
	using namespace UE::Color;

	if (bUseWorkingColorSpaceForInput && !InputColorSpace.Equals(UE::Color::FColorSpace::GetWorking()))
	{
		// Update in case the working color space changed
		UpdateColorSpaceAndTransform();
	}

	FLinearColor CalibratedColor = ColorSpaceTransform->Apply(InColor);

	// Apply gamma
	if (OutputGamma == EDMXPixelMappingGamma_RGBCMY::AsOutputColorSpace)
	{
		const EColorSpace SelectedOutputColorSpace = ConvertToOutputColorSpaceEnum(PixelMappingOutputColorSpace);
		EncodeGammaCorrection(PixelMappingOutputColorSpace, CalibratedColor);
	}
	else if (OutputGamma == EDMXPixelMappingGamma_RGBCMY::Custom)
	{
		EncodeGammaCorrection(CustomGamma, CalibratedColor);
	}

	// Buffer DMX values
	if (RedAttribute.IsValid())
	{
		const float Value = bSendCyan ? FMath::Abs(CalibratedColor.R - 1.f) : CalibratedColor.R;
		SetAttributeValue(RedAttribute, Value);
	}

	if (GreenAttribute.IsValid())
	{
		const float Value = bSendMagenta ? FMath::Abs(CalibratedColor.G - 1.f) : CalibratedColor.G;
		SetAttributeValue(GreenAttribute, Value);
	}

	if (BlueAttribute.IsValid())
	{
		const float Value = bSendYellow ? FMath::Abs(CalibratedColor.B - 1.f) : CalibratedColor.B;
		SetAttributeValue(BlueAttribute, Value);
	}

	if (LuminanceAttribute.IsValid())
	{
		if (LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromColor)
		{
			const float LuminanceFromColor = OutputColorSpace.GetLuminance(CalibratedColor);
			SetAttributeValue(LuminanceAttribute, FMath::Clamp(LuminanceFromColor, MinLuminance, MaxLuminance));
		}
		else if (LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::Constant)
		{
			SetAttributeValue(LuminanceAttribute, FMath::Clamp(Luminance, MinLuminance, MaxLuminance));
		}
		else if (LuminanceType == EDMXPixelMappingLuminanceType_RGBCMY::FromAlpha)
		{
			SetAttributeValue(LuminanceAttribute, FMath::Clamp(InColor.A, MinLuminance, MaxLuminance));
		}
	}
}

void UDMXPixelMappingColorSpace_RGBCMY::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		// Assets created before 5.5 used linear gamma
		if (Ar.CustomVer(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::DMXOutputAppliesGammaCorrection)
		{
			OutputGamma = EDMXPixelMappingGamma_RGBCMY::Linear;
		}
	}
#endif
}

void UDMXPixelMappingColorSpace_RGBCMY::PostLoad()
{
	Super::PostLoad();

	UpdateColorSpaceAndTransform();
}

#if WITH_EDITOR
void UDMXPixelMappingColorSpace_RGBCMY::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(FDMXAttributeName, Name) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, LuminanceType) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, LuminanceAttribute))
	{
		ClearCachedAttributeValues();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, PixelMappingOutputColorSpace) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace, bUseWorkingColorSpaceForInput))
	{
		UpdateColorSpaceAndTransform();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, MinLuminance))
	{
		if (MaxLuminance < MinLuminance)
		{
			Modify();
			MaxLuminance = MinLuminance;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_RGBCMY, MaxLuminance))
	{
		if (MinLuminance > MaxLuminance)
		{
			Modify();
			MinLuminance = MaxLuminance;
		}
	}
}
#endif // WITH_EDITOR

void UDMXPixelMappingColorSpace_RGBCMY::UpdateColorSpaceAndTransform()
{
	using namespace UE::Color;

	InputColorSpace = bUseWorkingColorSpaceForInput ? FColorSpace::GetWorking() : FColorSpace(EColorSpace::sRGB);

	const EColorSpace SelectedOutputColorSpace = ConvertToOutputColorSpaceEnum(PixelMappingOutputColorSpace);
	OutputColorSpace = FColorSpace(SelectedOutputColorSpace);

	ColorSpaceTransform = MakeUnique<FColorSpaceTransform>(InputColorSpace, OutputColorSpace);
}

UE::Color::EColorSpace UDMXPixelMappingColorSpace_RGBCMY::ConvertToOutputColorSpaceEnum(EDMXPixelMappingOutputColorSpace_RGBCMY InPixelMappingOutputColorSpaceEnum) const
{
	using namespace UE::Color;
	if (InPixelMappingOutputColorSpaceEnum == EDMXPixelMappingOutputColorSpace_RGBCMY::sRGB)
	{
		return EColorSpace::sRGB;
	}
	else if (InPixelMappingOutputColorSpaceEnum == EDMXPixelMappingOutputColorSpace_RGBCMY::Plasa)
	{
		return EColorSpace::PLASA_E1_54;
	}
	else if (InPixelMappingOutputColorSpaceEnum == EDMXPixelMappingOutputColorSpace_RGBCMY::Rec2020)
	{
		return EColorSpace::Rec2020;
	}
	else if (InPixelMappingOutputColorSpaceEnum == EDMXPixelMappingOutputColorSpace_RGBCMY::P3DCI)
	{
		return EColorSpace::P3DCI;
	}
	else if (InPixelMappingOutputColorSpaceEnum == EDMXPixelMappingOutputColorSpace_RGBCMY::P3D65)
	{
		return EColorSpace::P3D65;
	}

	return EColorSpace::None;
}

void UDMXPixelMappingColorSpace_RGBCMY::EncodeGammaCorrection(EDMXPixelMappingOutputColorSpace_RGBCMY InPixelMappingOutputColorSpaceEnum, FLinearColor& InOutColor)
{
	using namespace UE::Color;

	switch (InPixelMappingOutputColorSpaceEnum)
	{
	case EDMXPixelMappingOutputColorSpace_RGBCMY::P3D65:
		// Same as sRGB, fall through

	case EDMXPixelMappingOutputColorSpace_RGBCMY::sRGB:
		InOutColor.R = EncodeSRGB(InOutColor.R);
		InOutColor.G = EncodeSRGB(InOutColor.G);
		InOutColor.B = EncodeSRGB(InOutColor.B);
		break;

	case EDMXPixelMappingOutputColorSpace_RGBCMY::Rec2020:
		InOutColor.R = EncodeRec2020(InOutColor.R);
		InOutColor.G = EncodeRec2020(InOutColor.G);
		InOutColor.B = EncodeRec2020(InOutColor.B);
		break;

	case EDMXPixelMappingOutputColorSpace_RGBCMY::P3DCI:
		InOutColor.R = EncodeGamma26(InOutColor.R);
		InOutColor.G = EncodeGamma26(InOutColor.G);
		InOutColor.B = EncodeGamma26(InOutColor.B);
		break;

	case EDMXPixelMappingOutputColorSpace_RGBCMY::Plasa:
		// PLASA ANSI E1.54 is linear
		break;

	default:
		ensureMsgf(0, TEXT("Missing transfer function for color space. Cannot apply gamma correction."));
	}
}

void UDMXPixelMappingColorSpace_RGBCMY::EncodeGammaCorrection(float InCustomGamma, FLinearColor& InOutColor)
{
	InOutColor.R = FMath::Pow(InOutColor.R, 1.f / InCustomGamma);
	InOutColor.G = FMath::Pow(InOutColor.G, 1.f / InCustomGamma);
	InOutColor.B = FMath::Pow(InOutColor.B, 1.f / InCustomGamma);
}

float UDMXPixelMappingColorSpace_RGBCMY::EncodeRec2020(float InValue) const
{
	// Values are expected to be in the 0-1 range
	const float ClampedValue = FMath::Clamp(InValue, 0.f, 1.f);

	constexpr float Beta = 0.018053968510807f;
	constexpr float Alpha = 1 + 5.5f * Beta;

	if (ClampedValue < Beta)
	{
		return 4.5f * ClampedValue;
	}
	else
	{
		return Alpha * FMath::Pow(ClampedValue, 0.45f) - (Alpha - 1.f);
	}
}

