// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorSpace/DMXPixelMappingColorSpace_XYZ.h"


UDMXPixelMappingColorSpace_XYZ::UDMXPixelMappingColorSpace_XYZ()
	: XAttribute("X")
	, ZAttribute("Z")
	, LuminanceAttribute("Dimmer")
{}

void UDMXPixelMappingColorSpace_XYZ::SetRGBA(const FLinearColor& InColor)
{
	const UE::Color::FColorSpace& InputColorSpace = bUseWorkingColorSpaceForInput ?
		UE::Color::FColorSpace::GetWorking() :
		SRGBColorSpace;

	// Convert RGB to CIEXYZ
	const FMatrix44d Matrix = InputColorSpace.GetRgbToXYZ();
	FVector4 XYZW = Matrix.TransformVector(FVector(InColor));
	
	// Apply gamma to Y (the Z-Component of the vector)
	if (!FMath::IsNearlyEqual(CustomGamma, 1.f))
	{
		FVector3d xyY = UE::Color::XYZToxyY(XYZW);
		xyY.Z = FMath::Pow(xyY.Z, 1.f / CustomGamma);
		
		XYZW = UE::Color::xyYToXYZ(xyY);
	}

	// Buffer DMX values
	if (XAttribute.IsValid())
	{
		SetAttributeValue(XAttribute, XYZW.X);
	}

	if (LuminanceAttribute.IsValid())
	{
		const float Luminance = FMath::Clamp(XYZW.Y, MinLuminance, MaxLuminance);
		SetAttributeValue(LuminanceAttribute, Luminance);
	}

	if (ZAttribute.IsValid())
	{
		SetAttributeValue(ZAttribute, XYZW.Z);
	}
}

#if WITH_EDITOR
void UDMXPixelMappingColorSpace_XYZ::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_XYZ, XAttribute) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_XYZ, LuminanceAttribute) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_XYZ, ZAttribute))
	{
		ClearCachedAttributeValues();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_XYZ, MinLuminance))
	{
		if (MaxLuminance < MinLuminance)
		{
			Modify();
			MaxLuminance = MinLuminance;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_XYZ, MaxLuminance))
	{
		if (MinLuminance > MaxLuminance)
		{
			Modify();
			MinLuminance = MaxLuminance;
		}
	}
}
#endif // WITH_EDITOR
