// Copyright Epic Games, Inc. All Rights Reserved.

#include "ColorSpace/DMXPixelMappingColorSpace_xyY.h"


UDMXPixelMappingColorSpace_xyY::UDMXPixelMappingColorSpace_xyY()
	: XAttribute("CIE_X")
	, YAttribute("CIE_Y")
	, LuminanceAttribute("Dimmer")
	, SRGBColorSpace(UE::Color::EColorSpace::sRGB)
{}

void UDMXPixelMappingColorSpace_xyY::SetRGBA(const FLinearColor& InColor)
{
	const UE::Color::FColorSpace& InputColorSpace = bUseWorkingColorSpaceForInput ?
		UE::Color::FColorSpace::GetWorking() :
		SRGBColorSpace;

	// Convert RGB to CIE XYZ
	const FMatrix44d& Matrix = InputColorSpace.GetRgbToXYZ();
	const FVector4 XYZW = Matrix.TransformVector(FVector(InColor));
	FVector3d xyY = UE::Color::XYZToxyY(XYZW);

	// Apply gamma to Y (the Z-Component of the vector)
	if (!FMath::IsNearlyEqual(CustomGamma, 1.f))
	{
		xyY.Z = FMath::Pow(xyY.Z, 1.f / CustomGamma);
	}

	// Buffer DMX values
	if (!ensureMsgf(ColorSpaceRange != 0.0, TEXT("Coversion in PixelMapping Color Space xyY failed. Color space range is 0.")))
	{
		return;
	}

	if (XAttribute.IsValid())
	{
		const double x = xyY[0] / ColorSpaceRange;
		SetAttributeValue(XAttribute, x);
	}

	if (YAttribute.IsValid())
	{
		const double y = xyY[1] / ColorSpaceRange;
		SetAttributeValue(YAttribute, y);
	}

	if (LuminanceAttribute.IsValid())
	{
		const double Luminance = FMath::Clamp(xyY[2], MinLuminance, MaxLuminance);
		SetAttributeValue(LuminanceAttribute, Luminance);
	}
}

#if WITH_EDITOR
void UDMXPixelMappingColorSpace_xyY::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_xyY, XAttribute) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_xyY, YAttribute) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_xyY, LuminanceAttribute))
	{
		ClearCachedAttributeValues();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_xyY, MinLuminance))
	{
		if (MaxLuminance < MinLuminance)
		{
			Modify();
			MaxLuminance = MinLuminance;
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingColorSpace_xyY, MaxLuminance))
	{
		if (MinLuminance > MaxLuminance)
		{
			Modify();
			MinLuminance = MaxLuminance;
		}
	}
}
#endif // WITH_EDITOR
