// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingBaseComponent.h"

#include "DMXPixelMapping.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"


UDMXPixelMappingBaseComponent::FDMXPixelMappingOnComponentAdded UDMXPixelMappingBaseComponent::OnComponentAdded;
UDMXPixelMappingBaseComponent::FDMXPixelMappingOnComponentRemoved UDMXPixelMappingBaseComponent::OnComponentRemoved;
UDMXPixelMappingBaseComponent::FDMXPixelMappingOnComponentRenamed UDMXPixelMappingBaseComponent::OnComponentRenamed;


UDMXPixelMappingBaseComponent::FDMXPixelMappingOnComponentAdded& UDMXPixelMappingBaseComponent::UDMXPixelMappingBaseComponent::GetOnComponentAdded()
{
	return OnComponentAdded;
}

UDMXPixelMappingBaseComponent::FDMXPixelMappingOnComponentRemoved& UDMXPixelMappingBaseComponent::GetOnComponentRemoved()
{
	return OnComponentRemoved;
}

UDMXPixelMappingBaseComponent::FDMXPixelMappingOnComponentRenamed& UDMXPixelMappingBaseComponent::GetOnComponentRenamed()
{
	return OnComponentRenamed;
}

void UDMXPixelMappingBaseComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::UseWeakPtrForPixelMappingComponentParent)
		{
			// Upgrade from strong object references for parent to weak ones
			WeakParent = Parent_DEPRECATED;

			Parent_DEPRECATED = nullptr;
		}
	}
}

void UDMXPixelMappingBaseComponent::PostRename(UObject* OldOuter, const FName OldName) 
{
	Super::PostRename(OldOuter, OldName);

	OnComponentRenamed.Broadcast(this);
}

#if WITH_EDITOR
void UDMXPixelMappingBaseComponent::PostEditUndo()
{
	Super::PostEditUndo();

	if (HasValidParent())
	{
		if(GetParent()->Children.Contains(this))
		{
			GetParent()->AddChild(this);
		}
		else
		{
			GetParent()->RemoveChild(this);
		}
	}
}
#endif // WITH_EDITOR

const FName& UDMXPixelMappingBaseComponent::GetNamePrefix()
{
	ensureMsgf(false, TEXT("You must implement GetNamePrefix() in your child class"));

	static FName NamePrefix = TEXT("");
	return NamePrefix;
}

int32 UDMXPixelMappingBaseComponent::GetChildrenCount() const
{
	return Children.Num();
}

void UDMXPixelMappingBaseComponent::ForEachChild(TComponentPredicate Predicate, bool bIsRecursive)
{
	for (int32 ChildIdx = 0; ChildIdx < GetChildrenCount(); ChildIdx++)
	{
		if (UDMXPixelMappingBaseComponent* ChildComponent = GetChildAt(ChildIdx))
		{
			Predicate(ChildComponent);

			if (bIsRecursive)	
			{
				ForComponentAndChildren(ChildComponent, Predicate);
			}
		}
	}
}

UDMXPixelMapping* UDMXPixelMappingBaseComponent::GetPixelMapping() const
{
	if (const UDMXPixelMappingRootComponent* RootComponent = GetRootComponent())
	{
		return Cast<UDMXPixelMapping>(RootComponent->GetOuter());
	}

	return nullptr;
}

const UDMXPixelMappingRootComponent* UDMXPixelMappingBaseComponent::GetRootComponent() const
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return nullptr;
	}

	// If this is the Root Component
	if (const UDMXPixelMappingRootComponent* ThisRootComponent = Cast<UDMXPixelMappingRootComponent>(this))
	{
		return ThisRootComponent;
	}
	// Try to get a root component from object owner
	else
	{
		for (UDMXPixelMappingBaseComponent* Parent = GetParent(); Parent; Parent = Parent->GetParent())
		{
			if (const UDMXPixelMappingRootComponent* Root = Cast<UDMXPixelMappingRootComponent>(Parent))
	{
				return Root;
			}
		}
	}
	
		return nullptr;
	}

const UDMXPixelMappingRootComponent* UDMXPixelMappingBaseComponent::GetRootComponentChecked() const
{
	const UDMXPixelMappingRootComponent* RootComponent = GetRootComponent();
	check(RootComponent);
	return RootComponent;
}

UDMXPixelMappingRendererComponent* UDMXPixelMappingBaseComponent::GetRendererComponent()
{
	UDMXPixelMappingRendererComponent* RendererComponent = Cast<UDMXPixelMappingRendererComponent>(this);
	if (RendererComponent == nullptr)
	{
		RendererComponent = GetFirstParentByClass<UDMXPixelMappingRendererComponent>(this);
	}

	return RendererComponent;
}

void UDMXPixelMappingBaseComponent::ForComponentAndChildren(UDMXPixelMappingBaseComponent* Component, TComponentPredicate Predicate)
{
	if (Component != nullptr)
	{
		for (int32 ChildIdx = 0; ChildIdx < Component->GetChildrenCount(); ChildIdx++)
		{
			if (UDMXPixelMappingBaseComponent* ChildComponent = Component->GetChildAt(ChildIdx))
			{
				Predicate(ChildComponent);

				ForComponentAndChildren(ChildComponent, Predicate);
			}
		}
	}
}

UDMXPixelMappingBaseComponent* UDMXPixelMappingBaseComponent::GetChildAt(int32 InIndex) const
{
	if (Children.IsValidIndex(InIndex))
	{
		return Children[InIndex];
	}

	return nullptr;
}

void UDMXPixelMappingBaseComponent::AddChild(UDMXPixelMappingBaseComponent* InComponent)
{
#if WITH_EDITOR
	ensureMsgf(InComponent, TEXT("Trying to add nullptr to %s"), *GetUserName());
#endif 

	if (InComponent)
	{
		InComponent->WeakParent = this;

		// Allow children to be readded, this may be the case during Undo of a Remove
		Children.AddUnique(InComponent);
		InComponent->NotifyAddedToParent();

		// Broadcast the change
		OnComponentAdded.Broadcast(GetPixelMapping(), InComponent);
	}
}

void UDMXPixelMappingBaseComponent::RemoveChild(UDMXPixelMappingBaseComponent* ChildComponent)
{
	if (ChildComponent)
	{
		ChildComponent->ResetDMX(EDMXPixelMappingResetDMXMode::SendDefaultValues);

		Children.Remove(ChildComponent);
		ChildComponent->NotifyRemovedFromParent();

		ChildComponent->SetFlags(RF_Transactional);

		OnComponentRemoved.Broadcast(GetPixelMapping(), ChildComponent);
	}

	ensure(ChildComponent && !Children.Contains(ChildComponent) && ChildComponent->Children.Num() == 0 && !ChildComponent->Parent_DEPRECATED);
}

void UDMXPixelMappingBaseComponent::ClearChildren()
{
	for (UDMXPixelMappingBaseComponent* Component : TArray<UDMXPixelMappingBaseComponent*>(Children))
	{
		RemoveChild(Component);
	}
}

FString UDMXPixelMappingBaseComponent::GetUserFriendlyName() const
{
	return GetUserName();
}

FString UDMXPixelMappingBaseComponent::GetUserName() const
{
	return UserName.IsEmpty() ? GetName() : UserName;
}

void UDMXPixelMappingBaseComponent::SetUserName(const FString& NewName)
{
	UserName = NewName;
}

void UDMXPixelMappingBaseComponent::GetChildComponentsRecursively(TArray<UDMXPixelMappingBaseComponent*>& Components)
{
	ForComponentAndChildren(this, [&Components](UDMXPixelMappingBaseComponent* InComponent) {
		Components.Add(InComponent);
	});
}

void UDMXPixelMappingBaseComponent::NotifyAddedToParent()
{
	// Nothing in base
}

void UDMXPixelMappingBaseComponent::NotifyRemovedFromParent()
{
	for (UDMXPixelMappingBaseComponent* Child : TArray< UDMXPixelMappingBaseComponent*>(Children))
	{
		RemoveChild(Child);
	}

	Children.Reset();
}
