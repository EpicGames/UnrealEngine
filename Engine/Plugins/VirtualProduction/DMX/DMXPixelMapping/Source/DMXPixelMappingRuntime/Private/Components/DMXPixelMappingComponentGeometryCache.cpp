// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingComponentGeometryCache.h"

#include "Components/DMXPixelMappingOutputComponent.h"


namespace UE::DMX::PixelMapping::ComponentGeometryCache::Private
{
	FVector2D RotateVector(const FVector2D& Vector, double Sin, double Cos, const FVector2D& Pivot)
	{
		const FVector2d PivotToVector = Vector - Pivot;
		const FVector2d PivotToVectorRotated = FVector2D(
			Cos * PivotToVector.X - Sin * PivotToVector.Y,
			Sin * PivotToVector.X + Cos * PivotToVector.Y);

		return Pivot + PivotToVectorRotated;
	}
}

void FDMXPixelMappingComponentGeometryCache::Initialize(UDMXPixelMappingOutputComponent* InOwner, const FVector2D& InPosition, const FVector2D& InSize, double InRotation)
{
	WeakOwner = InOwner;
	Position = InPosition;
	Size = InSize;
	Rotation = InRotation;
}

FVector2D FDMXPixelMappingComponentGeometryCache::SetPositionAbsolute(const FVector2D& NewPosition)
{
	const FVector2D Delta = NewPosition - Position;

	Position = NewPosition;

	return Delta;
}

FVector2D FDMXPixelMappingComponentGeometryCache::SetPositionRotatedAbsolute(const FVector2D& NewPositionRotated)
{
	const FVector2D OldPosition = Position;
	const FVector2D Translation = NewPositionRotated - GetPositionRotatedAbsolute();

	Position += Translation;

	const FVector2D Delta = Position - OldPosition;
	return Delta;
}

FVector2D FDMXPixelMappingComponentGeometryCache::GetPositionRotatedAbsolute() const
{
	const FVector2D Pivot = Position + Size / 2.0;

	using namespace UE::DMX::PixelMapping::ComponentGeometryCache::Private;
	return RotateVector(Position, Sin, Cos, Pivot);
}

void FDMXPixelMappingComponentGeometryCache::SetSizeAbsolute(const FVector2D& InNewSize, FVector2D& OutDeltaSize, FVector2D& OutDeltaPosition)
{
	const FVector2D Pivot = Position + Size / 2.0;

	using namespace UE::DMX::PixelMapping::ComponentGeometryCache::Private;
	OutDeltaPosition = RotateVector(Position, Sin, Cos, Position + Size / 2.f) - RotateVector(Position, Sin, Cos, Position + InNewSize / 2.f);
	OutDeltaSize = InNewSize - Size;

	Size = InNewSize;
	Position += OutDeltaPosition;
}

void FDMXPixelMappingComponentGeometryCache::SetSizeAbsolute(const FVector2D& InNewSize)
{
	using namespace UE::DMX::PixelMapping::ComponentGeometryCache::Private;
	const FVector2D DeltaPosition = RotateVector(Position, Sin, Cos, Position + Size / 2.f) - RotateVector(Position, Sin, Cos, Position + InNewSize / 2.f);

	Size = InNewSize;
	Position += DeltaPosition;
}

double FDMXPixelMappingComponentGeometryCache::SetRotationAbsolute(double NewRotation)
{
	const double Delta = NewRotation - Rotation;

	Rotation = NewRotation;

	// Cache rotation as sin cos
	FMath::SinCos(&Sin, &Cos, FMath::DegreesToRadians(Rotation));

	return Delta;
}

void FDMXPixelMappingComponentGeometryCache::GetSinCos(double& OutSin, double& OutCos) const
{
	OutSin = Sin;
	OutCos = Cos;
}

void FDMXPixelMappingComponentGeometryCache::GetEdgesAbsolute(FVector2D& A, FVector2D& B, FVector2D& C, FVector2D& D) const
{
	const FVector2D Pivot = Position + Size / 2.0;

	using namespace UE::DMX::PixelMapping::ComponentGeometryCache::Private;
	A = RotateVector(Position, Sin, Cos, Pivot);
	B = RotateVector(FVector2D(Position.X + Size.X, Position.Y), Sin, Cos, Pivot);
	C = RotateVector(Position + Size, Sin, Cos, Pivot);
	D = RotateVector(FVector2D(Position.X, Position.Y + Size.Y), Sin, Cos, Pivot);
}

void FDMXPixelMappingComponentGeometryCache::PropagonatePositionChangesToChildren(const FVector2D& Translation)
{
	if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(WeakOwner.Get()))
	{
		constexpr bool bRecursive = false;
		OutputComponent->ForEachChildOfClass<UDMXPixelMappingOutputComponent>(
			[Translation](UDMXPixelMappingOutputComponent* Child)
			{
				Child->Modify();
				Child->SetPosition(Child->GetPosition() + Translation);
			},
			bRecursive);
	}
}

void FDMXPixelMappingComponentGeometryCache::PropagonateSizeChangesToChildren(const FVector2D& DeltaSize, const FVector2D& DeltaPosition)
{
	if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(WeakOwner.Get()))
	{
		const double RestoreRotation = Rotation;
		OutputComponent->SetRotation(0.0);

		const FVector2D Scalar = Size / (Size - DeltaSize);

		constexpr bool bRecursive = false;
		OutputComponent->ForEachChildOfClass<UDMXPixelMappingOutputComponent>(
			[Scalar, this](UDMXPixelMappingOutputComponent* Child)
			{
				const FVector2D OldChildPosition = Child->GetPosition();
				const FVector2D OldChildSize = Child->GetSize();

				Child->Modify();
				Child->SetSize(Child->GetSize() * Scalar);

				// Reposition after setting the size, as set size and changes the center.
				const FVector2D OldChildCenterRelative = OldChildPosition + OldChildSize / 2.0 - Position;
				const FVector2D NewChildCenterRelative = OldChildCenterRelative * Scalar;
				Child->SetPosition(Position + NewChildCenterRelative - Child->GetSize() / 2.0);
			},
			bRecursive);

		OutputComponent->SetRotation(RestoreRotation);
	}
}

void FDMXPixelMappingComponentGeometryCache::PropagonateRotationChangesToChildren(double DeltaRotation)
{
	if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(WeakOwner.Get()))
	{
		constexpr bool bRecursive = false;
		OutputComponent->ForEachChildOfClass<UDMXPixelMappingOutputComponent>(
			[DeltaRotation, this](UDMXPixelMappingOutputComponent* Child)
			{
				Child->Modify();
				Child->SetRotation(Child->GetRotation() + DeltaRotation);

				// Rotate center around parent pivot
				const FVector2D ParentPivotAbsolute = Position + Size / 2.0;
				const FVector2D OldCenterRelative = Child->GetPosition() - ParentPivotAbsolute + Child->GetSize() / 2.0;
				const FVector2D NewCenterRelative = OldCenterRelative.GetRotated(DeltaRotation);
				Child->SetPosition(ParentPivotAbsolute + NewCenterRelative - Child->GetSize() / 2.0);
			},
			bRecursive);
	}
}
