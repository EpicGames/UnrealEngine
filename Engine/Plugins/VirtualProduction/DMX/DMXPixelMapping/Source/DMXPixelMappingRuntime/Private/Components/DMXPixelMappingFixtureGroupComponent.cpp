// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingFixtureGroupComponent.h"

#include "Components/DMXPixelMappingComponentGeometryCache.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMapping.h"
#include "Library/DMXLibrary.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingFixtureGroupComponent"

UDMXPixelMappingFixtureGroupComponent::UDMXPixelMappingFixtureGroupComponent()
{
	SizeX = 500.f;
	SizeY = 500.f;
}

#if WITH_EDITOR
void UDMXPixelMappingFixtureGroupComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Call the parent at the first place
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupComponent, DMXLibrary))
	{
		OnDMXLibraryChangedDelegate.Broadcast();
	}

	InvalidatePixelMapRenderer();
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXPixelMappingFixtureGroupComponent::PostEditUndo()
{
	Super::PostEditUndo();

	// Restore Matrices 
	for (UDMXPixelMappingBaseComponent* Child : Children)
	{
		if (UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(Child))
		{
			MatrixComponent->HandleMatrixChanged();
		}
	}
}
#endif // WITH_EDITOR

const FName& UDMXPixelMappingFixtureGroupComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("Fixture Group");
	return NamePrefix;
}

void UDMXPixelMappingFixtureGroupComponent::AddChild(UDMXPixelMappingBaseComponent* InComponent)
{
	Super::AddChild(InComponent);

	// Make sure newly added childs are meaningfully smaller than this group
	if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(InComponent))
	{
		const FVector2D MySize = GetSize();
		const FVector2D ChildSize = OutputComponent->GetSize();
		if (MySize.X < ChildSize.X || MySize.Y < ChildSize.Y)
		{
			FVector2D DesiredChildSize = MySize / 2.f;
			DesiredChildSize.X = FMath::Max(1.f, DesiredChildSize.X);
			DesiredChildSize.Y = FMath::Max(1.f, DesiredChildSize.Y);

			OutputComponent->SetSize(DesiredChildSize);
		}
	}

	// Update Matrices once they were added
	if (UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(InComponent))
	{
		MatrixComponent->HandleMatrixChanged();
	}
}

void UDMXPixelMappingFixtureGroupComponent::ResetDMX(EDMXPixelMappingResetDMXMode ResetMode)
{
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent)
	{
		if (UDMXPixelMappingOutputComponent * Component = Cast<UDMXPixelMappingOutputComponent>(InComponent))
		{
			Component->ResetDMX(ResetMode);
		}
	}, false);
}

void UDMXPixelMappingFixtureGroupComponent::SendDMX()
{
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent) {
		if (UDMXPixelMappingOutputComponent* Component = Cast<UDMXPixelMappingOutputComponent>(InComponent))
		{
			Component->SendDMX();
		}
	}, false);
}

void UDMXPixelMappingFixtureGroupComponent::QueueDownsample()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent) {
		if (UDMXPixelMappingOutputComponent* Component = Cast<UDMXPixelMappingOutputComponent>(InComponent))
		{
			Component->QueueDownsample();
		}
	}, false);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FString UDMXPixelMappingFixtureGroupComponent::GetUserName() const
{
	if (UserName.IsEmpty())
	{
		const FString DMXLibraryName = DMXLibrary ? DMXLibrary->GetName() : LOCTEXT("NoDMXLibrary", "No DMX Library").ToString();
		return FString::Printf(TEXT("Fixture Group: %s"), *DMXLibraryName);
	}
	else
	{
		return UserName;
	}
}

void UDMXPixelMappingFixtureGroupComponent::SetPosition(const FVector2D& NewPosition)
{
	PositionX = NewPosition.X;
	PositionY = NewPosition.Y;

	const FVector2D Translation = CachedGeometry.SetPositionAbsolute(NewPosition);
	CachedGeometry.PropagonatePositionChangesToChildren(Translation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

void UDMXPixelMappingFixtureGroupComponent::SetPositionRotated(FVector2D NewRotatedPosition)
{
	const FVector2D Translation = CachedGeometry.SetPositionRotatedAbsolute(NewRotatedPosition);

	PositionX = CachedGeometry.GetPositionAbsolute().X;
	PositionY = CachedGeometry.GetPositionAbsolute().Y;

	CachedGeometry.PropagonatePositionChangesToChildren(Translation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

void UDMXPixelMappingFixtureGroupComponent::SetSize(const FVector2D& NewSize)
{
	SizeX = NewSize.X;
	SizeY = NewSize.Y;

	FVector2D DeltaSize;
	FVector2D DeltaPosition;
	CachedGeometry.SetSizeAbsolute(NewSize, DeltaSize, DeltaPosition);

	// Resizing rotated components moves their pivot, and by that their position without rotation.
	// Adjust for this so the component keeps its rotated position.
	SetPosition(CachedGeometry.GetPositionAbsolute());

	// Propagonate size changes to children if desired
#if WITH_EDITOR
	UDMXPixelMapping* PixelMapping = GetPixelMapping();
	if (PixelMapping && PixelMapping->bEditorScaleChildrenWithParent)
	{
		CachedGeometry.PropagonateSizeChangesToChildren(DeltaSize, DeltaPosition);
	}
#endif
}

void UDMXPixelMappingFixtureGroupComponent::SetRotation(double NewRotation)
{
	Rotation = NewRotation;

	const double DeltaRotation = CachedGeometry.SetRotationAbsolute(NewRotation);
	CachedGeometry.PropagonateRotationChangesToChildren(DeltaRotation);

#if WITH_EDITOR
		EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

#if WITH_EDITOR
const FText UDMXPixelMappingFixtureGroupComponent::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}
#endif // WITH_EDITOR

bool UDMXPixelMappingFixtureGroupComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	return Component && Component->IsA<UDMXPixelMappingRendererComponent>();
}

#undef LOCTEXT_NAMESPACE
