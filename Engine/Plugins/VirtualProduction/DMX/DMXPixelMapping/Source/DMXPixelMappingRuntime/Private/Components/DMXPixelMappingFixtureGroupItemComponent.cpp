// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"

#include "ColorSpace/DMXPixelMappingColorSpace_RGBCMY.h"
#include "Components/DMXPixelMappingComponentGeometryCache.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXConversions.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"
#include "DMXPixelMappingRuntimeUtils.h"
#include "DMXPixelMappingTypes.h"
#include "Engine/Texture.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "Library/DMXLibrary.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXTrace.h"
#include "Modulators/DMXModulator.h"
#include "TextureResource.h"
#include "UObject/Package.h"


DECLARE_CYCLE_STAT(TEXT("Send Fixture Group Item"), STAT_DMXPixelMaping_FixtureGroupItem, STATGROUP_DMXPIXELMAPPING);

#define LOCTEXT_NAMESPACE "DMXPixelMappingFixtureGroupItemComponent"

UDMXPixelMappingFixtureGroupItemComponent::UDMXPixelMappingFixtureGroupItemComponent()
	: DownsamplePixelIndex(0)
{
	ColorSpaceClass = UDMXPixelMappingColorSpace_RGBCMY::StaticClass();
	ColorSpace = CreateDefaultSubobject<UDMXPixelMappingColorSpace_RGBCMY>("ColorSpace");

	SizeX = 32.f;
	SizeY = 32.f;

#if WITH_EDITORONLY_DATA
	// Even tho deprecated, default values on deprecated properties need be set so they don't load their type's default value.
	ColorMode_DEPRECATED = EDMXColorMode::CM_RGB;
	AttributeR_DEPRECATED.SetFromName("Red");
	AttributeG_DEPRECATED.SetFromName("Green");
	AttributeB_DEPRECATED.SetFromName("Blue");
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	ZOrder = 2;
#endif // WITH_EDITOR
}

void UDMXPixelMappingFixtureGroupItemComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::UseDMXPixelMappingColorSpace)
		{
			// Upgrade to the DMXPixelMappingColorSpace default subobject
			if (!ensureMsgf(ColorSpace, TEXT("Missing default Subobject ColorSpace")))
			{
				ColorSpace = NewObject<UDMXPixelMappingColorSpace_RGBCMY>(this, "ColorSpace");
			}
			UDMXPixelMappingColorSpace_RGBCMY* ColorSpace_RGBCMY = Cast<UDMXPixelMappingColorSpace_RGBCMY>(ColorSpace);

			if (ColorMode_DEPRECATED == EDMXColorMode::CM_Monochrome)
			{
				ColorSpace_RGBCMY->LuminanceAttribute = MonochromeIntensity_DEPRECATED;
				ColorSpace_RGBCMY->RedAttribute = FDMXAttributeName(NAME_None);
				ColorSpace_RGBCMY->GreenAttribute = FDMXAttributeName(NAME_None);
				ColorSpace_RGBCMY->BlueAttribute = FDMXAttributeName(NAME_None);
			}
			else
			{
				ColorSpace_RGBCMY->bSendCyan = AttributeRInvert_DEPRECATED;
				ColorSpace_RGBCMY->bSendMagenta = AttributeGInvert_DEPRECATED;
				ColorSpace_RGBCMY->bSendYellow = AttributeBInvert_DEPRECATED;
				ColorSpace_RGBCMY->RedAttribute = AttributeR_DEPRECATED;
				ColorSpace_RGBCMY->GreenAttribute = AttributeG_DEPRECATED;
				ColorSpace_RGBCMY->BlueAttribute = AttributeB_DEPRECATED;
				ColorSpace_RGBCMY->LuminanceType = EDMXPixelMappingLuminanceType_RGBCMY::None;
				ColorSpace_RGBCMY->LuminanceAttribute = FDMXAttributeName(NAME_None);
			}
		}
	}
#endif

	InvalidatePixelMapRenderer();
}

void UDMXPixelMappingFixtureGroupItemComponent::PostInitProperties()
{
	Super::PostInitProperties();
	if (IsTemplate())
	{
		return;
	}

	UpdateRenderElement();
}

void UDMXPixelMappingFixtureGroupItemComponent::PostLoad()
{
	Super::PostLoad();
	if(IsTemplate())
	{
		return;
	}

	// Add valid modulators to modulator classes, remove invalid modulators
	for (int32 IndexModulator = 0; Modulators.IsValidIndex(IndexModulator); )
	{
		if (Modulators[IndexModulator])
		{
			ModulatorClasses.Add(Modulators[IndexModulator]->GetClass());
			IndexModulator++;
		}
		else
		{
			Modulators.RemoveAt(IndexModulator);
			if (!Modulators.IsValidIndex(IndexModulator++))
			{
				// Removed the last element
				break;
			}
		}
	}

	// Set the transient Color Space Class property to the initial value
	if (ensureMsgf(ColorSpace, TEXT("Unexpected invalid Color Space in Pixel Mapping Fixture Group Item %s."), *GetName()))
	{
		ColorSpaceClass = ColorSpace->GetClass();

#if WITH_EDITOR
		ColorSpace->GetOnPostEditChangedProperty().AddUObject(this, &UDMXPixelMappingFixtureGroupItemComponent::OnColorSpacePostEditChangeProperties);
#endif // WITH_EDITOR
	}
}

void UDMXPixelMappingFixtureGroupItemComponent::BeginDestroy() 
{
	Super::BeginDestroy();
	if (IsTemplate())
	{
		return;
	}

#if WITH_EDITOR
	// Set the transient Color Space Class property to the initial value
	if (ColorSpace)
	{
		ColorSpace->GetOnPostEditChangedProperty().RemoveAll(this);
	}
#endif // WITH_EDITOR

	PixelMapRenderElement.Reset();
}

#if WITH_EDITOR
void UDMXPixelMappingFixtureGroupItemComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupItemComponent, ColorSpaceClass))
	{
		if (ensureAlwaysMsgf(ColorSpaceClass, TEXT("Color Space Class was set to nullptr. This is not supported.")))
		{
			if (ColorSpace)
			{
				ColorSpace->GetOnPostEditChangedProperty().RemoveAll(this);
				ResetDMX(EDMXPixelMappingResetDMXMode::DoNotSendValues);
			}

			ColorSpace = NewObject<UDMXPixelMappingColorSpace>(this, ColorSpaceClass);
			ColorSpace->GetOnPostEditChangedProperty().AddUObject(this, &UDMXPixelMappingFixtureGroupItemComponent::OnColorSpacePostEditChangeProperties);
		}
	}

	InvalidatePixelMapRenderer();
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXPixelMappingFixtureGroupItemComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);
	
	// For consistency with Matrix, handling modulator class changes in runtime utils
	FDMXPixelMappingRuntimeUtils::HandleModulatorPropertyChange(this, PropertyChangedChainEvent, ModulatorClasses, MutableView(Modulators));
}
#endif // WITH_EDITOR

FString UDMXPixelMappingFixtureGroupItemComponent::GetUserName() const
{
	const UDMXEntityFixturePatch* Patch = FixturePatchRef.GetFixturePatch();
	if (Patch && UserName.IsEmpty())
	{
		return Patch->GetDisplayName();
	}
	else
	{
		return UserName;
	}
}

const FName& UDMXPixelMappingFixtureGroupItemComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("Fixture Item");
	return NamePrefix;
}

#if WITH_EDITOR
bool UDMXPixelMappingFixtureGroupItemComponent::IsVisible() const
{
	if (UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(GetParent()))
	{
		if (!FixtureGroupComponent->IsVisible())
		{
			return false;
		}
	}

	return Super::IsVisible();
}
#endif // WITH_EDITOR

void UDMXPixelMappingFixtureGroupItemComponent::ResetDMX(EDMXPixelMappingResetDMXMode ResetMode)
{
	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();

	if (FixturePatch)
	{
		if (ResetMode == EDMXPixelMappingResetDMXMode::SendZeroValues)
		{
			FixturePatch->SendZeroValues();
		}
		else if (ResetMode == EDMXPixelMappingResetDMXMode::SendDefaultValues)
		{
			FixturePatch->SendDefaultValues();
		}
	}
}

void UDMXPixelMappingFixtureGroupItemComponent::SendDMX()
{
	SCOPE_CYCLE_COUNTER(STAT_DMXPixelMaping_FixtureGroupItem);

	if (!PixelMapRenderElement.IsValid())
	{
		return;
	}

	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	if (!FixturePatch)
	{
		return;
	}

	UDMXLibrary* Library = FixturePatch->GetParentLibrary();
	if (!Library)
	{
		return;
	}

	if (!ColorSpace)
	{
		return;
	}

	ColorSpace->SetRGBA(PixelMapRenderElement->GetColor());
	TMap<FDMXAttributeName, float> AttributeToValueMap = ColorSpace->GetAttributeNameToValueMap();

	for (UDMXModulator* Modulator : Modulators)
	{
		Modulator->Modulate(FixturePatch, AttributeToValueMap, AttributeToValueMap);
		// No need to apply Matrix Modulators, this is not a Matrix
	}
	
	TMap<int32, uint8> ChannelToValueMap;
	for (const TTuple<FDMXAttributeName, float>& AttributeValuePair : AttributeToValueMap)
	{
		FDMXPixelMappingRuntimeUtils::ConvertNormalizedAttributeValueToChannelValue(FixturePatch, AttributeValuePair.Key, AttributeValuePair.Value, ChannelToValueMap);
	}

	// Send DMX
	UE_DMX_SCOPED_TRACE_SENDDMX(GetOutermost()->GetFName());
	UE_DMX_SCOPED_TRACE_SENDDMX(Library->GetFName());
	UE_DMX_SCOPED_TRACE_SENDDMX(*FixturePatch->GetDisplayName());
	for (const FDMXOutputPortSharedRef& OutputPort : Library->GetOutputPorts())
	{
		OutputPort->SendDMX(FixturePatch->GetUniverseID(), ChannelToValueMap);
	}
}

void UDMXPixelMappingFixtureGroupItemComponent::QueueDownsample()
{
	// DEPRECATED 5.3

	// Queue pixels into the downsample rendering
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent();

	if (!ensure(RendererComponent))
	{
		return;
	}
	
	UTexture* InputTexture = RendererComponent->GetRenderedInputTexture();
	if (!ensure(InputTexture))
	{
		return;
	}

	// Store pixel position
	DownsamplePixelIndex = RendererComponent->GetDownsamplePixelNum();

	const uint32 TextureSizeX = InputTexture->GetResource()->GetSizeX();
	const uint32 TextureSizeY = InputTexture->GetResource()->GetSizeY();
	check(TextureSizeX > 0 && TextureSizeY > 0);
	const FIntPoint PixelPosition = RendererComponent->GetPixelPosition(DownsamplePixelIndex);
	const FVector2D UV = FVector2D(GetPosition().X / TextureSizeX, GetPosition().Y / TextureSizeY);
	const FVector2D UVSize(GetSize().X / TextureSizeX, GetSize().Y / TextureSizeY);
	const FVector2D UVCellSize = UVSize / 2.f;
	constexpr bool bStaticCalculateUV = true;

	FDMXPixelMappingDownsamplePixelParamsV2 DownsamplePixelParams
	{ 
		PixelPosition,
		UV,
		UVSize,
		UVCellSize,
		CellBlendingQuality,
		bStaticCalculateUV
	};

	RendererComponent->AddPixelToDownsampleSet(MoveTemp(DownsamplePixelParams));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDMXPixelMappingFixtureGroupItemComponent::SetPosition(const FVector2D& NewPosition)
{
	Super::SetPosition(NewPosition);

	UpdateRenderElement();
}

void UDMXPixelMappingFixtureGroupItemComponent::SetSize(const FVector2D& NewSize)
{
	Super::SetSize(NewSize);

	UpdateRenderElement();
}

bool UDMXPixelMappingFixtureGroupItemComponent::IsOverParent() const
{
	if (UDMXPixelMappingFixtureGroupComponent* Parent = Cast<UDMXPixelMappingFixtureGroupComponent>(GetParent()))
	{
		FVector2D A;
		FVector2D B;
		FVector2D C;
		FVector2D D;
		CachedGeometry.GetEdgesAbsolute(A, B, C, D);

		return
			Parent->IsOverPosition(A) &&
			Parent->IsOverPosition(B) &&
			Parent->IsOverPosition(C) &&
			Parent->IsOverPosition(D);
	}

	return false;
}

TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement> UDMXPixelMappingFixtureGroupItemComponent::GetOrCreatePixelMapRenderElement()
{
	UpdateRenderElement();
	return PixelMapRenderElement.ToSharedRef();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDMXPixelMappingFixtureGroupItemComponent::RenderWithInputAndSendDMX()
{
	// DEPRECATED 5.3
	RenderAndSendDMX();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

bool UDMXPixelMappingFixtureGroupItemComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	if (const UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Component))
	{
		if (FixtureGroupComponent->DMXLibrary == FixturePatchRef.DMXLibrary)
		{
			return true;
		}
	}

	return false;
}

void UDMXPixelMappingFixtureGroupItemComponent::UpdateRenderElement()
{
	using namespace UE::DMXPixelMapping::Rendering;

	const UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent();
	const UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	const double InputTextureWidth = InputTexture ? InputTexture->GetSurfaceWidth() : 1.0;
	const double InputTextureHeight = InputTexture ? InputTexture->GetSurfaceHeight() : 1.0;

	FPixelMapRenderElementParameters Parameters;
	Parameters.UV = FVector2D(GetPosition().X / InputTextureWidth, GetPosition().Y / InputTextureHeight);
	Parameters.UVSize = FVector2D(GetSize().X / InputTextureWidth, GetSize().Y / InputTextureHeight);

	FVector2D A;
	FVector2D B;
	FVector2D C;
	FVector2D D;
	GetEdges(A, B, C, D);
	Parameters.UVTopLeftRotated = FVector2D(A.X / InputTextureWidth, A.Y / InputTextureHeight);
	Parameters.UVTopRightRotated = FVector2D(B.X / InputTextureWidth, B.Y / InputTextureHeight);

	Parameters.Rotation = GetRotation();
	Parameters.CellBlendingQuality = CellBlendingQuality;
	Parameters.bStaticCalculateUV = true;

	if (!PixelMapRenderElement.IsValid())
	{
		PixelMapRenderElement = MakeShared<FPixelMapRenderElement>(Parameters);
	}
	PixelMapRenderElement->SetParameters(Parameters);
}

#if WITH_EDITOR
void UDMXPixelMappingFixtureGroupItemComponent::OnColorSpacePostEditChangeProperties(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FDMXAttributeName, Name))
	{
		// Reset DMX when an attribute of the color space changed
		ResetDMX(EDMXPixelMappingResetDMXMode::DoNotSendValues);
	}
}
#endif // WITH_EDITOR

UDMXPixelMappingRendererComponent* UDMXPixelMappingFixtureGroupItemComponent::UDMXPixelMappingFixtureGroupItemComponent::GetRendererComponent() const
{
	return GetParent() ? Cast<UDMXPixelMappingRendererComponent>(GetParent()->GetParent()) : nullptr;
}

TMap<FDMXAttributeName, float> UDMXPixelMappingFixtureGroupItemComponent::CreateAttributeValues() const
{
	// DEPRECATED 5.2
	TMap<FDMXAttributeName, float> AttributeToValueMap;

	if (ensureMsgf(ColorSpace, TEXT("Unexpected invalid Color Space in Pixel Mapping Fixture Group Item %s."), *GetName()))
	{
		AttributeToValueMap = ColorSpace->GetAttributeNameToValueMap();
	}

	return AttributeToValueMap;
}

#undef LOCTEXT_NAMESPACE
