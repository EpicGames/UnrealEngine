// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingMatrixCellComponent.h"

#include "ColorSpace/DMXPixelMappingColorSpace.h"
#include "Components/DMXPixelMappingComponentGeometryCache.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXConversions.h"
#include "DMXPixelMappingRenderElement.h"
#include "DMXPixelMappingTypes.h"
#include "Engine/Texture.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXEntityFixtureType.h"
#include "TextureResource.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingMatrixPixelComponent"

UDMXPixelMappingMatrixCellComponent::UDMXPixelMappingMatrixCellComponent()
	: DownsamplePixelIndex(0)
{
#if WITH_EDITORONLY_DATA
	bLockInDesigner = true;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// EditorColor should no longer be accessed publicly, however it is ok to access it here.
	EditorColor = FLinearColor::White.CopyWithNewOpacity(.25f);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
}

void UDMXPixelMappingMatrixCellComponent::PostInitProperties()
{
	Super::PostInitProperties();
	if (IsTemplate())
	{
		return;
	}

	UpdateRenderElement();
}

void UDMXPixelMappingMatrixCellComponent::PostLoad()
{
	Super::PostLoad();
	if (IsTemplate())
	{
		return;
	}

	UpdateRenderElement();
}

void UDMXPixelMappingMatrixCellComponent::BeginDestroy()
{
	Super::BeginDestroy();
	if (IsTemplate())
	{
		return;
	}

	PixelMapRenderElement.Reset();
}

#if WITH_EDITOR
void UDMXPixelMappingMatrixCellComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Call the parent at the first place
	Super::PostEditChangeProperty(PropertyChangedEvent);

	UpdateRenderElement();

	InvalidatePixelMapRenderer();
}
#endif // WITH_EDITOR

bool UDMXPixelMappingMatrixCellComponent::IsOverParent() const
{
	if (UDMXPixelMappingMatrixComponent* Parent = Cast<UDMXPixelMappingMatrixComponent>(GetParent()))
	{
		FVector2D A;
		FVector2D B;
		FVector2D C;
		FVector2D D;
		CachedGeometry.GetEdgesAbsolute(A, B, C, D);

		return
			Parent->IsOverPosition(A) &&
			Parent->IsOverPosition(B) &&
			Parent->IsOverPosition(C) &&
			Parent->IsOverPosition(D);
	}

	return false;
}

const FName& UDMXPixelMappingMatrixCellComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("MatrixCell");
	return NamePrefix;
}

FString UDMXPixelMappingMatrixCellComponent::GetUserName() const
{
	UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(GetParent());
	UDMXEntityFixturePatch* FixturePatch = MatrixComponent ? MatrixComponent->FixturePatchRef.GetFixturePatch() : nullptr;
	if (FixturePatch && UserName.IsEmpty())
	{
		return FString::Printf(TEXT("%s: Cell %d"), *FixturePatch->GetDisplayName(), CellID);
	}
	else
	{
		return FString::Printf(TEXT("%s: Cell %d"), *UserName, CellID);
	}
}

void UDMXPixelMappingMatrixCellComponent::QueueDownsample()
{
	// DEPRECATED 5.3

	// Queue pixels into the downsample rendering
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(GetParent());
	UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent();

	if (!ensure(MatrixComponent))
	{
		return;
	}

	if (!ensure(RendererComponent))
	{
		return;
	}

	UTexture* InputTexture = RendererComponent->GetRenderedInputTexture();
	if (!ensure(InputTexture))
	{
		return;
	}

	// Store downsample index
	DownsamplePixelIndex = RendererComponent->GetDownsamplePixelNum();

	const uint32 TextureSizeX = InputTexture->GetResource()->GetSizeX();
	const uint32 TextureSizeY = InputTexture->GetResource()->GetSizeY();
	check(TextureSizeX > 0 && TextureSizeY > 0);
	const FIntPoint PixelPosition = RendererComponent->GetPixelPosition(DownsamplePixelIndex);
	const FVector2D UV = FVector2D(GetPosition().X / TextureSizeX, GetPosition().Y / TextureSizeY);
	const FVector2D UVSize(GetSize().X / TextureSizeX, GetSize().Y / TextureSizeY);
	const FVector2D UVCellSize = UVSize / 2.f;
	constexpr bool bStaticCalculateUV = true;
			
	FDMXPixelMappingDownsamplePixelParamsV2 DownsamplePixelParam
	{ 
		PixelPosition,
		UV,
		UVSize,
		UVCellSize,
		MatrixComponent->CellBlendingQuality,
		bStaticCalculateUV
	};

	RendererComponent->AddPixelToDownsampleSet(MoveTemp(DownsamplePixelParam));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDMXPixelMappingMatrixCellComponent::SetCellCoordinate(FIntPoint InCellCoordinate)
{
	CellCoordinate = InCellCoordinate;
}

bool UDMXPixelMappingMatrixCellComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* OtherComponent) const
{
	return OtherComponent && OtherComponent == GetParent();
}

TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement> UDMXPixelMappingMatrixCellComponent::GetOrCreatePixelMapRenderElement()
{
	UpdateRenderElement();
	return PixelMapRenderElement.ToSharedRef();
}

void UDMXPixelMappingMatrixCellComponent::UpdateRenderElement()
{
	using namespace UE::DMXPixelMapping::Rendering;
	
	UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(GetParent());
	if (!MatrixComponent)
	{
		return;
	}

	const UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent();
	const UTexture* InputTexture = RendererComponent ? RendererComponent->GetRenderedInputTexture() : nullptr;
	const double InputTextureWidth = InputTexture ? InputTexture->GetSurfaceWidth() : 1.0;
	const double InputTextureHeight = InputTexture ? InputTexture->GetSurfaceHeight() : 1.0;

	FPixelMapRenderElementParameters Parameters;
	Parameters.UV = FVector2D(GetPosition().X / InputTextureWidth, GetPosition().Y / InputTextureHeight);
	Parameters.UVSize = FVector2D(GetSize().X / InputTextureWidth, GetSize().Y / InputTextureHeight);

	FVector2D A;
	FVector2D B;
	FVector2D C;
	FVector2D D;
	GetEdges(A, B, C, D);

	Parameters.UVTopLeftRotated = FVector2D(A.X / InputTextureWidth, A.Y / InputTextureHeight);
	Parameters.UVTopRightRotated = FVector2D(B.X / InputTextureWidth, B.Y / InputTextureHeight);

	Parameters.Rotation = GetRotation();
	Parameters.CellBlendingQuality = MatrixComponent->CellBlendingQuality;
	Parameters.bStaticCalculateUV = true;

	if (!PixelMapRenderElement.IsValid())
	{
		PixelMapRenderElement = MakeShared<FPixelMapRenderElement>(Parameters);
	}
	else
	{
		PixelMapRenderElement->SetParameters(Parameters);
	}
}

UDMXPixelMappingRendererComponent* UDMXPixelMappingMatrixCellComponent::GetRendererComponent() const
{
	for (UDMXPixelMappingBaseComponent* ParentComponent = GetParent(); ParentComponent; ParentComponent = ParentComponent->GetParent())
	{
		if (UDMXPixelMappingRendererComponent* RendererComponent = Cast<UDMXPixelMappingRendererComponent>(ParentComponent))
		{
			return RendererComponent;
		}
	}
	return nullptr;
}

FLinearColor UDMXPixelMappingMatrixCellComponent::GetPixelMapColor() const
{
	return PixelMapRenderElement->GetColor();
}

#undef LOCTEXT_NAMESPACE
