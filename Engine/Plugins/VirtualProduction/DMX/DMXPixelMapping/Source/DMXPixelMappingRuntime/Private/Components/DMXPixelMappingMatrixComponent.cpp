// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingMatrixComponent.h"

#include "Algo/Sort.h"
#include "ColorSpace/DMXPixelMappingColorSpace_RGBCMY.h"
#include "Components/DMXPixelMappingComponentGeometryCache.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "DMXConversions.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"
#include "DMXPixelMappingRuntimeLog.h"
#include "DMXPixelMappingRuntimeUtils.h"
#include "DMXPixelMappingTypes.h"
#include "ColorSpace/DMXPixelMappingColorSpace_RGBCMY.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXTrace.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "Modulators/DMXModulator.h"
#include "Templates/DMXPixelMappingComponentTemplate.h"
#include "UObject/Package.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingMatrixComponent"

UDMXPixelMappingMatrixComponent::UDMXPixelMappingMatrixComponent()
{
	ColorSpaceClass = UDMXPixelMappingColorSpace_RGBCMY::StaticClass();
	ColorSpace = CreateDefaultSubobject<UDMXPixelMappingColorSpace_RGBCMY>("ColorSpace");

	SizeX = 32.f;
	SizeY = 32.f;

#if WITH_EDITORONLY_DATA
	bExpanded = false;

	// Even tho deprecated, default values on deprecated properties need be set so they don't load their type's default value.
	ColorMode_DEPRECATED = EDMXColorMode::CM_RGB;
	AttributeR_DEPRECATED.SetFromName("Red");
	AttributeG_DEPRECATED.SetFromName("Green");
	AttributeB_DEPRECATED.SetFromName("Blue");
#endif // WITH_EDITORONLY_DATA
}

void UDMXPixelMappingMatrixComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if WITH_EDITOR
	Ar.UsingCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID);
	if(Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::ChangePixelMappingMatrixComponentToFixturePatchReference)
		{
			// Upgrade from custom FixturePatchMatrixRef to FixturePatchRef
			FixturePatchRef.SetEntity(FixturePatchMatrixRef_DEPRECATED.GetFixturePatch());
		}

		if (Ar.CustomVer(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::UseDMXPixelMappingColorSpace)
		{		
			// Upgrade to the DMXPixelMappingColorSpace default subobject
			if (!ensureMsgf(ColorSpace, TEXT("Missing default Subobject ColorSpace")))
			{
				ColorSpace = NewObject<UDMXPixelMappingColorSpace_RGBCMY>(this, "ColorSpace");
			}
			UDMXPixelMappingColorSpace_RGBCMY* ColorSpace_RGBCMY = Cast<UDMXPixelMappingColorSpace_RGBCMY>(ColorSpace);

			if (ColorMode_DEPRECATED == EDMXColorMode::CM_Monochrome)
			{
				ColorSpace_RGBCMY->LuminanceAttribute = MonochromeIntensity_DEPRECATED;
				ColorSpace_RGBCMY->RedAttribute = FDMXAttributeName(NAME_None);
				ColorSpace_RGBCMY->GreenAttribute = FDMXAttributeName(NAME_None);
				ColorSpace_RGBCMY->BlueAttribute = FDMXAttributeName(NAME_None);
			}
			else
			{
				ColorSpace_RGBCMY->bSendCyan = AttributeRInvert_DEPRECATED;
				ColorSpace_RGBCMY->bSendMagenta = AttributeGInvert_DEPRECATED;
				ColorSpace_RGBCMY->bSendYellow = AttributeBInvert_DEPRECATED;
				ColorSpace_RGBCMY->RedAttribute = AttributeR_DEPRECATED;
				ColorSpace_RGBCMY->GreenAttribute = AttributeG_DEPRECATED;
				ColorSpace_RGBCMY->BlueAttribute = AttributeB_DEPRECATED;
				ColorSpace_RGBCMY->LuminanceType = EDMXPixelMappingLuminanceType_RGBCMY::None;
				ColorSpace_RGBCMY->LuminanceAttribute = FDMXAttributeName(NAME_None);
			}
		}
	}
#endif
}

void UDMXPixelMappingMatrixComponent::PostLoad()
{
	Super::PostLoad();

	if (IsTemplate())
	{
		return;
	}

	// Add valid modulators to modulator classes, remove invalid modulators
	for (int32 IndexModulator = 0; Modulators.IsValidIndex(IndexModulator); )
	{
		if (Modulators[IndexModulator])
		{
			ModulatorClasses.Add(Modulators[IndexModulator]->GetClass());
			IndexModulator++;
		}
		else
		{
			Modulators.RemoveAt(IndexModulator);
			if (!Modulators.IsValidIndex(IndexModulator++))
			{
				// Removed the last element
				break;
			}
		}
	}

	// Set the transient Color Space Class property to the initial value
	if (ensureMsgf(ColorSpace, TEXT("Unexpected invalid Color Space in Pixel Mapping Fixture Group Item %s."), *GetName()))
	{
		ColorSpaceClass = ColorSpace->GetClass();
	}
}

void UDMXPixelMappingMatrixComponent::PostInitProperties()
{
	Super::PostInitProperties();

	if (IsTemplate())
	{
		return;
	}

	// Listen to Fixture Type and Fixture Patch changes
	UDMXEntityFixtureType::GetOnFixtureTypeChanged().AddUObject(this, &UDMXPixelMappingMatrixComponent::OnFixtureTypeChanged);
	UDMXEntityFixturePatch::GetOnFixturePatchChanged().AddUObject(this, &UDMXPixelMappingMatrixComponent::OnFixturePatchChanged);
}

void UDMXPixelMappingMatrixComponent::LogInvalidProperties()
{
	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	if (IsValid(FixturePatch))
	{
		const FDMXFixtureMode* ModePtr = FixturePatch->GetActiveMode();
		if (!ModePtr)
		{
			UE_LOGF(LogDMXPixelMappingRuntime, Warning, "%ls has no valid Active Mode set. %ls will not receive DMX.", *FixturePatch->GetDisplayName(), *GetName());
		}
		else if (!FixturePatch->GetFixtureType())
		{
			UE_LOGF(LogDMXPixelMappingRuntime, Warning, "%ls has no valid Fixture Type set. %ls will not receive DMX.", *FixturePatch->GetDisplayName(), *GetName());
		}
		else if(Children.Num() != ModePtr->FixtureMatrixConfig.XCells * ModePtr->FixtureMatrixConfig.YCells)
			{
				UE_LOGF(LogDMXPixelMappingRuntime, Warning, "Number of cells in %ls no longer matches %ls. %ls will not function properly.", *GetName(), *FixturePatch->GetFixtureType()->Name, *GetName());
			}
		}
	else
	{
		UE_LOGF(LogDMXPixelMappingRuntime, Warning, "%ls has no valid Fixture Patch set.", *GetName());
	}
}

#if WITH_EDITOR
void UDMXPixelMappingMatrixComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Note, property changes of fixture patch are listened for in tick

	// Call the parent at the first place
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, ColorSpaceClass))
	{
		if (ensureAlwaysMsgf(ColorSpaceClass, TEXT("Color Space Class was set to nullptr. This is not supported.")))
		{
			ColorSpace = NewObject<UDMXPixelMappingColorSpace>(this, ColorSpaceClass);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, FixturePatchRef) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, CoordinateGrid) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, bInvertCellsX) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, bInvertCellsY) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(FDMXEntityReference, DMXLibrary))
	{
		HandleMatrixChanged();
	}

	InvalidatePixelMapRenderer();
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXPixelMappingMatrixComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	// For consistency with GroupItem, handling modulator class changes in runtime utils
	FDMXPixelMappingRuntimeUtils::HandleModulatorPropertyChange(this, PropertyChangedChainEvent, ModulatorClasses, MutableView(Modulators));
}
#endif // WITH_EDITOR

#if WITH_EDITOR
const FText UDMXPixelMappingMatrixComponent::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}
#endif // WITH_EDITOR

bool UDMXPixelMappingMatrixComponent::IsOverParent() const
{
	if (UDMXPixelMappingFixtureGroupComponent* Parent = Cast<UDMXPixelMappingFixtureGroupComponent>(GetParent()))
	{
		FVector2D A;
		FVector2D B;
		FVector2D C;
		FVector2D D;
		CachedGeometry.GetEdgesAbsolute(A, B, C, D);

		return
			Parent->IsOverPosition(A) &&
			Parent->IsOverPosition(B) &&
			Parent->IsOverPosition(C) &&
			Parent->IsOverPosition(D);
	}

	return false;
}

const FName& UDMXPixelMappingMatrixComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("Matrix");
	return NamePrefix;
}

void UDMXPixelMappingMatrixComponent::ResetDMX(EDMXPixelMappingResetDMXMode ResetMode)
{
	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();

	if (FixturePatch)
	{
		if (ResetMode == EDMXPixelMappingResetDMXMode::SendZeroValues)
		{
			FixturePatch->SendZeroValues();
		}
		else if (ResetMode == EDMXPixelMappingResetDMXMode::SendDefaultValues)
		{
			FixturePatch->SendDefaultValues();
		}
	}
}

void UDMXPixelMappingMatrixComponent::SendDMX()
{
	if (!ensureMsgf(ColorSpace, TEXT("Invalid color space in Pixel Mapping Matrix Component '%s'"), *GetName()))
	{
		return;
	}

	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	const FDMXFixtureMode* ActiveModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;
	if (FixturePatch && ActiveModePtr && ActiveModePtr->bFixtureMatrixEnabled)
	{
		TArray<UDMXPixelMappingMatrixCellComponent*> CellComponents;
		Algo::TransformIf(Children, CellComponents,
			[](UDMXPixelMappingBaseComponent* BaseComponent)
			{
				return BaseComponent != nullptr;
			},
			[](UDMXPixelMappingBaseComponent* BaseComponent)
			{
				return Cast<UDMXPixelMappingMatrixCellComponent>(BaseComponent);
			});
		Algo::SortBy(CellComponents, [ActiveModePtr](UDMXPixelMappingMatrixCellComponent* Component)
			{
				// Sort by cell ID
				return Component->CellID;
			});

		// Accumulate matrix cell data
		TArray<FDMXNormalizedAttributeValueMap> AttributeToValueMapArray;
		AttributeToValueMapArray.Reserve(CellComponents.Num());
		Algo::Transform(CellComponents, AttributeToValueMapArray, [this](UDMXPixelMappingMatrixCellComponent* CellComponent)
			{	
				// Get the color data from the rendered component
				ColorSpace->SetRGBA(CellComponent->GetPixelMapColor());

				FDMXNormalizedAttributeValueMap MapStruct;
				MapStruct.Map = ColorSpace->GetAttributeNameToValueMap();
				return MapStruct;
			});
		if (!ensureMsgf(AttributeToValueMapArray.Num() == CellComponents.Num(), TEXT("Mismatch num cell attributes and num attribute to value maps. Cannot send DMX for Matrix Component %s"), *GetName()))
		{
			return;
		}

		// Apply matrix modulators
		for (UDMXModulator* Modulator : Modulators)
		{
			Modulator->ModulateMatrix(FixturePatch, AttributeToValueMapArray, AttributeToValueMapArray);
		}

		// Write channel values
		const int32 MatrixStartingChannel = FixturePatch->GetStartingChannel() + ActiveModePtr->FixtureMatrixConfig.FirstCellChannel - 1;
		const int32 MatrixCellSize = ActiveModePtr->FixtureMatrixConfig.GetNumChannels();

		int32 ChannelOffset = 0;
		TMap<int32, uint8> ChannelToValueMap;
		for (const FDMXNormalizedAttributeValueMap& AttributeToValueMap : AttributeToValueMapArray)
		{
			for (const FDMXFixtureCellAttribute& CellAttribute : ActiveModePtr->FixtureMatrixConfig.CellAttributes)
			{
				const float* ValuePtr = AttributeToValueMap.Map.Find(CellAttribute.Attribute);
				if (ValuePtr)
				{
					const TArray<uint8> Values = FDMXConversions::NormalizedDMXValueToByteArray(*ValuePtr, CellAttribute.DataType, CellAttribute.bUseLSBMode);
					for (int32 ValueIndex = 0; ValueIndex < Values.Num(); ValueIndex++)
					{
						const int32 Channel = MatrixStartingChannel + ChannelOffset + ValueIndex;
						ChannelToValueMap.Add(Channel, Values[ValueIndex]);
					}
				}

				ChannelOffset += CellAttribute.GetNumChannels();
			}
		}

		// Write modulator generated non-matrix attributes. This is important to allow modulators to generate non-matrix attribute values
		TMap<FDMXAttributeName, float> ModulatorGeneratedAttributeValueMap;
		for (UDMXModulator* Modulator : Modulators)
		{
			Modulator->Modulate(FixturePatch, ModulatorGeneratedAttributeValueMap, ModulatorGeneratedAttributeValueMap);
		}
		for (const TTuple<FDMXAttributeName, float>& AttributeValuePair : ModulatorGeneratedAttributeValueMap)
		{
			FDMXPixelMappingRuntimeUtils::ConvertNormalizedAttributeValueToChannelValue(FixturePatch, AttributeValuePair.Key, AttributeValuePair.Value, ChannelToValueMap);
		}

		// Send DMX
		if (UDMXLibrary* Library = FixturePatch->GetParentLibrary())
		{
			UE_DMX_SCOPED_TRACE_SENDDMX(GetOutermost()->GetFName());
			UE_DMX_SCOPED_TRACE_SENDDMX(Library->GetFName());
			for (const FDMXOutputPortSharedRef& OutputPort : Library->GetOutputPorts())
			{
				OutputPort->SendDMX(FixturePatch->GetUniverseID(), ChannelToValueMap);
			}
		}
	}
}

bool UDMXPixelMappingMatrixComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	if (const UDMXPixelMappingFixtureGroupComponent* FixtureGroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Component))
	{
		if (FixtureGroupComponent->DMXLibrary == FixturePatchRef.DMXLibrary)
		{
			return true;
		}
	}

	return false;
}

FString UDMXPixelMappingMatrixComponent::GetUserName() const
{
	const UDMXEntityFixturePatch* Patch = FixturePatchRef.GetFixturePatch();
	if (Patch && UserName.IsEmpty())
	{
		return FString::Printf(TEXT("%s"), *Patch->GetDisplayName());
	}
	else
	{
		return UserName;
	}
}

void UDMXPixelMappingMatrixComponent::SetPosition(const FVector2D& NewPosition)
{
	PositionX = NewPosition.X;
	PositionY = NewPosition.Y;

	const FVector2D Translation = CachedGeometry.SetPositionAbsolute(NewPosition);
	CachedGeometry.PropagonatePositionChangesToChildren(Translation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif

	InvalidatePixelMapRenderer();
}

void UDMXPixelMappingMatrixComponent::SetPositionRotated(FVector2D NewRotatedPosition)
{
	const FVector2D Translation = CachedGeometry.SetPositionRotatedAbsolute(NewRotatedPosition);

	PositionX = CachedGeometry.GetPositionAbsolute().X;
	PositionY = CachedGeometry.GetPositionAbsolute().Y;

	CachedGeometry.PropagonatePositionChangesToChildren(Translation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif

	InvalidatePixelMapRenderer();
}

void UDMXPixelMappingMatrixComponent::SetSize(const FVector2D& NewSize)
{
	SizeX = NewSize.X;
	SizeY = NewSize.Y;

	FVector2D DeltaSize;
	FVector2D DeltaPosition;
	CachedGeometry.SetSizeAbsolute(NewSize, DeltaSize, DeltaPosition);

	// Resizing rotated components moves their pivot, and by that their position without rotation.
	// Adjust for this so the component keeps its rotated position.
	SetPosition(CachedGeometry.GetPositionAbsolute());

	CachedGeometry.PropagonateSizeChangesToChildren(DeltaSize, DeltaPosition);

	InvalidatePixelMapRenderer();
}

void UDMXPixelMappingMatrixComponent::SetRotation(double NewRotation)
{
	Rotation = NewRotation;

	const double DeltaRotation = CachedGeometry.SetRotationAbsolute(NewRotation);
	CachedGeometry.PropagonateRotationChangesToChildren(DeltaRotation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif

	InvalidatePixelMapRenderer();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDMXPixelMappingMatrixComponent::RenderWithInputAndSendDMX()
{
	// DEPRECATED 5.3
	RenderAndSendDMX();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UDMXPixelMappingMatrixComponent::HandleMatrixChanged()
{
	UDMXPixelMapping* PixelMapping = GetPixelMapping();
	UDMXPixelMappingRootComponent* RootComponent = PixelMapping ? PixelMapping->GetRootComponent() : nullptr;
	if (!RootComponent)
	{
		// Component was removed from pixel mapping
		return;
	}

	const double RestoreRotation = GetRotation();
	SetRotation(0.0);
	CoordinateGrid = 0;

	// Remove all existing children and rebuild them anew
	for (UDMXPixelMappingBaseComponent* Child : TArray<UDMXPixelMappingBaseComponent*>(Children))
	{
		RemoveChild(Child);
	}

	UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch();
	UDMXEntityFixtureType* FixtureType = FixturePatch ? FixturePatch->GetFixtureType() : nullptr;
	const FDMXFixtureMode* ModePtr = FixturePatch ? FixturePatch->GetActiveMode() : nullptr;
	if (FixturePatch && FixtureType && ModePtr && ModePtr->bFixtureMatrixEnabled)
	{
		TArray<FDMXCell> MatrixCells;
		if (FixturePatch->GetAllMatrixCells(MatrixCells))
		{
			Distribution = ModePtr->FixtureMatrixConfig.PixelMappingDistribution;
			CoordinateGrid = FIntPoint(ModePtr->FixtureMatrixConfig.XCells, ModePtr->FixtureMatrixConfig.YCells);

			for (const FDMXCell& Cell : MatrixCells)
			{
				TSharedPtr<FDMXPixelMappingComponentTemplate> ComponentTemplate = MakeShared<FDMXPixelMappingComponentTemplate>(UDMXPixelMappingMatrixCellComponent::StaticClass());
				UDMXPixelMappingMatrixCellComponent* Component = ComponentTemplate->CreateComponent<UDMXPixelMappingMatrixCellComponent>(RootComponent);

				Component->CellID = Cell.CellID;
				Component->SetCellCoordinate(Cell.Coordinate);

				AddChild(Component);
			}
		}
	}

	// Set child transform
	CellSize = FVector2D::ZeroVector;
	if (CoordinateGrid.X > 0 && CoordinateGrid.Y > 0)
	{
		CellSize = FVector2D(GetSize().X / CoordinateGrid.X, GetSize().Y / CoordinateGrid.Y);
	}

	auto InvertCellsIfRequiredLambda = [this](FIntPoint CellCoordinate)
		{
			CellCoordinate.X = bInvertCellsX ? FMath::Abs(CellCoordinate.X - CoordinateGrid.X) - 1 : CellCoordinate.X;
			CellCoordinate.Y = bInvertCellsY ? FMath::Abs(CellCoordinate.Y - CoordinateGrid.Y) - 1 : CellCoordinate.Y;
			return CellCoordinate;
		};

	constexpr bool bUpdateSizeRecursive = false;
	ForEachChildOfClass<UDMXPixelMappingMatrixCellComponent>([this, &InvertCellsIfRequiredLambda](UDMXPixelMappingMatrixCellComponent* ChildComponent)
		{
			ChildComponent->SetRotation(GetRotation());
			ChildComponent->SetSize(CellSize);

			const FIntPoint CellCoordinate = InvertCellsIfRequiredLambda(ChildComponent->GetCellCoordinate());
			ChildComponent->SetPosition(GetPosition() + FVector2D(CellSize * CellCoordinate));
		},
		bUpdateSizeRecursive);

	SetRotation(RestoreRotation);

	InvalidatePixelMapRenderer();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GetOnMatrixChanged().Broadcast(PixelMapping, this);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDMXPixelMappingMatrixComponent::QueueDownsample()
{
	// Deprecated 5.3
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent) {
		if (UDMXPixelMappingOutputComponent* Component = Cast<UDMXPixelMappingOutputComponent>(InComponent))
		{
			Component->QueueDownsample();
		}
	}, false);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UDMXPixelMappingMatrixComponent::OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType)
{
	if (UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch())
	{
		if (FixturePatch->GetFixtureType() == FixtureType)
		{
			HandleMatrixChanged();

			LogInvalidProperties();
		}
	}
}

void UDMXPixelMappingMatrixComponent::OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch)
{
	if (FixturePatch && FixturePatch == FixturePatchRef.GetEntity())
	{
		HandleMatrixChanged();

		LogInvalidProperties();
	}
}

#undef LOCTEXT_NAMESPACE
