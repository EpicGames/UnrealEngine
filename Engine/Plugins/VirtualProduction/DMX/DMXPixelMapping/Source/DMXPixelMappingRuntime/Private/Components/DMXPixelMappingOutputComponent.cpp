// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingOutputComponent.h"

#include "Components/DMXPixelMappingComponentGeometryCache.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"


#define LOCTEXT_NAMESPACE "DMXPixelMappingOutputComponent"

UDMXPixelMappingOutputComponent::UDMXPixelMappingOutputComponent()
{
#if WITH_EDITOR
	bLockInDesigner = false;
	bVisibleInDesigner = true;
	ZOrder = 0;
#endif // WITH_EDITOR

	// It is important to initialize with properties, as the getters make use of the geometry cache.
	CachedGeometry.Initialize(this, FVector2D(PositionX, PositionY), FVector2D(SizeX, SizeY), Rotation);
}

void UDMXPixelMappingOutputComponent::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	EditorPositionWithRotation = GetPosition();
#endif
}

void UDMXPixelMappingOutputComponent::PostLoad()
{
	Super::PostLoad();

	// It is important to initialize with properties, as the getters make use of the geometry cache.
	CachedGeometry.Initialize(this, FVector2D(PositionX, PositionY), FVector2D(SizeX, SizeY), Rotation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

#if WITH_EDITOR
void UDMXPixelMappingOutputComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, EditorPositionWithRotation))
	{
		SetPositionRotated(EditorPositionWithRotation);
	} 
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, SizeX) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, SizeY))
	{
		SetSize(FVector2D(SizeX, SizeY));
	}
	else if(PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, Rotation))
	{
		SetRotation(Rotation);
	}
}
#endif // WITH_EDITOR

bool UDMXPixelMappingOutputComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	return false;
}

#if WITH_EDITOR
const FText UDMXPixelMappingOutputComponent::GetPaletteCategory()
{
	ensureMsgf(false, TEXT("You must implement GetPaletteCategory() in your child class"));

	return LOCTEXT("Uncategorized", "Uncategorized");
}
#endif // WITH_EDITOR

#if WITH_EDITOR
bool UDMXPixelMappingOutputComponent::IsVisible() const
{
	const bool bAssignedToParent = GetParent() && GetParent()->Children.Contains(this);

	return bVisibleInDesigner && bAssignedToParent;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXPixelMappingOutputComponent::SetZOrder(int32 NewZOrder)
{
	ZOrder = NewZOrder;
}
#endif // WITH_EDITOR

#if WITH_EDITOR
FLinearColor UDMXPixelMappingOutputComponent::GetEditorColor() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// EditorColor should not be publicly accessed. However it is ok to access it here.
	return EditorColor;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

bool UDMXPixelMappingOutputComponent::IsOverParent() const
{
	// By default all components are over their parent.
	// E.g. Renderer is always over the root, group is always over the renderer.
	return true;
}

bool UDMXPixelMappingOutputComponent::IsOverPosition(const FVector2D& OtherPosition) const
{
	FVector2D A;
	FVector2D B;
	FVector2D C;
	FVector2D D;
	CachedGeometry.GetEdgesAbsolute(A, B, C, D);

	const FVector2D AM = OtherPosition - A;
	const FVector2D AB = B - A;
	const FVector2D AD = D - A;
	
	const double AMDotAB = FVector2D::DotProduct(AM, AB);
	const double MagnituteSquaredAB = FVector2D::DotProduct(AB, AB);
	const double AMDotAD = FVector2D::DotProduct(AM, AD);
	const double MagnituteSquaredAD = FVector2D::DotProduct(AD, AD);

	constexpr double Tolerance = 0.1;
	const bool bPointsTowardsRectangleArea = -Tolerance <= AMDotAB && -Tolerance <= AMDotAD;
	const bool bIsAMSmallerThanSides = AMDotAB <= MagnituteSquaredAB + Tolerance && AMDotAD < MagnituteSquaredAD + Tolerance;

	return bPointsTowardsRectangleArea && bIsAMSmallerThanSides;
}

bool UDMXPixelMappingOutputComponent::OverlapsComponent(UDMXPixelMappingOutputComponent* Other) const
{
	// DEPRECATED 5.4
	if (Other)
	{
		FVector2D A;
		FVector2D B;
		FVector2D C;
		FVector2D D;
		CachedGeometry.GetEdgesAbsolute(A, B, C, D);

		return
			Other->IsOverPosition(A) ||
			Other->IsOverPosition(B) ||
			Other->IsOverPosition(C) ||
			Other->IsOverPosition(D);
	}

	return false;
}

FVector2D UDMXPixelMappingOutputComponent::GetPosition() const
{
	return CachedGeometry.GetPositionAbsolute();
}

void UDMXPixelMappingOutputComponent::SetPosition(const FVector2D& NewPosition) 
{
	PositionX = NewPosition.X;
	PositionY = NewPosition.Y;

	CachedGeometry.SetPositionAbsolute(NewPosition);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

FVector2D UDMXPixelMappingOutputComponent::GetPositionRotated() const
{
	return CachedGeometry.GetPositionRotatedAbsolute();
}

void UDMXPixelMappingOutputComponent::SetPositionRotated(FVector2D NewRotatedPosition)
{
	CachedGeometry.SetPositionRotatedAbsolute(NewRotatedPosition);

	PositionX = CachedGeometry.GetPositionAbsolute().X;
	PositionY = CachedGeometry.GetPositionAbsolute().Y;

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

void UDMXPixelMappingOutputComponent::GetEdges(FVector2D& A, FVector2D& B, FVector2D& C, FVector2D& D) const
{
	return CachedGeometry.GetEdgesAbsolute(A, B, C, D);
}

void UDMXPixelMappingOutputComponent::SetSize(const FVector2D& NewSize)
{
	SizeX = NewSize.X;
	SizeY = NewSize.Y;

	CachedGeometry.SetSizeAbsolute(NewSize);
}

FVector2D UDMXPixelMappingOutputComponent::GetSize() const
{
	return CachedGeometry.GetSizeAbsolute();
}

void UDMXPixelMappingOutputComponent::SetRotation(double NewRotation)
{
	Rotation = NewRotation;

	CachedGeometry.SetRotationAbsolute(NewRotation);

#if WITH_EDITOR
	EditorPositionWithRotation = CachedGeometry.GetPositionRotatedAbsolute();
#endif
}

double UDMXPixelMappingOutputComponent::GetRotation() const
{
	return CachedGeometry.GetRotationAbsolute();
}

void UDMXPixelMappingOutputComponent::InvalidatePixelMapRenderer()
{
	if (UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent())
	{
		RendererComponent->InvalidatePixelMapRenderer();
	}
}

UDMXPixelMappingRendererComponent* UDMXPixelMappingOutputComponent::FindRendererComponent() const
{
	UDMXPixelMappingRendererComponent* ParentRendererComponent = nullptr;

	if (GetParent())
	{
		for (UDMXPixelMappingBaseComponent* ParentComponent = GetParent(); ParentComponent; ParentComponent = ParentComponent->GetParent())
		{
			ParentRendererComponent = Cast<UDMXPixelMappingRendererComponent>(ParentComponent);
			if (ParentRendererComponent)
			{
				break;
			}
		}
	}

	return ParentRendererComponent;
}

#if WITH_EDITOR
void UDMXPixelMappingOutputComponent::ZOrderTopmost()
{
	UDMXPixelMappingRendererComponent* RendererComponent = FindRendererComponent();
	if (!RendererComponent)
	{
		return;
	}

	// Gather all components, sort them by zorder and rebase all
	constexpr bool bRecursive = true;

	TArray<UDMXPixelMappingOutputComponent*> AllOutputComoponents;
	RendererComponent->ForEachChildOfClass<UDMXPixelMappingOutputComponent>([&AllOutputComoponents](UDMXPixelMappingOutputComponent* Component)
		{
			AllOutputComoponents.Add(Component);
		}, bRecursive);
	Algo::SortBy(AllOutputComoponents, &UDMXPixelMappingOutputComponent::GetZOrder);

	int32 NewZOrder = 0;
	for (UDMXPixelMappingOutputComponent* OutputComponent : AllOutputComoponents)
	{
		OutputComponent->SetZOrder(++NewZOrder);
	}

	// Order this component and its children topmost
	SetZOrder(++NewZOrder);
	for (UDMXPixelMappingOutputComponent* OutputComponent : AllOutputComoponents)
	{
		// Increment zorder for all children that are a sibling of this component
		for (UDMXPixelMappingBaseComponent* Parent = OutputComponent->GetParent(); Parent; Parent = Parent->GetParent())
		{
			if (Parent == this)
			{
				OutputComponent->SetZOrder(++NewZOrder);
			}
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
