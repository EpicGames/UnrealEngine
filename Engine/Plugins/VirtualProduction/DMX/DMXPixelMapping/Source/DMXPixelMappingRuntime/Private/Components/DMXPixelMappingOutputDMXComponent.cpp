// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingOutputDMXComponent.h"

#include "Library/DMXEntityFixturePatch.h"


#if WITH_EDITOR
FLinearColor UDMXPixelMappingOutputDMXComponent::GetEditorColor() const
{
	if (bUsePatchColor)
	{
		if (const UDMXEntityFixturePatch* FixturePatch = FixturePatchRef.GetFixturePatch())
		{
			return FixturePatch->EditorColor;
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Editor color should not be publicly accessed. However it is ok to access it here.
	return EditorColor;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR
