// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingRendererComponent.h"

#include "Async/Async.h"
#include "Blueprint/UserWidget.h"
#include "Components/DMXPixelMappingComponentGeometryCache.h"
#include "Components/DMXPixelMappingFixtureGroupComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DMXPixelMapping.h"
#include "DMXPixelMappingPixelMapRenderer.h"
#include "DMXPixelMappingPreprocessRenderer.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"
#include "DMXPixelMappingTypes.h"
#include "DMXStats.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureRenderTarget2D.h"
#include "IDMXPixelMappingRenderer.h"
#include "IDMXPixelMappingRendererModule.h"
#include "Materials/MaterialInterface.h"
#include "Modulators/DMXModulator.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/Package.h"
#include "Widgets/Layout/SConstraintCanvas.h"

#if WITH_EDITOR
#include "LevelEditor.h"
#endif

DECLARE_CYCLE_STAT(TEXT("PixelMapping Render"), STAT_DMXPixelMappingRender, STATGROUP_DMX);
DECLARE_CYCLE_STAT(TEXT("PixelMapping SendDMX"), STAT_DMXPixelMappingSendDMX, STATGROUP_DMX);
DECLARE_CYCLE_STAT(TEXT("PixelMapping RenderInputTexture"), STAT_DMXPixelMappingRenderInputTexture, STATGROUP_DMX);


#define LOCTEXT_NAMESPACE "DMXPixelMappingRendererComponent"

UDMXPixelMappingRendererComponent::UDMXPixelMappingRendererComponent()
{
	static constexpr const TCHAR* UnrealEngineLogoPath = TEXT("Texture2D'/DMXPixelMapping/Textures/T_UnrealEngineLogo.T_UnrealEngineLogo'");
	ConstructorHelpers::FObjectFinder<UTexture> DefaultTexture(UnrealEngineLogoPath, LOAD_NoWarn);
	if (ensureAlwaysMsgf(DefaultTexture.Succeeded(), TEXT("Failed to load Default Texture for Pixel Mapping from '%s'"), UnrealEngineLogoPath))
	{
		InputTexture = DefaultTexture.Object;
		RendererType = EDMXPixelMappingRendererType::Texture;
	}

	PreprocessRenderer = CreateDefaultSubobject<UDMXPixelMappingPreprocessRenderer>("PreprocessRenderer");
	PixelMapRenderer = CreateDefaultSubobject<UDMXPixelMappingPixelMapRenderer>("PixelMapRenderer");

	Brightness = 1.0f;

	UDMXPixelMappingBaseComponent::GetOnComponentAdded().AddUObject(this, &UDMXPixelMappingRendererComponent::OnComponentAddedOrRemoved);
	UDMXPixelMappingBaseComponent::GetOnComponentRemoved().AddUObject(this, &UDMXPixelMappingRendererComponent::OnComponentAddedOrRemoved);
}

const FName& UDMXPixelMappingRendererComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("Renderer");
	return NamePrefix;
}

bool UDMXPixelMappingRendererComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	return Component && Component->GetClass() == UDMXPixelMappingRootComponent::StaticClass();
}

void UDMXPixelMappingRendererComponent::PostInitProperties()
{
	Super::PostInitProperties();

	if (!IsTemplate())
	{
		UpdatePreprocessRenderer();
	}
}

void UDMXPixelMappingRendererComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FDMXPixelMappingMainStreamObjectVersion::GUID) < FDMXPixelMappingMainStreamObjectVersion::RendererComponentHoldsLayoutRect)
		{	
			// Assets created before 5.4 do not store the layout rect, so they cannot follow the texture size by default
			bChildrenFollowSize = false;
		}
	}
}

void UDMXPixelMappingRendererComponent::PostLoad()
{
	Super::PostLoad();

	if (!IsTemplate())
	{
		UpdatePreprocessRenderer();
	}
}

#if WITH_EDITOR
void UDMXPixelMappingRendererComponent::PostEditUndo()
{
	Super::PostEditUndo();

	InvalidatePixelMapRenderer();
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void UDMXPixelMappingRendererComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	// Call the parent at the first place
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);

	if (PropertyChangedChainEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		UpdatePreprocessRenderer();
		LetChildrenFollowSize();
	}	
	
	const FName PropertyName = PropertyChangedChainEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, PixelFormat))
	{
		InvalidatePixelMapRenderer();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingRendererComponent, Brightness))
	{
		const TSharedPtr<IDMXPixelMappingRenderer>& Renderer = GetRenderer();
		if (Renderer.IsValid())
		{
			Renderer->SetBrightness(Brightness);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

void UDMXPixelMappingRendererComponent::UpdatePreprocessRenderer()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!PixelMappingRenderer_DEPRECATED.IsValid())
	{	
		// To keep support for deprecated functions, still create the old pixel mapping renderer 
		PixelMappingRenderer_DEPRECATED = IDMXPixelMappingRendererModule::Get().CreateRenderer();
		PixelMappingRenderer_DEPRECATED->SetBrightness(Brightness);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UserWidget = nullptr;

	const EPixelFormat Format = GetFormatFromDynamicRange();
	switch (RendererType)
	{
	case(EDMXPixelMappingRendererType::Texture):
		PreprocessRenderer->SetInputTexture(InputTexture.Get(), Format);
		break;

	case(EDMXPixelMappingRendererType::Material):
		PreprocessRenderer->SetInputMaterial(InputMaterial.Get(), Format);
		break;

	case(EDMXPixelMappingRendererType::UMG):
		UserWidget = CreateWidget(TryGetWorld(), InputWidget);
		PreprocessRenderer->SetInputUserWidget(UserWidget.Get(), Format);
		break;

	default:
		checkf(0, TEXT("Invalid Renderer Type in DMXPixelMappingRendererComponent"));
	}
}

void UDMXPixelMappingRendererComponent::InvalidatePixelMapRenderer()
{
	bInvalidatePixelMap = true;
}

bool UDMXPixelMappingRendererComponent::GetPixelMappingComponentModulators(FDMXEntityFixturePatchRef FixturePatchRef, TArray<UDMXModulator*>& DMXModulators)
{
	for (const UDMXPixelMappingBaseComponent* Child : Children)
	{
		if (const UDMXPixelMappingFixtureGroupComponent* GroupComponent = Cast<UDMXPixelMappingFixtureGroupComponent>(Child))
		{
			for (const UDMXPixelMappingBaseComponent* ChildOfGroupComponent : GroupComponent->Children)
			{
				if (const UDMXPixelMappingFixtureGroupItemComponent* GroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(ChildOfGroupComponent))
				{
					if (GroupItemComponent->FixturePatchRef.GetFixturePatch() == FixturePatchRef.GetFixturePatch())
					{
						DMXModulators = GroupItemComponent->Modulators;
						return true;
					}
				}
				else if (const UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(ChildOfGroupComponent))
				{
					if (MatrixComponent->FixturePatchRef.GetFixturePatch() == FixturePatchRef.GetFixturePatch())
					{
						DMXModulators = MatrixComponent->Modulators;
						return true;
					}
				}
			}
		}
	}

	return false;
}

TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>> UDMXPixelMappingRendererComponent::GetPixelMapRenderElements() const
{
	return PixelMapRenderElements;
}

void UDMXPixelMappingRendererComponent::ResetDMX(EDMXPixelMappingResetDMXMode ResetMode)
{
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent)
		{
			if (UDMXPixelMappingOutputComponent* Component = Cast<UDMXPixelMappingOutputComponent>(InComponent))
			{
				Component->ResetDMX(ResetMode);
			}
		}, false);
}

void UDMXPixelMappingRendererComponent::SendDMX()
{
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent)
		{
			if (UDMXPixelMappingOutputComponent* Component = Cast<UDMXPixelMappingOutputComponent>(InComponent))
			{
				SCOPE_CYCLE_COUNTER(STAT_DMXPixelMappingSendDMX);

				Component->SendDMX();
			}
		}, false);
}

void UDMXPixelMappingRendererComponent::Render()
{
	SCOPE_CYCLE_COUNTER(STAT_DMXPixelMappingRender);
	
	PreprocessRenderer->Render();

	UTexture* PreprocessedTexture = PreprocessRenderer->GetRenderedTexture();
	if (!PreprocessedTexture)
	{
		return;
	}

	// Always size to texture
	const FVector2D TextureSize(PreprocessedTexture->GetSurfaceWidth(), PreprocessedTexture->GetSurfaceHeight());
	if (GetSize() != TextureSize)
	{
		SetSize(TextureSize);
		LetChildrenFollowSize();

		bInvalidatePixelMap = true;
	}

	// Update render elements if invalidated
	if (bInvalidatePixelMap)
	{
		PixelMapRenderElements.Reset();

		constexpr bool bRecursive = true;
		ForEachChild([this](UDMXPixelMappingBaseComponent* Component)
			{
				if (UDMXPixelMappingFixtureGroupItemComponent* FixtureGroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(Component))
				{
					PixelMapRenderElements.Add(FixtureGroupItemComponent->GetOrCreatePixelMapRenderElement());
				}
				else if (UDMXPixelMappingMatrixCellComponent* MatrixCellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(Component))
				{
					PixelMapRenderElements.Add(MatrixCellComponent->GetOrCreatePixelMapRenderElement());
				}
			}, bRecursive);

		const EPixelFormat Format = GetFormatFromDynamicRange();
		PixelMapRenderer->SetElements(PixelMapRenderElements, Format);

		bInvalidatePixelMap = false;
	}

	PixelMapRenderer->Render(PreprocessedTexture, Brightness);
}

void UDMXPixelMappingRendererComponent::RenderAndSendDMX()
{
	Render();
	SendDMX();
}

FString UDMXPixelMappingRendererComponent::GetUserName() const
{
	if (!UserName.IsEmpty())
	{
		return UserName;
	}

	constexpr TCHAR NoSourceString[] = TEXT("None");
	switch (RendererType)
	{
	case(EDMXPixelMappingRendererType::Texture):
		return FString::Printf(TEXT("Pixel Mapping: %s"), InputTexture ? *InputTexture->GetName() : NoSourceString);

	case(EDMXPixelMappingRendererType::Material):
		return FString::Printf(TEXT("Pixel Mapping: %s"), InputMaterial ? *InputMaterial->GetName() : NoSourceString);

	case(EDMXPixelMappingRendererType::UMG):
		return FString::Printf(TEXT("Pixel Mapping: %s"), InputWidget ? *InputWidget->GetName() : NoSourceString);

	default:
		checkf(0, TEXT("Invalid Renderer Type in DMXPixelMappingRendererComponent"));
	}

	return FString();
}

UTexture* UDMXPixelMappingRendererComponent::GetRenderedInputTexture() const
{
	return PreprocessRenderer ? PreprocessRenderer->GetRenderedTexture() : nullptr;
}

void UDMXPixelMappingRendererComponent::OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component)
{
	InvalidatePixelMapRenderer();
}

void UDMXPixelMappingRendererComponent::LetChildrenFollowSize()
{
	UDMXPixelMapping* PixelMapping = GetPixelMapping();
	if (!PixelMapping || !bChildrenFollowSize || !PreprocessRenderer)
	{
		return;
	}

	const FVector2D NewSize = PreprocessRenderer->GetResultingSize2D();
	
	// Handle the case where the layout rect was never stored (new assets, and assets created before 5.4).
	// In this case, simply initialize the LayoutRect member.
	if (LayoutRect == FVector2D::ZeroVector)
	{
		LayoutRect = NewSize;
		return;
	}
	
	// Skip unchanged values, or if the current size is zero (no texture).
	if (NewSize == LayoutRect ||
		NewSize == FVector2D::ZeroVector)
	{
		return;
	}

#if WITH_EDITOR
	// In editor, temporarily disable scale children with parent
	TGuardValue Guard(PixelMapping->bEditorScaleChildrenWithParent, false);
#endif 

	// Scale position and size of all children
	const FVector2D Scalar = NewSize / LayoutRect;

	constexpr bool bRecursive = true;
	ForEachChild(
		[&Scalar](UDMXPixelMappingBaseComponent* Component)
		{
			if (UDMXPixelMappingOutputComponent* OutputComponent = Cast<UDMXPixelMappingOutputComponent>(Component))
			{
				if (OutputComponent->GetClass() == UDMXPixelMappingFixtureGroupComponent::StaticClass() ||
					OutputComponent->GetClass() == UDMXPixelMappingFixtureGroupItemComponent::StaticClass() ||
					OutputComponent->GetClass() == UDMXPixelMappingMatrixComponent::StaticClass())
				{
					OutputComponent->Modify();

					const FVector2D NewPosition = OutputComponent->GetPosition() * Scalar;
					OutputComponent->SetPosition(NewPosition);

					const FVector2D NewSize = OutputComponent->GetSize() * Scalar;
					OutputComponent->SetSize(NewSize);
				}

			}
		}, 
		bRecursive);

	// Remember the new layout rect
	Modify();
	LayoutRect = NewSize;
}

EPixelFormat UDMXPixelMappingRendererComponent::GetFormatFromDynamicRange() const
{
	UTexture2D* InputTexture2D = Cast<UTexture2D>(InputTexture);

	if (PixelFormat == EDMXPixelMappingRendererPixelFormat::Auto &&
		RendererType == EDMXPixelMappingRendererType::Texture &&
		InputTexture2D)
	{
		const EPixelFormat TexturePixelFormat = InputTexture2D->GetPixelFormat();
	
		// Propagonate the pixel format, if it is supported by render targets. 
		// As there doesn't seem to be an engine call to do this conversion,
		// use (the opposite) logic of GetPixelFormatFromRenderTargetFormat in TextureRenderTarget2D.h (5.4).
		if (TexturePixelFormat == PF_G8 ||
			TexturePixelFormat == PF_R8G8 ||
			TexturePixelFormat == PF_B8G8R8A8 ||
			TexturePixelFormat == PF_R16F ||
			TexturePixelFormat == PF_G16R16F ||
			TexturePixelFormat == PF_FloatRGBA ||
			TexturePixelFormat == PF_R32_FLOAT ||
			TexturePixelFormat == PF_G32R32F ||
			TexturePixelFormat == PF_A32B32G32R32F ||
			TexturePixelFormat == PF_A2B10G10R10)
		{
			return TexturePixelFormat;
		}
	}
	else if (PixelFormat == EDMXPixelMappingRendererPixelFormat::RGBA16F)
	{
		return PF_FloatRGBA;
	}

	return PF_B8G8R8A8;
}

UWorld* UDMXPixelMappingRendererComponent::TryGetWorld() const
{
	UWorld* World = nullptr;
	if (GIsEditor)
	{
#if WITH_EDITOR
		World = GEditor->GetEditorWorldContext().World();
#endif
	}
	else
	{
		World = GWorld;
	}

	return World;
}


///////////////////////////////
// DEPRECATED MEMBERS 5.3

const FIntPoint UDMXPixelMappingRendererComponent::MaxDownsampleBufferTargetSize_DEPRECATED = FIntPoint(4096);
const FLinearColor UDMXPixelMappingRendererComponent::ClearTextureColor_DEPRECATED = FLinearColor::Black;

UWorld* UDMXPixelMappingRendererComponent::GetWorld() const
{
	// DEPRECATED 5.3

	UWorld* World = nullptr;
	if (GIsEditor)
	{
#if WITH_EDITOR
		World = GEditor->GetEditorWorldContext().World();
#endif
	}
	else
	{
		World = GWorld;
	}

	return World;
}

#if WITH_EDITOR
void UDMXPixelMappingRendererComponent::RenderEditorPreviewTexture()
{
	// DEPRECATED 5.3

	if (!DownsampleBufferTarget_DEPRECATED)
	{
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TSharedPtr<IDMXPixelMappingRenderer>& Renderer = GetRenderer();
	if (!ensure(Renderer))
	{
		return;
	}

	TArray<FDMXPixelMappingDownsamplePixelPreviewParam> PixelPreviewParams;
	PixelPreviewParams.Reserve(DownsamplePixelCount_DEPRECATED);
	
	ForEachChild([this, &PixelPreviewParams](UDMXPixelMappingBaseComponent* InComponent) {
		if (UDMXPixelMappingOutputDMXComponent* Component = Cast<UDMXPixelMappingOutputDMXComponent>(InComponent))
		{
			FDMXPixelMappingDownsamplePixelPreviewParam PixelPreviewParam;
			PixelPreviewParam.ScreenPixelSize = Component->GetSize();
			PixelPreviewParam.ScreenPixelPosition = Component->GetPosition();
			PixelPreviewParam.DownsamplePosition = GetPixelPosition(Component->GetDownsamplePixelIndex());

			PixelPreviewParams.Add(MoveTemp(PixelPreviewParam));
		}
	}, true);

	Renderer->RenderPreview(GetPreviewRenderTarget()->GetResource(), DownsampleBufferTarget_DEPRECATED->GetResource(), MoveTemp(PixelPreviewParams));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

#if WITH_EDITOR
UTextureRenderTarget2D* UDMXPixelMappingRendererComponent::GetPreviewRenderTarget()
{	
	// DEPRECATED 5.3
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (PreviewRenderTarget_DEPRECATED == nullptr)
	{
		PreviewRenderTarget_DEPRECATED = CreateRenderTarget(TEXT("DMXPreviewRenderTarget_DEPRECATED"));
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return PreviewRenderTarget_DEPRECATED;
}
#endif // WITH_EDITOR


FIntPoint UDMXPixelMappingRendererComponent::GetPixelPosition(int32 InPosition) const
{
	// DEPRECATED 5.3

	const int32 YRows = InPosition / MaxDownsampleBufferTargetSize_DEPRECATED.X;
	return FIntPoint(InPosition % MaxDownsampleBufferTargetSize_DEPRECATED.X, YRows);
}

void UDMXPixelMappingRendererComponent::CreateOrUpdateDownsampleBufferTarget()
{
	// DEPRECATED 5.3

	// Create texture if it does not exists
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (DownsampleBufferTarget_DEPRECATED == nullptr)
	{
		DownsampleBufferTarget_DEPRECATED = CreateRenderTarget(TEXT("DMXPixelMappingDownsampleBufferTarget"));
	}

	const int32 PreviousDownsamplePixelCount = DownsamplePixelCount_DEPRECATED;
	const int32 TotalDownsamplePixelCount = GetTotalDownsamplePixelCount();

	if (TotalDownsamplePixelCount > 0 &&
		TotalDownsamplePixelCount != PreviousDownsamplePixelCount)
	{
		// Make sure total pixel count less then max texture size MaxDownsampleBufferTargetSize.X * MaxDownsampleBufferTargetSize.Y
		if (!ensure(TotalDownsamplePixelCount < (MaxDownsampleBufferTargetSize_DEPRECATED.X * MaxDownsampleBufferTargetSize_DEPRECATED.Y)))
		{
			return;
		}

		/**
			* if total pixel count less then max size x texture high equal 1
			* and texture widht dynamic from 1 up to MaxDownsampleBufferTargetSize.X
			* |0,1,2,3,4,5,...,n|
			*/
		if (TotalDownsamplePixelCount <= MaxDownsampleBufferTargetSize_DEPRECATED.X)
		{
			constexpr uint32 TargetSizeY = 1;
			DownsampleBufferTarget_DEPRECATED->ResizeTarget(TotalDownsamplePixelCount, TargetSizeY);
		}
		/**
		* if total pixel count more then max size x. At this case it should resize X and Y for buffer texture target
		* |0,1,2,3,4,5,..., MaxDownsampleBufferTargetSize.X|
		* |0,1,2,3,4,5,..., MaxDownsampleBufferTargetSize.X|
		* |................................................|
		* |MaxDownsampleBufferTargetSize.Y.................|
		*/
		else
		{
			const uint32 TargetSizeY = ((TotalDownsamplePixelCount - 1) / MaxDownsampleBufferTargetSize_DEPRECATED.X) + 1;
			DownsampleBufferTarget_DEPRECATED->ResizeTarget(MaxDownsampleBufferTargetSize_DEPRECATED.X, TargetSizeY);
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UDMXPixelMappingRendererComponent::AddPixelToDownsampleSet(FDMXPixelMappingDownsamplePixelParamsV2&& InDownsamplePixelParam)
{
	// DEPRECATED 5.3

	const FScopeLock Lock(&DownsampleBufferCS_DEPRECATED);
	DownsamplePixelParams_DEPRECATED.Emplace(InDownsamplePixelParam);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

int32 UDMXPixelMappingRendererComponent::GetDownsamplePixelNum()
{
	// DEPRECATED 5.3

	const FScopeLock Lock(&DownsampleBufferCS_DEPRECATED);
	return DownsamplePixelParams_DEPRECATED.Num();
}

void UDMXPixelMappingRendererComponent::SetDownsampleBuffer(TArray<FLinearColor>&& InDownsampleBuffer, FIntRect InRect)
{
	// DEPRECATED 5.3

	check(IsInRenderingThread());

	if (!bWasEverRendered_DEPRECATED)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (int32 PixelIndex = 0; PixelIndex < GetTotalDownsamplePixelCount(); PixelIndex++)
		{
			ResetColorDownsampleBufferPixel(PixelIndex);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		bWasEverRendered_DEPRECATED = true;
	}

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);
	DownsampleBuffer_DEPRECATED = MoveTemp(InDownsampleBuffer);
}

bool UDMXPixelMappingRendererComponent::GetDownsampleBufferPixel(const int32 InDownsamplePixelIndex, FLinearColor& OutLinearColor)
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);


	if (!DownsampleBuffer_DEPRECATED.IsValidIndex(InDownsamplePixelIndex))
	{
		return false;
	}

	OutLinearColor = DownsampleBuffer_DEPRECATED[InDownsamplePixelIndex];
	return true;
}

bool UDMXPixelMappingRendererComponent::GetDownsampleBufferPixels(const int32 InDownsamplePixelIndexStart, const int32 InDownsamplePixelIndexEnd, TArray<FLinearColor>& OutLinearColors)
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);

	// Could be out of the range when texture resizing on GPU thread
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsPixelRangeValid(InDownsamplePixelIndexStart, InDownsamplePixelIndexEnd))
	{
		return false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		OutLinearColors.Reset(InDownsamplePixelIndexEnd - InDownsamplePixelIndexStart + 1);
	for (int32 PixelIndex = InDownsamplePixelIndexStart; PixelIndex <= InDownsamplePixelIndexEnd; ++PixelIndex)
	{
		OutLinearColors.Add(DownsampleBuffer_DEPRECATED[PixelIndex]);
	}

	return true;
}

bool UDMXPixelMappingRendererComponent::ResetColorDownsampleBufferPixel(const int32 InDownsamplePixelIndex)
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);

	if (DownsampleBuffer_DEPRECATED.IsValidIndex(InDownsamplePixelIndex))
	{
		DownsampleBuffer_DEPRECATED[InDownsamplePixelIndex] = FLinearColor::Black;
		return true;
	}

	return false;
}

bool UDMXPixelMappingRendererComponent::ResetColorDownsampleBufferPixels(const int32 InDownsamplePixelIndexStart, const int32 InDownsamplePixelIndexEnd)
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);

	// Could be out of the range when texture resizing on GPU thread
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsPixelRangeValid(InDownsamplePixelIndexStart, InDownsamplePixelIndexEnd))
	{
		return false;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (int32 PixelIndex = InDownsamplePixelIndexStart; PixelIndex <= InDownsamplePixelIndexEnd; ++PixelIndex)
	{
		DownsampleBuffer_DEPRECATED[PixelIndex] = FLinearColor::Black;
	}

	return true;
}

void UDMXPixelMappingRendererComponent::EmptyDownsampleBuffer()
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);

	DownsampleBuffer_DEPRECATED.Empty();
}

int32 UDMXPixelMappingRendererComponent::GetTotalDownsamplePixelCount()
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);

	// Reset pixel counter
	DownsamplePixelCount_DEPRECATED = 0;

	// Count all pixels
	constexpr bool bIsRecursive = true;
	ForEachChildOfClass<UDMXPixelMappingOutputComponent>([&](UDMXPixelMappingOutputComponent* InComponent)
		{
			if (Cast<UDMXPixelMappingOutputDMXComponent>(InComponent))
			{
				DownsamplePixelCount_DEPRECATED++;
			}
		}, bIsRecursive);

	return DownsamplePixelCount_DEPRECATED;
}

bool UDMXPixelMappingRendererComponent::IsPixelRangeValid(const int32 InDownsamplePixelIndexStart, const int32 InDownsamplePixelIndexEnd) const
{
	// DEPRECATED 5.3

	FScopeLock ScopeLock(&DownsampleBufferCS_DEPRECATED);

	if (InDownsamplePixelIndexEnd >= InDownsamplePixelIndexStart &&
		DownsampleBuffer_DEPRECATED.IsValidIndex(InDownsamplePixelIndexStart) &&
		DownsampleBuffer_DEPRECATED.IsValidIndex(InDownsamplePixelIndexEnd))
	{
		return true;
	}

	return false;
}

#if WITH_EDITOR
void UDMXPixelMappingRendererComponent::ResizePreviewRenderTarget(uint32 InSizeX, uint32 InSizeY)
{	
	// DEPRECATED 5.3
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UTextureRenderTarget2D* Target = GetPreviewRenderTarget();

	if ((InSizeX > 0 && InSizeY > 0) && (Target->SizeX != InSizeX || Target->SizeY != InSizeY))
	{
		check(Target);
		Target->ResizeTarget(InSizeX, InSizeY);
		Target->UpdateResourceImmediate();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif // WITH_EDITOR

UTextureRenderTarget2D* UDMXPixelMappingRendererComponent::CreateRenderTarget(const FName& InBaseName)
{
	// DEPRECATED 5.3

	const FName TargetName = MakeUniqueObjectName(this, UTextureRenderTarget2D::StaticClass(), InBaseName);
	UTextureRenderTarget2D* RenderTarget = NewObject<UTextureRenderTarget2D>(this, TargetName);
	RenderTarget->ClearColor = ClearTextureColor_DEPRECATED;
	constexpr bool bInForceLinearGamma = false;
	RenderTarget->InitCustomFormat(GetSize().X, GetSize().Y, EPixelFormat::PF_B8G8R8A8, bInForceLinearGamma);

	return RenderTarget;
}

#undef LOCTEXT_NAMESPACE
