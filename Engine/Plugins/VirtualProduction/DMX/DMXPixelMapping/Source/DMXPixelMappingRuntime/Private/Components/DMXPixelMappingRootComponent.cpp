// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingRootComponent.h"
#include "Components/DMXPixelMappingRendererComponent.h"


void UDMXPixelMappingRootComponent::PostLoad()
{
	Super::PostLoad();

	ForEachChildOfClass<UDMXPixelMappingRendererComponent>([this](UDMXPixelMappingRendererComponent* RendererComponent)
		{
			CachedRendererComponentsByName.Add(RendererComponent->GetFName(), RendererComponent);
		}, false);

	GetOnComponentRenamed().AddUObject(this, &UDMXPixelMappingRootComponent::OnComponentRenamed);
}

void UDMXPixelMappingRootComponent::BeginDestroy()
{
	Super::BeginDestroy();

	// Explicitly reset the children, needed to clear entangled references in these
	Children.Reset();
}

const FName& UDMXPixelMappingRootComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("Root");
	return NamePrefix;
}

void UDMXPixelMappingRootComponent::AddChild(UDMXPixelMappingBaseComponent* InComponent)
{
	Super::AddChild(InComponent);

	if (UDMXPixelMappingRendererComponent* RendererComponent = Cast<UDMXPixelMappingRendererComponent>(InComponent))
	{
		CachedRendererComponentsByName.FindOrAdd(InComponent->GetFName(), RendererComponent);
	}
}

void UDMXPixelMappingRootComponent::RemoveChild(UDMXPixelMappingBaseComponent* InComponent)
{
	Super::RemoveChild(InComponent);

	CachedRendererComponentsByName.Remove(InComponent->GetFName());
}

void UDMXPixelMappingRootComponent::ResetDMX(EDMXPixelMappingResetDMXMode ResetMode)
{
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent)
	{
		if (UDMXPixelMappingRendererComponent* Component = Cast<UDMXPixelMappingRendererComponent>(InComponent))
		{
			Component->ResetDMX(ResetMode);
		}
	}, false);
}

void UDMXPixelMappingRootComponent::SendDMX()
{
	ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent)
	{
		if (UDMXPixelMappingRendererComponent* Component = Cast<UDMXPixelMappingRendererComponent>(InComponent))
		{
			Component->SendDMX();
		}
	}, false);
}

void UDMXPixelMappingRootComponent::Render()
{
	// Only ever render once per frame
	if (LastRenderedFrameNumber < GFrameNumber ||
		GFrameNumber == 0)
	{
		LastRenderedFrameNumber = GFrameNumber;
	
		ForEachChild([&](UDMXPixelMappingBaseComponent* InComponent)
			{
				if (UDMXPixelMappingRendererComponent* Component = Cast<UDMXPixelMappingRendererComponent>(InComponent))
				{
					Component->Render();
				}
			}, false);
	}
}

void UDMXPixelMappingRootComponent::RenderAndSendDMX()
{
	Render();
	SendDMX();
}

bool UDMXPixelMappingRootComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	return false;
}

UDMXPixelMappingRendererComponent* UDMXPixelMappingRootComponent::FindRendererComponentByName(const FName& Name) const
{
	if (const TObjectPtr<UDMXPixelMappingRendererComponent>* RendererComponentPtr = CachedRendererComponentsByName.Find(Name))
	{
		return *RendererComponentPtr;
	}

	return nullptr;
}

void UDMXPixelMappingRootComponent::OnComponentRenamed(UDMXPixelMappingBaseComponent* RenamedComponent)
{
	CachedRendererComponentsByName.Reset();
	ForEachChildOfClass<UDMXPixelMappingRendererComponent>([this](UDMXPixelMappingRendererComponent* RendererComponent)
		{
			CachedRendererComponentsByName.Add(RendererComponent->GetFName(), RendererComponent);
		}, false);
}

FString UDMXPixelMappingRootComponent::GetUserName() const
{
	if (UserName.IsEmpty())
	{
		return GetOuter()->GetName();
	}
	else
	{
		return UserName;
	}
}
