// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/DMXPixelMappingScreenComponent.h"

#include "DMXPixelMappingRuntimeLog.h"
#include "DMXPixelMappingTypes.h"
#include "DMXPixelMappingUtils.h"
#include "Components/DMXPixelMappingRendererComponent.h"
#include "IO/DMXOutputPort.h"
#include "IO/DMXOutputPortReference.h"
#include "IO/DMXPortManager.h"
#include "TextureResource.h"

#if WITH_EDITOR
#include "DMXPixelMappingComponentWidget.h"
#include "SDMXPixelMappingScreenComponentBox.h"
#else
#include "DMXPixelMappingTypes.h"
#endif // WITH_EDITOR

#include "Engine/Texture.h"


DECLARE_CYCLE_STAT(TEXT("Send Screen"), STAT_DMXPixelMaping_SendScreen, STATGROUP_DMXPIXELMAPPING);

#define LOCTEXT_NAMESPACE "DEPRECATED_DMXPixelMappingScreenComponent"

// DMXPixelMappingScreenComponent is fully deprecated, accept the deprecated implementation still makes use of it
PRAGMA_DISABLE_DEPRECATION_WARNINGS 

const FVector2D UDEPRECATED_DMXPixelMappingScreenComponent::MinGridSize = FVector2D(1.f);

UDEPRECATED_DMXPixelMappingScreenComponent::UDEPRECATED_DMXPixelMappingScreenComponent()
	: bSendToAllOutputPorts(true)
{
	SetSize(FVector2D(500.f, 500.f)); 
	
	NumXCells = 10;
	NumYCells = 10;

	PixelFormat = EDMXCellFormat::PF_RGB;
	bIgnoreAlphaChannel = true;

	LocalUniverse = 1;
	StartAddress = 1;
	PixelIntensity = 1;
	AlphaIntensity = 1;
	Distribution = EDMXPixelMappingDistribution::TopLeftToRight;
}

#if WITH_EDITOR
void UDEPRECATED_DMXPixelMappingScreenComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent)
{
	// Call the parent at the first place
	Super::PostEditChangeChainProperty(PropertyChangedChainEvent);
	
	if (PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, OutputPortReferences))
	{
		// Rebuild the set of ports
		OutputPorts.Reset();
		for (const FDMXOutputPortReference& OutputPortReference : OutputPortReferences)
		{
			const FDMXOutputPortSharedRef* OutputPortPtr = FDMXPortManager::Get().GetOutputPorts().FindByPredicate([&OutputPortReference](const FDMXOutputPortSharedRef& OutputPort) {
				return OutputPort->GetPortGuid() == OutputPortReference.GetPortGuid();
				});

			if (OutputPortPtr)
			{
				OutputPorts.Add(*OutputPortPtr);
			}
		}
	}
	
	if (PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, NumXCells) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, NumYCells) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, LocalUniverse) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, StartAddress) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, Distribution) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, PixelFormat) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, bShowAddresses) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDEPRECATED_DMXPixelMappingScreenComponent, bShowUniverse))
	{
		if (ScreenComponentBox.IsValid())
		{
			FDMXPixelMappingScreenComponentGridParams GridParams;
			GridParams.bShowAddresses = bShowAddresses;
			GridParams.bShowUniverse = bShowUniverse;
			GridParams.Distribution = Distribution;
			GridParams.NumXCells = NumXCells;
			GridParams.NumYCells = NumYCells;
			GridParams.PixelFormat = PixelFormat;
			GridParams.LocalUniverse = LocalUniverse;
			GridParams.StartAddress = StartAddress;

			ScreenComponentBox->RebuildGrid(GridParams);
		}
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
const FText UDEPRECATED_DMXPixelMappingScreenComponent::GetPaletteCategory()
{
	return LOCTEXT("Common", "Common");
}
#endif // WITH_EDITOR

const FName& UDEPRECATED_DMXPixelMappingScreenComponent::GetNamePrefix()
{
	static FName NamePrefix = TEXT("DEPRECATED DMX Screen");
	return NamePrefix;
}

void UDEPRECATED_DMXPixelMappingScreenComponent::AddColorToSendBuffer(const FColor& InColor, TArray<uint8>& OutDMXSendBuffer)
{
	if (PixelFormat == EDMXCellFormat::PF_R)
	{
		OutDMXSendBuffer.Add(InColor.R);
	}
	else if (PixelFormat == EDMXCellFormat::PF_G)
	{
		OutDMXSendBuffer.Add(InColor.G);
	}
	else if (PixelFormat == EDMXCellFormat::PF_B)
	{
		OutDMXSendBuffer.Add(InColor.B);
	}
	else if (PixelFormat == EDMXCellFormat::PF_RG)
	{
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.G);
	}
	else if (PixelFormat == EDMXCellFormat::PF_RB)
	{
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.B);
	}
	else if (PixelFormat == EDMXCellFormat::PF_GB)
	{
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.B);
	}
	else if (PixelFormat == EDMXCellFormat::PF_GR)
	{
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.R);
	}
	else if (PixelFormat == EDMXCellFormat::PF_BR)
	{
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(InColor.R);
	}
	else if (PixelFormat == EDMXCellFormat::PF_BG)
	{
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(InColor.G);
	}
	else if (PixelFormat == EDMXCellFormat::PF_RGB)
	{
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.B);
	}
	else if (PixelFormat == EDMXCellFormat::PF_BRG)
	{
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.G);
	}
	else if (PixelFormat == EDMXCellFormat::PF_GRB)
	{
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.B);
	}
	else if (PixelFormat == EDMXCellFormat::PF_GBR)
	{
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(InColor.R);
	}
	else if (PixelFormat == EDMXCellFormat::PF_RGBA)
	{
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(bIgnoreAlphaChannel ? 0 : InColor.A);
	}
	else if (PixelFormat == EDMXCellFormat::PF_GBRA)
	{
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(bIgnoreAlphaChannel ? 0 : InColor.A);
	}
	else if (PixelFormat == EDMXCellFormat::PF_BRGA)
	{
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(bIgnoreAlphaChannel ? 0 : InColor.A);
	}
	else if (PixelFormat == EDMXCellFormat::PF_GRBA)
	{
		OutDMXSendBuffer.Add(InColor.G);
		OutDMXSendBuffer.Add(InColor.R);
		OutDMXSendBuffer.Add(InColor.B);
		OutDMXSendBuffer.Add(bIgnoreAlphaChannel ? 0 : InColor.A);
	}
}

UDMXPixelMappingRendererComponent* UDEPRECATED_DMXPixelMappingScreenComponent::GetRendererComponent() const
{
	return Cast<UDMXPixelMappingRendererComponent>(GetParent());
}

void UDEPRECATED_DMXPixelMappingScreenComponent::SendDMX()
{
	SCOPE_CYCLE_COUNTER(STAT_DMXPixelMaping_SendScreen);

	UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent();
	if (!ensure(RendererComponent))
	{
		return;
	}

	if (LocalUniverse < 0)
	{
		UE_LOGF(LogDMXPixelMappingRuntime, Warning, "LocalUniverse < 0");
		return;
	}

	// Helper to send to the correct ports, depending on bSendToAllOutputPorts
	auto SendDMXToPorts = [this](int32 InUniverseID, const TMap<int32, uint8>& InChannelToValueMap)
	{
		if (bSendToAllOutputPorts)
		{
			for (const FDMXOutputPortSharedRef& OutputPort : FDMXPortManager::Get().GetOutputPorts())
			{
				OutputPort->SendDMX(InUniverseID, InChannelToValueMap);
			}
		}
		else
		{
			for (const FDMXOutputPortSharedRef& OutputPort : OutputPorts)
			{
				OutputPort->SendDMX(InUniverseID, InChannelToValueMap);
			}
		}
	};

	TArray<FLinearColor> UnsortedList; 
	if (RendererComponent->GetDownsampleBufferPixels(PixelDownsamplePositionRange.Key, PixelDownsamplePositionRange.Value, UnsortedList))
	{
		TArray<FLinearColor> SortedList;
		SortedList.Reserve(UnsortedList.Num());
		FDMXPixelMappingUtils::TextureDistributionSort<FLinearColor>(Distribution, NumXCells, NumYCells, UnsortedList, SortedList);

		// Sending only if there enough space at least for one pixel
		if (!FDMXPixelMappingUtils::CanFitCellIntoChannels(PixelFormat, StartAddress))
		{
			return;
		}

		// Prepare Universes for send
		TArray<uint8> SendBuffer;
		for (const FLinearColor& LinearColor : SortedList)
		{
			constexpr bool bUseSRGB = true;
			FColor Color = LinearColor.ToFColor(bUseSRGB);
		
			const float MaxValue = 255.f;
			Color.R = static_cast<uint8>(FMath::Min(Color.R * PixelIntensity, MaxValue));
			Color.G = static_cast<uint8>(FMath::Min(Color.G * PixelIntensity, MaxValue));
			Color.B = static_cast<uint8>(FMath::Min(Color.B * PixelIntensity, MaxValue));
			Color.A = static_cast<uint8>(FMath::Min(Color.A * AlphaIntensity, MaxValue));;
			AddColorToSendBuffer(Color, SendBuffer);
		}

		// Start sending
		const uint32 UniverseMaxChannels = FDMXPixelMappingUtils::GetUniverseMaxChannels(PixelFormat, StartAddress);
		uint32 SendDMXIndex = StartAddress;
		int32 UniverseToSend = LocalUniverse;
		const int32 SendBufferNum = SendBuffer.Num();
		TMap<int32, uint8> ChannelToValueMap;
		for (int32 FragmentMapIndex = 0; FragmentMapIndex < SendBufferNum; FragmentMapIndex++)
		{
			// ready to send here
			if (SendDMXIndex > UniverseMaxChannels)
			{
				SendDMXToPorts(UniverseToSend, ChannelToValueMap);

				// Now reset
				ChannelToValueMap.Empty();
				SendDMXIndex = StartAddress;
				UniverseToSend++;
			}

			// should be channels from 1...UniverseMaxChannels
			ChannelToValueMap.Add(SendDMXIndex, SendBuffer[FragmentMapIndex]);
			if (!ensureMsgf(SendDMXIndex < UniverseMaxChannels || SendDMXIndex > 0, TEXT("Pixel Mapping Screen Component trying to send out of universe range.")))
			{
				break;
			}

			// send dmx if next iteration is the last one
			if ((SendBufferNum > FragmentMapIndex + 1) == false)
			{
				SendDMXToPorts(UniverseToSend, ChannelToValueMap);
				break;
			}

			SendDMXIndex++;
		}
	}
}

void UDEPRECATED_DMXPixelMappingScreenComponent::QueueDownsample()
{
	// Queue pixels into the downsample rendering
	UDMXPixelMappingRendererComponent* RendererComponent = GetRendererComponent();
	if (!ensure(RendererComponent))
	{
		return;
	}

	UTexture* InputTexture = RendererComponent->GetRenderedInputTexture();
	if (!ensure(InputTexture))
	{
		return;
	}

	const uint32 TextureSizeX = InputTexture->GetResource()->GetSizeX();
	const uint32 TextureSizeY = InputTexture->GetResource()->GetSizeY();
	check(TextureSizeX > 0 && TextureSizeY > 0);
	
	constexpr bool bStaticCalculateUV = true;
	const FVector2D SizePixel(GetSize().X / NumXCells, GetSize().Y / NumYCells);
	const FVector2D UVSize(SizePixel.X / TextureSizeX, SizePixel.Y / TextureSizeY);
	const FVector2D UVCellSize = UVSize / 2.f;
	const int32 PixelNum = NumXCells * NumYCells;
	const FVector4 PixelFactor(1.f, 1.f, 1.f, 1.f);
	const FIntVector4 InvertPixel(0);

	// Start of downsample index
	PixelDownsamplePositionRange.Key = RendererComponent->GetDownsamplePixelNum();

	int32 IterationCount = 0;
	ForEachPixel([&](const int32 InXYIndex, const int32 XIndex, const int32 YIndex)
        {
            const FIntPoint PixelPosition = RendererComponent->GetPixelPosition(InXYIndex + PixelDownsamplePositionRange.Key);
            const FVector2D UV = FVector2D((GetPosition().X + SizePixel.X * XIndex) / TextureSizeX, (GetPosition().Y + SizePixel.Y * YIndex) / TextureSizeY);

            FDMXPixelMappingDownsamplePixelParamsV2 DownsamplePixelParam
            {
                PixelPosition,
                UV,
                UVSize,
                UVCellSize,
                CellBlendingQuality,
                bStaticCalculateUV
            };

            RendererComponent->AddPixelToDownsampleSet(MoveTemp(DownsamplePixelParam));

            IterationCount = InXYIndex;
        });

	// End of downsample index
	PixelDownsamplePositionRange.Value = PixelDownsamplePositionRange.Key + IterationCount;
}

void UDEPRECATED_DMXPixelMappingScreenComponent::RenderWithInputAndSendDMX()
{
	// DEPRECATED 5.3
	RenderAndSendDMX();
}

bool UDEPRECATED_DMXPixelMappingScreenComponent::CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
{
	return Component && Component->IsA<UDMXPixelMappingRendererComponent>();
}

const FVector2D UDEPRECATED_DMXPixelMappingScreenComponent::GetScreenPixelSize() const
{
	return FVector2D(GetSize().X / NumXCells, GetSize().Y / NumYCells);
}

void UDEPRECATED_DMXPixelMappingScreenComponent::ForEachPixel(ForEachPixelCallback InCallback)
{
	int32 IndexXY = 0;
	for (int32 NumYIndex = 0; NumYIndex < NumYCells; ++NumYIndex)
	{
	for (int32 NumXIndex = 0; NumXIndex < NumXCells; ++NumXIndex)
	{
			InCallback(IndexXY, NumXIndex, NumYIndex);
			IndexXY++;
		}
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
