// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMapping.h"

#include "Components/DMXPixelMappingRendererComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingMatrixCellComponent.h"
#include "DMXPixelMappingMainStreamObjectVersion.h"
#include "Library/DMXEntityFixturePatch.h"
#include "UObject/LinkerLoad.h"

#if WITH_EDITOR
#include "DMXPixelMappingComponentWidget.h"
#include "SDMXPixelMappingComponentBox.h"
#endif // WITH_EDITOR


UDMXPixelMapping::UDMXPixelMapping()
	: ResetDMXMode(EDMXPixelMappingResetDMXMode::DoNotSendValues)
{
#if WITH_EDITOR
	SnapGridColor = FLinearColor::White.CopyWithNewOpacity(.12f);
#endif // WITH_EDITOR
}

void UDMXPixelMapping::StartSendingDMX()
{
	bIsPaused = false;
	bIsSendingDMX = true;
}

void UDMXPixelMapping::StopSendingDMX()
{
	if (RootComponent)
	{
		RootComponent->ResetDMX(ResetDMXMode);
	}

	bIsPaused = false;
	bIsSendingDMX = false;
}

void UDMXPixelMapping::PauseSendingDMX()
{
	bIsPaused = true;
	bIsSendingDMX = false;
}

void UDMXPixelMapping::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FDMXPixelMappingMainStreamObjectVersion::GUID);
}

void UDMXPixelMapping::PostLoad()
{
	Super::PostLoad();

	CreateOrLoadObjects();
}

void UDMXPixelMapping::PreloadWithChildren()
{
	ConditionalPreload();

	ForEachComponent([this](UDMXPixelMappingBaseComponent* InComponent)
	{
		if (InComponent)
		{
			InComponent->ConditionalPreload();
		}
	});
}

void UDMXPixelMapping::DestroyInvalidComponents()
{
	TArray<UDMXPixelMappingBaseComponent*> CachedComponents;
	ForEachComponent([&CachedComponents](UDMXPixelMappingBaseComponent* InComponent)
		{
			CachedComponents.Add(InComponent);
		});

	for (UDMXPixelMappingBaseComponent* Component : CachedComponents)
	{
		if (!Component->ValidateProperties())
		{
			TArray<UDMXPixelMappingBaseComponent*> CachedChildren = Component->Children;
			for (UDMXPixelMappingBaseComponent* Child : CachedChildren)
			{
				Component->RemoveChild(Child);
			}

			if (Component->GetParent())
			{
				Component->GetParent()->RemoveChild(Component);
			}
		}
	}
}

void UDMXPixelMapping::CreateOrLoadObjects()
{
	// Create RootComponent if it doesn't exist.
	if (RootComponent == nullptr)
	{
		UDMXPixelMappingRootComponent* DefaultComponent = UDMXPixelMappingRootComponent::StaticClass()->GetDefaultObject<UDMXPixelMappingRootComponent>();
		FName UniqueName = MakeUniqueObjectName(this, UDMXPixelMappingRootComponent::StaticClass(), DefaultComponent->GetNamePrefix());

		RootComponent = NewObject<UDMXPixelMappingRootComponent>(this, UDMXPixelMappingRootComponent::StaticClass(), UniqueName, RF_Transactional);
	}
}

UDMXPixelMappingBaseComponent* UDMXPixelMapping::FindComponent(UDMXEntityFixturePatch* FixturePatch) const
{
	if (!FixturePatch || !FixturePatch->IsValidLowLevel())
	{
		return nullptr;
	}

	UDMXPixelMappingBaseComponent* Component = nullptr;

	ForEachComponent([&Component, FixturePatch](UDMXPixelMappingBaseComponent* InComponent) {
		if (UDMXPixelMappingFixtureGroupItemComponent* GroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(InComponent))
		{
			if (GroupItemComponent->IsValidLowLevel() && GroupItemComponent->FixturePatchRef.GetFixturePatch() == FixturePatch)
			{
				Component = GroupItemComponent;
				return;
			}
		}
		else if (UDMXPixelMappingMatrixCellComponent* MatrixCellComponent = Cast<UDMXPixelMappingMatrixCellComponent>(InComponent))
		{
			if (UDMXPixelMappingMatrixComponent* ParentMatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(InComponent))
			{
				if (MatrixCellComponent->IsValidLowLevel() && ParentMatrixComponent->FixturePatchRef == FixturePatch)
				{
					Component = MatrixCellComponent;
					return;
				}
			}
		}
	});

	return Component;
}

UDMXPixelMappingBaseComponent* UDMXPixelMapping::FindComponent(const FName& InName) const
{
	UDMXPixelMappingBaseComponent* FoundComponent = nullptr;

	ForEachComponent([&](UDMXPixelMappingBaseComponent* InComponent) {
		if (InComponent->GetFName() == InName)
		{
			FoundComponent = InComponent;
		}
	});

	return FoundComponent;
}

void UDMXPixelMapping::RemoveComponent(UDMXPixelMappingBaseComponent* InComponent)
{
	ensureMsgf(InComponent, TEXT("Trying to remove invalid component."));

	if (InComponent)
	{
#if WITH_EDITOR
		ensureMsgf(InComponent->GetParent(), TEXT("Trying to remove component %s but it has no valid parent."), *InComponent->GetUserName());
#endif

		if (UDMXPixelMappingBaseComponent* Parent = InComponent->GetParent())
		{
			if (InComponent && InComponent != RootComponent)
			{
				Parent->RemoveChild(InComponent);
			}
		}
	}
}

ETickableTickType UDMXPixelMapping::GetTickableTickType() const
{
	return ETickableTickType::Conditional;
}

bool UDMXPixelMapping::IsTickable() const
{
	return bIsSendingDMX && !bIsPaused;
}

void UDMXPixelMapping::Tick(float DeltaTime)
{
	TArray<UDMXPixelMappingRendererComponent*> RendererComponents;
	GetAllComponentsOfClass<UDMXPixelMappingRendererComponent>(RendererComponents);

	for (UDMXPixelMappingRendererComponent* RendererComponent : RendererComponents)
	{
		if (RendererComponent)
		{
			RendererComponent->RenderAndSendDMX();
		}
	}
}

void UDMXPixelMapping::ForEachComponent(TComponentPredicate Predicate) const
{
	if (RootComponent)
	{
		Predicate(RootComponent);

		UDMXPixelMappingBaseComponent::ForComponentAndChildren(RootComponent, Predicate);
	}
}
