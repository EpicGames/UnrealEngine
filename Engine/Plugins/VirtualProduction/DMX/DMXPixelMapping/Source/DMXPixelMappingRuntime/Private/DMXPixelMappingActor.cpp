// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingActor.h"

#include "Components/DMXPixelMappingBaseComponent.h"
#include "Components/SceneComponent.h"
#include "DMXPixelMapping.h"

ADMXPixelMappingActor::ADMXPixelMappingActor()
{
	RootSceneComponent = CreateDefaultSubobject<USceneComponent>("SceneComponent");
	RootComponent = RootSceneComponent;
}

void ADMXPixelMappingActor::SetPixelMapping(UDMXPixelMapping* InPixelMapping)
{
	if (!ensureAlwaysMsgf(!PixelMapping, TEXT("Tried to set a Pixel Mapping for %s, but it already has one set. Changing the pixel mapping is not currently supported."), *GetName()))
	{
		return;
	}

	PixelMapping = InPixelMapping;
}

void ADMXPixelMappingActor::StartSendingDMX()
{
	if (PixelMapping)
	{
		PixelMapping->StartSendingDMX();
	}
}

void ADMXPixelMappingActor::StopSendingDMX()
{
	if (PixelMapping)
	{
		PixelMapping->StopSendingDMX();
	}
}

void ADMXPixelMappingActor::PauseSendingDMX()
{
	if (PixelMapping)
	{
		PixelMapping->PauseSendingDMX();
	}
}

bool ADMXPixelMappingActor::IsSendingDMX() const
{
	return PixelMapping ? PixelMapping->IsSendingDMX() : false;
}

void ADMXPixelMappingActor::SetStopMode(EDMXPixelMappingResetDMXMode ResetMode)
{
	if (PixelMapping)
	{
		PixelMapping->SetResetDMXMode(ResetMode);
	}
}

void ADMXPixelMappingActor::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	ApplySendDMXInEditorState();
#endif // WITH_EDITOR
}

void ADMXPixelMappingActor::BeginPlay()
{
	Super::BeginPlay();

	if (bAutoActivate)
	{
		StartSendingDMX();
	}

#if WITH_EDITOR
	bIsPlayInWorld = true;
#endif 
}

void ADMXPixelMappingActor::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

#if WITH_EDITOR
	bIsPlayInWorld = false;
	ApplySendDMXInEditorState();
#else
	StopSendingDMX();
#endif
}

#if WITH_EDITOR
void ADMXPixelMappingActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(ADMXPixelMappingActor, bSendDMXInEditor) ||
		PropertyName == GET_MEMBER_NAME_CHECKED(ADMXPixelMappingActor, bAutoActivate))
	{
		ApplySendDMXInEditorState();
	}
}
#endif // WITH_EDITOR

#if WITH_EDITOR
void ADMXPixelMappingActor::ApplySendDMXInEditorState()
{
	if (PixelMapping)
	{
		const bool bShouldSendDMX = bAutoActivate && (bIsPlayInWorld || bSendDMXInEditor);
		if (bShouldSendDMX)
		{
			PixelMapping->StartSendingDMX();
		}
		else
		{
			PixelMapping->StopSendingDMX();
		}
	}
}
#endif // WITH_EDITOR
