// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRuntimeLog.h"

DEFINE_LOG_CATEGORY(LogDMXPixelMappingRuntime)