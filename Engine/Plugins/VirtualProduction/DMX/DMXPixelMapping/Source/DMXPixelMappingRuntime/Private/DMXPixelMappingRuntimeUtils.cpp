// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMXPixelMappingRuntimeUtils.h"

#include "DMXConversions.h"
#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Modulators/DMXModulator.h"



void FDMXPixelMappingRuntimeUtils::ConvertNormalizedAttributeValueToChannelValue(UDMXEntityFixturePatch* InFixturePatch, const FDMXAttributeName& InAttributeName, float InNormalizedValue, TMap<int32, uint8>& InOutChannelToValueMap)
{
	if (InFixturePatch)
	{
		const FDMXFixtureMode* ModePtr = InFixturePatch->GetActiveMode();

		if (ModePtr)
		{
			const int32 StartingChannel = InFixturePatch->GetStartingChannel();

			// Build the value map from the attributes
			TMap<int32, uint8> ChannelToValueMap;

			const FDMXFixtureFunction* FunctionPtr = ModePtr->Functions.FindByPredicate([InAttributeName](const FDMXFixtureFunction& Function) {
				return Function.Attribute.Name == InAttributeName;
				});

			if (FunctionPtr)
			{
				const int32 AttributeStartingChannel = StartingChannel + FunctionPtr->Channel - 1; // +FunctionPtr->ChannelOffset;

				uint8 NumBytes = static_cast<uint8>(FunctionPtr->DataType) + 1;

				TArray<uint8> Bytes = FDMXConversions::NormalizedDMXValueToByteArray(InNormalizedValue, FunctionPtr->DataType, FunctionPtr->bUseLSBMode);

				for (int32 IndexByte = 0; IndexByte < Bytes.Num(); IndexByte++)
				{
					InOutChannelToValueMap.Add(AttributeStartingChannel + IndexByte, Bytes[IndexByte]);
				}
			}
		}
	}
}

#if WITH_EDITOR
void FDMXPixelMappingRuntimeUtils::HandleModulatorPropertyChange(UDMXPixelMappingBaseComponent* Component, FPropertyChangedChainEvent& PropertyChangedChainEvent, const TArray<TSubclassOf<UDMXModulator>>& ModulatorClasses, TArray<UDMXModulator*>& InOutModulators)
{
	UDMXPixelMappingFixtureGroupItemComponent* GroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(Component);
	UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(Component);

	ensureMsgf(GroupItemComponent || MatrixComponent, TEXT("Can only call FDMXPixelMappingRuntimeUtils::HandleModulatorPropertyChange with matrix or group item components."));

	FName PropertyName = PropertyChangedChainEvent.GetPropertyName();

	if (PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingFixtureGroupItemComponent, ModulatorClasses) ||
		PropertyChangedChainEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingMatrixComponent, ModulatorClasses))
	{
		// Keep modulators sync with the ModulatorClass array
		const int32 ArrayIndex = PropertyChangedChainEvent.GetArrayIndex(PropertyName.ToString());
		if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayClear)
		{
			InOutModulators.Empty();
		}
		else if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::ArrayRemove)
		{
			if (InOutModulators.IsValidIndex(ArrayIndex))
			{
				InOutModulators.RemoveAt(ArrayIndex);
			}
		}
		else if (PropertyChangedChainEvent.ChangeType == EPropertyChangeType::Duplicate)
		{
			// Keep modulators sync with the ModulatorClass array
			if (InOutModulators.IsValidIndex(ArrayIndex))
			{
				UDMXModulator* DuplicateModulator = DuplicateObject<UDMXModulator>(InOutModulators[ArrayIndex], Component);

				if (InOutModulators.IsValidIndex(ArrayIndex + 1))
				{
					InOutModulators.Insert(DuplicateModulator, ArrayIndex + 1);
				}
				else
				{
					InOutModulators.Add(DuplicateModulator);
				}
			}
		}

		// In any case sync the arrays
		if (InOutModulators.Num() > ModulatorClasses.Num())
		{
			// Same Size
			InOutModulators.SetNum(ModulatorClasses.Num());
		}

		// Same classes
		int32 ModulatorIndex = 0;
		for (TSubclassOf<UDMXModulator> ModulatorClass : ModulatorClasses)
		{
			if (ModulatorClass.Get())
			{
				if (!InOutModulators.IsValidIndex(ModulatorIndex))
				{
					UDMXModulator* NewModulator = NewObject<UDMXModulator>(Component, ModulatorClass, NAME_None, RF_Transactional | RF_Public);
					InOutModulators.Add(NewModulator);
				}
				else if (InOutModulators[ModulatorIndex]->GetClass() != ModulatorClass)
				{
					InOutModulators[ModulatorIndex] = NewObject<UDMXModulator>(Component, ModulatorClass, NAME_None, RF_Transactional | RF_Public);
				}

				ModulatorIndex++;
			}
			else if (InOutModulators.IsValidIndex(ModulatorIndex))
			{
				InOutModulators.RemoveAt(ModulatorIndex);
			}
		}
	}
}
#endif // WITH_EDITOR
