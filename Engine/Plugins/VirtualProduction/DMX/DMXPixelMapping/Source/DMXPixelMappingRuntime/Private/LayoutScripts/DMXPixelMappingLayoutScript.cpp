// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutScripts/DMXPixelMappingLayoutScript.h"

#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"
#include "MVR/Types/DMXMVRFixtureNode.h"


FDMXPixelMappingLayoutToken::FDMXPixelMappingLayoutToken(TWeakObjectPtr<UDMXPixelMappingOutputComponent> OutputComponent)
{
	if (!OutputComponent.IsValid())
	{
		return;
	}

	Component = OutputComponent;

	PositionX = OutputComponent->GetPosition().X;
	PositionY = OutputComponent->GetPosition().Y;
	SizeX = OutputComponent->GetSize().X;
	SizeY = OutputComponent->GetSize().Y;

	// Layout tokens are created with an unrotated parent. Hence the relative rotation is the current rotation.
	RelativeRotation = OutputComponent->GetRotation();

	InitializeFixtureID();
}

void FDMXPixelMappingLayoutToken::InitializeFixtureID()
{
	UDMXEntityFixturePatch* FixturePatch = nullptr;
	if (UDMXPixelMappingFixtureGroupItemComponent* GroupItemComponent = Cast<UDMXPixelMappingFixtureGroupItemComponent>(Component.Get()))
	{
		FixturePatch = GroupItemComponent->FixturePatchRef.GetFixturePatch();
	}
	else if (UDMXPixelMappingMatrixComponent* MatrixComponent = Cast<UDMXPixelMappingMatrixComponent>(Component.Get()))
	{
		FixturePatch = MatrixComponent->FixturePatchRef.GetFixturePatch();
	}

	if (!FixturePatch)
	{
		return;
	}

	const FGuid& MVRUUID = FixturePatch->GetMVRFixtureUUID();
	if (!MVRUUID.IsValid())
	{
		return;
	}

	const UDMXLibrary* DMXLibrary = FixturePatch->GetParentLibrary();
	if (!DMXLibrary)
	{
		return;
	}

	// We can use the lazy General Scene Description
	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary->GetLazyGeneralSceneDescription();
	if (!GeneralSceneDescription)
	{
		return;
	}
	
	UDMXMVRFixtureNode* FixtureNode = GeneralSceneDescription->FindFixtureNode(MVRUUID);
	if (!FixtureNode)
	{
		return;
	}

	LexTryParseString(FixtureID, *FixtureNode->FixtureID);
}
