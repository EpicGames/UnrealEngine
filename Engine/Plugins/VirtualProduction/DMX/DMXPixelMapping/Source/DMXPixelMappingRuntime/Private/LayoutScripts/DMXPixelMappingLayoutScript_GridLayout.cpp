// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutScripts/DMXPixelMappingLayoutScript_GridLayout.h"

#include "DMXRuntimeUtils.h"

#include "Types/SlateEnums.h"


#if WITH_EDITOR
void UDMXPixelMappingLayoutScript_GridLayout::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		return;
	}

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingLayoutScript_GridLayout, Columns))
	{
		ClampColumns();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UDMXPixelMappingLayoutScript_GridLayout, Rows))
	{
		ClampRows();
	}
}
#endif // WITH_EDITOR

void UDMXPixelMappingLayoutScript_GridLayout::Layout_Implementation(const TArray<FDMXPixelMappingLayoutToken>& InTokens, TArray<FDMXPixelMappingLayoutToken>& OutTokens)
{
	// Create an array of position IDs sorted by Distribution 
	TArray<int32> PositionIDs;

	const int32 NumCells = Columns * Rows;
	PositionIDs.Reserve(NumCells);
	for (int32 ID = 0; ID < NumCells; ID++)
	{
		PositionIDs.Add(ID);
	}
	TArray<int32> DistributedPositionIDs;
	DistributedPositionIDs.Reserve(NumCells);
	FDMXRuntimeUtils::PixelMappingDistributionSort(Distribution, Columns, Rows, PositionIDs, DistributedPositionIDs);

	// Find a size for each Token, no padding
	if (Columns == 0 || Rows == 0)
	{
		return;
	}

	// No empty rows:
	// Example, user sets the grid to 1x9 interactively, but there's 10 cells (tokens).
	// Hence UI flips to 2x9, to respect his 9 value, and compensate for the missing cell.
	// Now the Grid is 2x9 = 18 cells, but we have only 10 cells. The last 4 columns remain empty.
	const int32 ActualRows = FMath::RoundFromZero((float)NumTokens / Columns);

	// Cell Size
	float ComponentSizeX = ParentComponentSize.X / Columns;
	float ComponentSizeY = ParentComponentSize.Y / ActualRows;

	// Copy position for alignment 
	FVector2D AbsolutePosition = ParentComponentPosition;

	// Horizontal Alignment
	if (HorizontalAlignment != EHorizontalAlignment::HAlign_Fill)
	{
		if (ComponentSizeX > ComponentSizeY)
		{
			ComponentSizeX = ComponentSizeY;
		}

		if (HorizontalAlignment == EHorizontalAlignment::HAlign_Center)
		{
			AbsolutePosition.X = AbsolutePosition.X + (ParentComponentSize.X - ComponentSizeX * Columns) / 2.f;
		}
		else if (HorizontalAlignment == EHorizontalAlignment::HAlign_Right)
		{
			AbsolutePosition.X = AbsolutePosition.X + ParentComponentSize.X - ComponentSizeX * Columns;
		}
	}

	// Vertical alignment
	if (VerticalAlignment != EVerticalAlignment::VAlign_Fill)
	{
		if (ComponentSizeY > ComponentSizeX)
		{
			ComponentSizeY = ComponentSizeX;
		}

		if (VerticalAlignment == EVerticalAlignment::VAlign_Center)
		{
			AbsolutePosition.Y = AbsolutePosition.Y + (ParentComponentSize.Y - ComponentSizeY * ActualRows) / 2.f;
		}
		else if (VerticalAlignment == EVerticalAlignment::VAlign_Bottom)
		{
			AbsolutePosition.Y = AbsolutePosition.Y + ParentComponentSize.Y - ComponentSizeY * ActualRows;
		}
	}

	// Clamp padding so it cannot overdraw
	float ClampedPadding = FMath::Clamp(Padding, 0.f, FMath::Min(ComponentSizeX, ComponentSizeY));

	// Layout cells
	OutTokens = InTokens;
	for (int32 PositionIDIndex = 0; PositionIDIndex < DistributedPositionIDs.Num(); PositionIDIndex++)
	{
		if (OutTokens.IsValidIndex(PositionIDIndex))
		{
			const int32 Column = DistributedPositionIDs[PositionIDIndex] % Columns;
			const int32 Row = DistributedPositionIDs[PositionIDIndex] / Columns;

			OutTokens[PositionIDIndex].PositionX = AbsolutePosition.X + Column * ComponentSizeX + ClampedPadding / 2.f;
			OutTokens[PositionIDIndex].PositionY = AbsolutePosition.Y + Row * ComponentSizeY + ClampedPadding / 2.f;
			OutTokens[PositionIDIndex].SizeX = FMath::Max(UE_SMALL_NUMBER, ComponentSizeX - ClampedPadding);
			OutTokens[PositionIDIndex].SizeY = FMath::Max(UE_SMALL_NUMBER, ComponentSizeY - ClampedPadding);
			OutTokens[PositionIDIndex].RelativeRotation = ParentRotation;
		}
		else
		{
			break;
		}
	}
}

void UDMXPixelMappingLayoutScript_GridLayout::SetNumTokens(int32 NewNumTokens) 
{
	if (NumTokens != NewNumTokens)
	{
		NumTokens = NewNumTokens;

		ClampColumns();
		ClampRows();
	}
}

void UDMXPixelMappingLayoutScript_GridLayout::ClampColumns()
{
	// At least one Token, and a 1x1 Grid
	NumTokens = NumTokens == 0 ? 1 : NumTokens;
	Columns = FMath::Clamp(Columns, 1, NumTokens);

	Rows = NumTokens / Columns;

	while (Columns * Rows < NumTokens)
	{
		Rows++;
	}
}

void UDMXPixelMappingLayoutScript_GridLayout::ClampRows()
{	
	// At least one Token, and a 1x1 Grid
	NumTokens = NumTokens == 0 ? 1 : NumTokens;
	Rows = FMath::Clamp(Rows, 1, NumTokens);

	Columns = NumTokens / Rows;

	while (Columns * Rows < NumTokens)
	{
		Columns++;
	}
}
