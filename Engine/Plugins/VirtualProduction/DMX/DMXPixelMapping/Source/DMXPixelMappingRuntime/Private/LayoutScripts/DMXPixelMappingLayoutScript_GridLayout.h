// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LayoutScripts/DMXPixelMappingLayoutScript.h"

#include "Library/DMXEntityFixtureType.h"


#include "DMXPixelMappingLayoutScript_GridLayout.generated.h"

enum EHorizontalAlignment : int;
enum EVerticalAlignment : int;


/** Arranges the components in a grid */
UCLASS(DisplayName = "Grid Layout", AutoExpandCategories = ("Layout Settings"))
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingLayoutScript_GridLayout
	: public UDMXPixelMappingLayoutScript

{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject interface

	//~ Begin DMXPixelMappingScript interface
	virtual void Layout_Implementation(const TArray<FDMXPixelMappingLayoutToken>& InTokens, TArray<FDMXPixelMappingLayoutToken>& OutTokens);
	virtual void SetNumTokens(int32 NewNumTokens) override;
	//~ End DMXPixelMappingScript interface

	/** Num Columns in the grid */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	int32 Columns = 1;

	/** Num Rows in the grid */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	int32 Rows = 1;

	/** Padding of the cells */
	UPROPERTY(EditAnywhere, Category = "Layout Settings", Meta = (ClampMin = "0", UIMin = "0"))
	float Padding = 0.f;

	/** Horizontal text alignment */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	TEnumAsByte<EHorizontalAlignment> HorizontalAlignment;

	/** Vertical text alignment */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	TEnumAsByte<EVerticalAlignment> VerticalAlignment;

	/** How cells are distributed, compared to their previous distribution */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	EDMXPixelMappingDistribution Distribution = EDMXPixelMappingDistribution::TopLeftToRight;

private:
	/** Clamps colums to rows given num tokens */
	void ClampColumns();

	/** Clamps colums to rows given num tokens */
	void ClampRows();

	/** Auto-size is enabled when either horizontal or vertical aligmenet are set to fill */
	UPROPERTY()
	bool bAutoSizeMode = true;
};
