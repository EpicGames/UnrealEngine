// Copyright Epic Games, Inc. All Rights Reserved.

#include "LayoutScripts/DMXPixelMappingLayoutScript_LayoutByMVR.h"

#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Library/DMXEntityFixturePatch.h"
#include "Library/DMXLibrary.h"
#include "MVR/Types/DMXMVRFixtureNode.h"
#include "MVR/DMXMVRGeneralSceneDescription.h"

#include "Algo/Find.h"


void UDMXPixelMappingLayoutScript_LayoutByMVR::Layout_Implementation(const TArray<FDMXPixelMappingLayoutToken>& InTokens, TArray<FDMXPixelMappingLayoutToken>& OutTokens)
{
	TMap<const UDMXMVRFixtureNode*, FDMXPixelMappingLayoutToken> FixtureNodeToLayoutTokenMap = GetFixtureNodeToLayoutTokenMap(InTokens);

	// Find bounds of all components
	FBox2D MVRBounds(FVector2D::ZeroVector, FVector2D::ZeroVector);
	for (const TTuple<const UDMXMVRFixtureNode*, FDMXPixelMappingLayoutToken>& FixtureNodeToLayoutTokenPair : FixtureNodeToLayoutTokenMap)
	{
		const FTransform Transform = FixtureNodeToLayoutTokenPair.Key->GetTransformAbsolute();
		const FVector2D MVRPosition2D = GetPosition2DFromTransform(Transform);
		
		MVRBounds.Min.X = FMath::Min(MVRBounds.Min.X, MVRPosition2D.X);
		MVRBounds.Min.Y = FMath::Min(MVRBounds.Min.Y, MVRPosition2D.Y);
		MVRBounds.Max.X = FMath::Max(MVRBounds.Max.X, MVRPosition2D.X);
		MVRBounds.Max.Y = FMath::Max(MVRBounds.Max.Y, MVRPosition2D.Y);
	}

	// Apply margin
	MVRBounds.Min.X -= MarginCentimeters.Left;
	MVRBounds.Min.Y -= MarginCentimeters.Top;
	MVRBounds.Max.X += MarginCentimeters.Right;
	MVRBounds.Max.Y += MarginCentimeters.Bottom;

	// Don't process further if there are no meaningful MVR translations
	if (MVRBounds.GetSize().IsNearlyZero())
	{
		return;
	}

	// Ignore zero parent component size
	if (ParentComponentSize.IsNearlyZero())
	{
		return;
	}

	// Scale to fit parent
	const FVector2D Offset = -MVRBounds.Min;
	const double RatioX = FMath::IsNearlyZero(MVRBounds.GetSize().X) ? 0.f : (ParentComponentSize.X - ComponentSizePixels) / MVRBounds.GetSize().X;
	const double RatioY = FMath::IsNearlyZero(MVRBounds.GetSize().Y) ? 0.f : (ParentComponentSize.Y - ComponentSizePixels) / MVRBounds.GetSize().Y;
	for (TTuple<const UDMXMVRFixtureNode*, FDMXPixelMappingLayoutToken>& FixtureNodeToLayoutTokenPair : FixtureNodeToLayoutTokenMap)
	{
		// Size
		FixtureNodeToLayoutTokenPair.Value.SizeX = ComponentSizePixels;
		FixtureNodeToLayoutTokenPair.Value.SizeY = ComponentSizePixels;

		// Position
		const FTransform Transform = FixtureNodeToLayoutTokenPair.Key->GetTransformAbsolute();
		const FVector2D RelativePosition2D = (GetPosition2DFromTransform(Transform) + Offset) * FVector2D(RatioX, RatioY);
		const FVector2D AbsolutePosition2D = RelativePosition2D + ParentComponentPosition;

		FixtureNodeToLayoutTokenPair.Value.PositionX = AbsolutePosition2D.X;
		FixtureNodeToLayoutTokenPair.Value.PositionY = AbsolutePosition2D.Y;

		OutTokens.Add(FixtureNodeToLayoutTokenPair.Value);
	}
}

TMap<const UDMXMVRFixtureNode*, FDMXPixelMappingLayoutToken> UDMXPixelMappingLayoutScript_LayoutByMVR::GetFixtureNodeToLayoutTokenMap(const TArray<FDMXPixelMappingLayoutToken>& InTokens) const
{
	TMap<const UDMXMVRFixtureNode*, FDMXPixelMappingLayoutToken> Result;
	UDMXLibrary* DMXLibrary = GetDMXLibrary(InTokens);
	if (!DMXLibrary)
	{
		return Result;
	}
	
#if WITH_EDITOR
	DMXLibrary->UpdateGeneralSceneDescription();
#endif

	UDMXMVRGeneralSceneDescription* GeneralSceneDescription = DMXLibrary->GetLazyGeneralSceneDescription();
	if (!GeneralSceneDescription)
	{
		return Result;
	}

	TArray<UDMXMVRFixtureNode*> FixtureNodes;
	GeneralSceneDescription->GetFixtureNodes(FixtureNodes);

	for (const FDMXPixelMappingLayoutToken& Token : InTokens)
	{
		UDMXEntityFixturePatch* FixturePatch = GetFixturePatch(Token);
		if (!FixturePatch)
		{
			continue;
		}

		const UDMXMVRFixtureNode* const* FixtureNodePtr = Algo::FindByPredicate(FixtureNodes, [FixturePatch](const UDMXMVRFixtureNode* FixtureNode)
			{
				return FixtureNode->UUID == FixturePatch->GetMVRFixtureUUID();
			});

		if (FixtureNodePtr)
		{
			Result.Add(*FixtureNodePtr, Token);
		}
	}

	return Result;
}

UDMXLibrary* UDMXPixelMappingLayoutScript_LayoutByMVR::GetDMXLibrary(const TArray<FDMXPixelMappingLayoutToken>& InTokens) const
{
	UDMXLibrary* DMXLibrary = nullptr;
	for (const FDMXPixelMappingLayoutToken& Token : InTokens)
	{
		UDMXEntityFixturePatch* FixturePatch = GetFixturePatch(Token);

		if (FixturePatch && FixturePatch->GetParentLibrary())
		{
			if (!ensureMsgf(!DMXLibrary || FixturePatch->GetParentLibrary() == DMXLibrary, TEXT("Unexpected layout tokens source from different DMX Libraries. This is not supported.")))
			{
				return nullptr;
			}

			DMXLibrary = FixturePatch->GetParentLibrary();
		}
	}

	return DMXLibrary;
}

UDMXEntityFixturePatch* UDMXPixelMappingLayoutScript_LayoutByMVR::GetFixturePatch(const FDMXPixelMappingLayoutToken& Token) const
{
	if (UDMXPixelMappingFixtureGroupItemComponent* GroupItem = Cast<UDMXPixelMappingFixtureGroupItemComponent>(Token.Component.Get()))
	{
		return GroupItem->FixturePatchRef.GetFixturePatch();
	}
	else if (UDMXPixelMappingMatrixComponent* Matrix = Cast<UDMXPixelMappingMatrixComponent>(Token.Component.Get()))
	{
		return Matrix->FixturePatchRef.GetFixturePatch();
	}

	return nullptr;
}

FVector2D UDMXPixelMappingLayoutScript_LayoutByMVR::GetPosition2DFromTransform(const FTransform& Transform) const
{
	switch (ProjectionPlane)
	{
	case EDMXPixelMappingMVRProjectionPlane::XY:
		return FVector2D(Transform.GetTranslation().X, Transform.GetTranslation().Y);

	case EDMXPixelMappingMVRProjectionPlane::XZ:
		return FVector2D(Transform.GetTranslation().X, Transform.GetTranslation().Z);

	case EDMXPixelMappingMVRProjectionPlane::YZ:
		return FVector2D(Transform.GetTranslation().Y, Transform.GetTranslation().Z);

	case EDMXPixelMappingMVRProjectionPlane::YX:
		return FVector2D(Transform.GetTranslation().Y, Transform.GetTranslation().X);

	case EDMXPixelMappingMVRProjectionPlane::ZX:
		return FVector2D(Transform.GetTranslation().Z, Transform.GetTranslation().X);

	case EDMXPixelMappingMVRProjectionPlane::ZY:
		return FVector2D(Transform.GetTranslation().Z, Transform.GetTranslation().Y);

	default:
		checkf(0, TEXT("Unhandled enum value"));
	}

	return FVector2D::ZeroVector;
}
