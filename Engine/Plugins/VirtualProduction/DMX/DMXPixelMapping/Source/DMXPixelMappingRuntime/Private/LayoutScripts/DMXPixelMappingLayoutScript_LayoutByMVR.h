// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LayoutScripts/DMXPixelMappingLayoutScript.h"


#include "Layout/Margin.h"

#include "DMXPixelMappingLayoutScript_LayoutByMVR.generated.h"

class UDMXEntityFixturePatch;
class UDMXLibrary;

class UDMXMVRFixtureNode;


/** Defines which MVR coordinates are projected to a Pixel Mapping Layout */
UENUM(BlueprintType)
enum class EDMXPixelMappingMVRProjectionPlane : uint8
{
	XY,
	XZ,
	YZ,
	YX,
	ZX,
	ZY
};

/** Arranges the components given their coordinates in the DMX Library's MVR specs */
UCLASS(DisplayName = "Layout by MVR", AutoExpandCategories = ("Layout Settings"))
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingLayoutScript_LayoutByMVR
	: public UDMXPixelMappingLayoutScript

{
	GENERATED_BODY()

public:
	//~ Begin DMXPixelMappingScript interface
	virtual void Layout_Implementation(const TArray<FDMXPixelMappingLayoutToken>& InTokens, TArray<FDMXPixelMappingLayoutToken>& OutTokens);
	//~ End DMXPixelMappingScript interface

	/** Which axis or axes are projected to the Pixel Mapping Layout */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	EDMXPixelMappingMVRProjectionPlane ProjectionPlane;
	
	/** Margin of the MVR fixtures, in centimeters */
	UPROPERTY(EditAnywhere, Category = "Layout Settings")
	FMargin MarginCentimeters;

	/** Size each component should take, in pixels */
	UPROPERTY(EditAnywhere, Category = "Layout Settings", Meta = (ClampMin = "1", UIMin = "1"))
	float ComponentSizePixels = 10.f;

private:
	/** Returns a map of Fixture Nodes and the Layout Tokens they correspond to */
	TMap<const UDMXMVRFixtureNode*, FDMXPixelMappingLayoutToken> GetFixtureNodeToLayoutTokenMap(const TArray<FDMXPixelMappingLayoutToken>& InTokens) const;

	/** Returns the DMX Library the components of this script use */
	UDMXLibrary* GetDMXLibrary(const TArray<FDMXPixelMappingLayoutToken>& InTokens) const;

	/** Returns the Fixture Patch of the Token, or nullptr if the token has no patch */
	UDMXEntityFixturePatch* GetFixturePatch(const FDMXPixelMappingLayoutToken& Token) const;

	/** Returns XY in 2D space from transform, depending on ProjectionAxes */
	FVector2D GetPosition2DFromTransform(const FTransform& Transform) const;
};
