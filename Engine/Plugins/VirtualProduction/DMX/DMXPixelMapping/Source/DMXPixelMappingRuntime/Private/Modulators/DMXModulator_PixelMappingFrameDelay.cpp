// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modulators/DMXModulator_PixelMappingFrameDelay.h"


void UDMXModulator_PixelMappingFrameDelay::Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues)
{
	if (DelayFrames > 0)
	{
		const int32 WakeFrame = FMath::Clamp(GFrameNumber + DelayFrames, static_cast<uint32>(0), static_cast<uint32>(TNumericLimits<int32>::Max()));

		// Find the last element that is awake
		const int32 ElementToWakeIndex = Buffer.FindLastByPredicate([](const TSharedRef<FBufferElement>& Element)
			{
				return Element->WakeFrame <= GFrameNumber;
			});

		// Output it
		if (ElementToWakeIndex != INDEX_NONE)
		{
			OutNormalizedAttributeValues = Buffer[ElementToWakeIndex]->NormalizedAttributeValues;
		}

		// Keep sleeping elements
		Buffer.RemoveAll([](const TSharedRef<FBufferElement>& Element)
			{
				return Element->WakeFrame <= GFrameNumber;
			});

		// Add the new received map to the buffer
		TSharedRef<FBufferElement> NewBufferElement = MakeShared<FBufferElement>(WakeFrame, InNormalizedAttributeValues);
		Buffer.Add(NewBufferElement);
	}
	else
	{
		OutNormalizedAttributeValues = InNormalizedAttributeValues;
	}
}

void UDMXModulator_PixelMappingFrameDelay::ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues)
{
	if (DelayFrames > 0)
	{
		const int32 WakeFrame = FMath::Clamp(GFrameNumber + DelayFrames, static_cast<uint32>(0), static_cast<uint32>(TNumericLimits<int32>::Max()));
	
		// Find the last element that is awake
		const int32 ElementToWakeIndex = MatrixBuffer.FindLastByPredicate([](const TSharedRef<FMatrixBufferElement>& Element)
			{
				return Element->WakeFrame <= GFrameNumber;
			});

		// Output it
		if (ElementToWakeIndex != INDEX_NONE)
		{
			OutNormalizedMatrixAttributeValues = MatrixBuffer[ElementToWakeIndex]->NormalizedMatrixAttributeValues;
		}

		// Keep sleeping elements
		MatrixBuffer.RemoveAll([](const TSharedRef<FMatrixBufferElement>& Element)
			{
				return Element->WakeFrame <= GFrameNumber;
			});

		// Add the new received map to the buffer
		TSharedRef<FMatrixBufferElement> NewMatrixBufferElement = MakeShared<FMatrixBufferElement>(WakeFrame, InNormalizedMatrixAttributeValues);
		MatrixBuffer.Add(NewMatrixBufferElement);
	}
	else
	{
		OutNormalizedMatrixAttributeValues = InNormalizedMatrixAttributeValues;
	}
}
