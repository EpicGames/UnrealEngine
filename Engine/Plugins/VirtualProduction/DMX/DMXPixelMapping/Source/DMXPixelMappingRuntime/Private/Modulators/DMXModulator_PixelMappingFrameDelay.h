// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXTypes.h"
#include "Modulators/DMXModulator.h"


#include "DMXModulator_PixelMappingFrameDelay.generated.h"


/** Delays the incoming singal by number of Delay Frames. Only useful for PixelMapping and constant frame rates. */
UCLASS(NotBlueprintable, DisplayName = "DMX Modulator Constant Frame Delay", AutoExpandCategories = ("DMX"))
class DMXPIXELMAPPINGRUNTIME_API UDMXModulator_PixelMappingFrameDelay
	: public UDMXModulator

{
	GENERATED_BODY()

public:
	/** RGB to CMY implementation */
	virtual void Modulate_Implementation(UDMXEntityFixturePatch* FixturePatch, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues, TMap<FDMXAttributeName, float>& OutNormalizedAttributeValues) override;

	/** Matrix RGB to CMY implementation */
	virtual void ModulateMatrix_Implementation(UDMXEntityFixturePatch* FixturePatch, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues, TArray<FDMXNormalizedAttributeValueMap>& OutNormalizedMatrixAttributeValues) override;

	/** The time by which signals are delayed in Seconds */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "DMX", Meta = (ClampMin = "0", UIMin = "0", ClampMax = "120", UIMax = "120"))
	int32 DelayFrames = 0;

	/** An element in the buffer for attribute values */
	class FBufferElement
		: public TSharedFromThis<FBufferElement>
	{
	public:
		FBufferElement()
		{}

		FBufferElement(int32 InWakeFrame, const TMap<FDMXAttributeName, float>& InNormalizedAttributeValues)
			: NormalizedAttributeValues(InNormalizedAttributeValues)
			, WakeFrame(InWakeFrame)
		{}

		/** Map of attribute values with timestamp */
		TMap<FDMXAttributeName, float> NormalizedAttributeValues;

		/** Time when values should be output */
		uint32 WakeFrame = 0;
	};

	/** An element in the buffer for matrix attribute values */
	class FMatrixBufferElement
		: public TSharedFromThis<FMatrixBufferElement>
	{
	public:
		FMatrixBufferElement()
		{}

		FMatrixBufferElement(int32 InWakeFrame, const TArray<FDMXNormalizedAttributeValueMap>& InNormalizedMatrixAttributeValues)
			: NormalizedMatrixAttributeValues(InNormalizedMatrixAttributeValues)
			, WakeFrame(InWakeFrame)
		{}

		/** Map of matrix attribute values with timestamps */
		TArray<FDMXNormalizedAttributeValueMap> NormalizedMatrixAttributeValues;

		/** Time when values should be output */
		uint32 WakeFrame = 0;
	};

	/** Buffer of data */
	TArray<TSharedRef<FBufferElement>> Buffer;

	/** Buffer of data */
	TArray<TSharedRef<FMatrixBufferElement>> MatrixBuffer;
};
