// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/DMXPixelMappingComponentTemplate.h"

#include "Components/DMXPixelMappingFixtureGroupItemComponent.h"
#include "Components/DMXPixelMappingMatrixComponent.h"
#include "Components/DMXPixelMappingRootComponent.h"
#include "DMXPixelMapping.h"
#include "Library/DMXEntityFixturePatch.h"


FDMXPixelMappingComponentTemplate::FDMXPixelMappingComponentTemplate(TSubclassOf<UDMXPixelMappingBaseComponent> InComponentClass)
	: ComponentClass(InComponentClass.Get())
{
	check(InComponentClass);

	UDMXPixelMappingBaseComponent* DefaultComponent = ComponentClass->GetDefaultObject<UDMXPixelMappingBaseComponent>();

	Name = FText::FromString(DefaultComponent->GetNamePrefix().ToString());
}

FDMXPixelMappingComponentTemplate::FDMXPixelMappingComponentTemplate(TSubclassOf<UDMXPixelMappingBaseComponent> InComponentClass, const FDMXEntityFixturePatchRef& InFixturePatchRef)
	: ComponentClass(InComponentClass.Get())
	, FixturePatchRef(InFixturePatchRef)
{
	check(ComponentClass.IsValid());

	// Only for matrix and group item components
	check(ComponentClass.Get() == UDMXPixelMappingFixtureGroupItemComponent::StaticClass() || ComponentClass.Get() == UDMXPixelMappingMatrixComponent::StaticClass());

	UDMXPixelMappingBaseComponent* DefaultComponent = ComponentClass->GetDefaultObject<UDMXPixelMappingBaseComponent>();

	Name = FText::FromString(DefaultComponent->GetNamePrefix().ToString());
}

#if WITH_EDITOR
FText FDMXPixelMappingComponentTemplate::GetCategory() const
{
	UDMXPixelMappingOutputComponent* OutputComponent = ComponentClass->GetDefaultObject<UDMXPixelMappingOutputComponent>();
	return OutputComponent->GetPaletteCategory();
}
#endif // WITH_EDITOR

UDMXPixelMappingBaseComponent* FDMXPixelMappingComponentTemplate::CreateComponentInternal(UDMXPixelMappingRootComponent* InRootComponent)
{
	const UDMXPixelMapping* PixelMapping = InRootComponent ? InRootComponent->GetPixelMapping() : nullptr;
	if (!ensureMsgf(PixelMapping && InRootComponent, TEXT("Tried to create components from template but PixelMapping or RootComponent were invalid.")))
	{
		return nullptr;
	}

	// Create a unique name
	UDMXPixelMappingBaseComponent* DefaultComponent = ComponentClass->GetDefaultObject<UDMXPixelMappingBaseComponent>();
	FName UniqueName = MakeUniqueObjectName(InRootComponent, ComponentClass.Get(), DefaultComponent->GetNamePrefix());

	UDMXPixelMappingBaseComponent* NewComponent = nullptr;
	if (ComponentClass.Get() == UDMXPixelMappingFixtureGroupItemComponent::StaticClass())
	{
		// Create the Component
		UDMXPixelMappingFixtureGroupItemComponent* NewGroupItemComponent = NewObject<UDMXPixelMappingFixtureGroupItemComponent>(InRootComponent, ComponentClass.Get(), UniqueName, RF_Transactional | RF_Public);

		NewGroupItemComponent->FixturePatchRef = FixturePatchRef;

		NewComponent = NewGroupItemComponent;
	}
	else if (ComponentClass.Get() == UDMXPixelMappingMatrixComponent::StaticClass())
	{
		UDMXPixelMappingMatrixComponent* NewMatrixComponent = NewObject<UDMXPixelMappingMatrixComponent>(InRootComponent, ComponentClass.Get(), UniqueName, RF_Transactional | RF_Public);
		NewMatrixComponent->FixturePatchRef = FixturePatchRef;

		NewComponent = NewMatrixComponent;
	}
	else
	{
		UDMXPixelMappingBaseComponent* NewBaseComponent = NewObject<UDMXPixelMappingBaseComponent>(InRootComponent, ComponentClass.Get(), UniqueName, RF_Transactional | RF_Public);

		NewComponent = NewBaseComponent;
	}
	check(NewComponent);

	// Initialize name and color of components that use a patch with data from the patch
	if (UDMXPixelMappingOutputDMXComponent* NewComponentWithPatch = Cast<UDMXPixelMappingOutputDMXComponent>(NewComponent))
	{
		const UDMXEntityFixturePatch* Patch = NewComponentWithPatch->FixturePatchRef.GetFixturePatch();
		if (Patch)
		{
			// Create a better unique name and set it
			UniqueName = MakeUniqueObjectName(NewComponentWithPatch->GetOuter(), NewComponentWithPatch->GetClass(), *Patch->Name);
			const FString NewNameStr = UniqueName.ToString();
			NewComponentWithPatch->Rename(*NewNameStr);

#if WITH_EDITOR
			// Set if the component should follow the patch color
			NewComponentWithPatch->bUsePatchColor = PixelMapping->bNewComponentsUsePatchColor;
#endif // WITH_EDITOR
		}
	}

	return NewComponent;
}
