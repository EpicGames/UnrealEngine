// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "DMXPixelMappingSubsystem.generated.h"


UCLASS()
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingSubsystem
	: public UEngineSubsystem
{
	GENERATED_BODY()

public:
	/** Get a DMX PixelMapping Subsystem, pure version */
	UFUNCTION(BlueprintPure, Category = "DMX PixelMapping Subsystem", meta = (BlueprintInternalUseOnly = "true"))
	static UDMXPixelMappingSubsystem* GetDMXPixelMappingSubsystem_Pure();

	/** Get a DMX PixelMapping Subsystem, callable version */
	UFUNCTION(BlueprintCallable, Category = "DMX PixelMapping Subsystem", meta = (BlueprintInternalUseOnly = "true"))
	static UDMXPixelMappingSubsystem* GetDMXPixelMappingSubsystem_Callable();

	/** Load Pixel Mapping asset */
	UFUNCTION(BlueprintPure, Category = "DMX|PixelMapping")
	UDMXPixelMapping* GetDMXPixelMapping(UDMXPixelMapping* InPixelMapping);

	/** 
	 * Get Renderer component. Only for K2 blueprint nodes 
	 *
	 * @param			InDMXPixelMapping Pixel Mapping UObject Asset
	 * @param			InComponentName Name of looking renderer component
	 *
	 * @return Pointer to the component or nullptr
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "InComponentName"), Category = "DMX|PixelMapping")
	UDMXPixelMappingRendererComponent* GetRendererComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName);

	/**
	 * Get OutputDMX component. Only for K2 blueprint nodes
	 *
	 * @param			InDMXPixelMapping Pixel Mapping UObject Asset
	 * @param			InComponentName Name of looking OutputDMX component
	 *
	 * @return Pointer to the component or nullptr
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "InComponentName"), Category = "DMX|PixelMapping")
	UDMXPixelMappingOutputDMXComponent* GetOutputDMXComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName);

	/**
	 * Get FixtureGroup component. Only for K2 blueprint nodes
	 *
	 * @param			InDMXPixelMapping Pixel Mapping UObject Asset
	 * @param			InComponentName Name of looking FixtureGroup component
	 *
	 * @return Pointer to the component or nullptr
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "InComponentName"), Category = "DMX|PixelMapping")
	UDMXPixelMappingFixtureGroupComponent* GetFixtureGroupComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName);

	/**
	 * Get Matrix component. Only for K2 blueprint nodes
	 *
	 * @param			InDMXPixelMapping Pixel Mapping UObject Asset
	 * @param			InComponentName Name of looking Matrix component
	 *
	 * @return Pointer to the component or nullptr
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "true", AutoCreateRefTerm = "InComponentName"), Category = "DMX|PixelMapping")
	UDMXPixelMappingMatrixComponent* GetMatrixComponent(UDMXPixelMapping* InDMXPixelMapping, const FName& InComponentName);
};
