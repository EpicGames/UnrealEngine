// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXAttribute.h"
#include "UObject/Object.h"

#include "DMXPixelMappingColorSpace.generated.h"


/** 
 * Base class for Pixel Mapping Color Spaces. 
 * Note, implementations must be thread-safe. 
 */
UCLASS(Abstract)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingColorSpace
	: public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR
	DECLARE_MULTICAST_DELEGATE_OneParam(FDMXPixelMappingColorSpacePostEditChangePropertyDelegate, FPropertyChangedEvent& /** PropertyChangedEvent */)
#endif // WITH_EDITOR

public:
	/** Sets the RGBA value to be converted to the implemented Color Space */
	virtual void SetRGBA(const FLinearColor& InColor) PURE_VIRTUAL(UDMXPixelMappingColorSpace::SetRGBA, return;);

	/** Resets the color to black, and any luminance to zero */
	void ResetToBlack();

	/** Returns the Attribute Name to Value Map of the Color currently set */
	const TMap<FDMXAttributeName, float>& GetAttributeNameToValueMap() { return AttributeNameToValueMap; }

#if WITH_EDITOR
	/** Returns a delegate broadcast when a property changed. Useful to handle attribute name changes. */
	FDMXPixelMappingColorSpacePostEditChangePropertyDelegate& GetOnPostEditChangedProperty() { return OnPostEditChangeProperty; }
#endif // WITH_EDITOR 

	/** When checked, uses Unreal Working Color Space for input conversion, otherwise, assumes the input is sRGB */
	UPROPERTY(EditAnywhere, Category = "Color Space", Meta = (DisplayPriority = -1))
	bool bUseWorkingColorSpaceForInput = true;

protected:
	//~ Begin UObject interface
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject interface

	/** Sets an Attribute Value */
	void SetAttributeValue(const FDMXAttributeName& AttributeName, float Value);

	/** Clears cached Attribtues */
	void ClearCachedAttributeValues();

private:
	/** Buffer holding results of the conversion */
	TMap<FDMXAttributeName, float> AttributeNameToValueMap;

#if WITH_EDITOR
	/** Event raised Post Edit Change Property */
	FDMXPixelMappingColorSpacePostEditChangePropertyDelegate OnPostEditChangeProperty;
#endif // WITH_EDITOR
};
