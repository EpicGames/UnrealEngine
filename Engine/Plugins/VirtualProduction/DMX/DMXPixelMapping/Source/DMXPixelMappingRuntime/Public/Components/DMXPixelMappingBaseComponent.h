// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DMXPixelMappingBaseComponent.generated.h"


DECLARE_STATS_GROUP(TEXT("DMXPixelMapping"), STATGROUP_DMXPIXELMAPPING, STATCAT_Advanced);

class FDMXPixelMappingComponentTemplate;
class UDMXPixelMappingRendererComponent;
class UDMXPixelMapping;
class UDMXPixelMappingRootComponent;


UENUM(BlueprintType)
enum class EDMXPixelMappingResetDMXMode : uint8
{
	SendDefaultValues UMETA(DisplayName = "Send Default Values"),
	SendZeroValues UMETA(DisplayName = "Send Zero Values"),
	DoNotSendValues UMETA(DisplayName = "Keep Last Mapped Values")
};

/**
 * Base class for all DMX Pixel Mapping components. 
 */
UCLASS(BlueprintType, NotBlueprintable, Abstract)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingBaseComponent
	: public UObject
{
	GENERATED_BODY()

	using TComponentPredicate = TFunctionRef<void(UDMXPixelMappingBaseComponent*)>;

	template <typename Type>
	using TComponentPredicateType = TFunctionRef<void(Type*)>;

	DECLARE_EVENT_TwoParams(UDMXPixelMappingBaseComponent, FDMXPixelMappingOnComponentAdded, UDMXPixelMapping* /** PixelMapping */, UDMXPixelMappingBaseComponent* /** AddedComponent */);
	DECLARE_EVENT_TwoParams(UDMXPixelMappingBaseComponent, FDMXPixelMappingOnComponentRemoved, UDMXPixelMapping* /** PixelMapping */, UDMXPixelMappingBaseComponent* /** RemovedComponent */);
	DECLARE_EVENT_OneParam(UDMXPixelMappingBaseComponent, FDMXPixelMappingOnComponentRenamed, UDMXPixelMappingBaseComponent* /** RenamedComponent */);

public:
	/** Gets an Event broadcast when a component was added */
	static FDMXPixelMappingOnComponentAdded& GetOnComponentAdded();

	/** Gets an Event broadcast when a component was added */
	static FDMXPixelMappingOnComponentRemoved& GetOnComponentRemoved();

	/** Gets an Event broadcast this uobject was renamed. Note this is only raised after UObject::Rename, not after SetUserName.*/
	static FDMXPixelMappingOnComponentRenamed& GetOnComponentRenamed();
	
	//~ Begin UObject interface
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR
	//~ End UObject interface

	/**
	 * Should log properties that were changed in underlying fixture patch or fixture type
	 *
	 * @return		Returns true if properties are valid.
	 */
	virtual bool ValidateProperties() { return true; }

	/**
	* Helper function for generating UObject name, the child should implement their own logic for Prefix name generation.
	*/
	virtual const FName& GetNamePrefix();

	/*------------------------------------------
		UDMXPixelMappingBaseComponent interface
	--------------------------------------------*/

	/** Get the number of children components */
	int32 GetChildrenCount() const;

	/** Get the child component by the given index. */
	UDMXPixelMappingBaseComponent* GetChildAt(int32 Index) const;

	/** Get all children to belong to this component */
	const TArray<UDMXPixelMappingBaseComponent*>& GetChildren() const { return Children; }

	/** Gathers descendant child Components of a parent Component. */
	void GetChildComponentsRecursively(TArray<UDMXPixelMappingBaseComponent*>& Components);

	/**
	 * Add a new child componet
	 *
	 * @param InComponent    Component instance object
	 */
	virtual void AddChild(UDMXPixelMappingBaseComponent* InComponent);

	/** Remove the child component by the given component object. */
	virtual void RemoveChild(UDMXPixelMappingBaseComponent* InComponent);

	/** Remove all children */
	void ClearChildren();

	/** Check if a Component can be moved under another one (used for copy/move/duplicate) */
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const
	{
		return false;
	}

	/** DEPRECATED 5.3 */
	UE_DEPRECATED(5.3, "Renamed GetUserName to be more explicit aobout the intent. The user name is displayed if set, otherwise the object name.")
	virtual FString GetUserFriendlyName() const;

	/** Returns the name of the component. If user name is set returns the custom user name (see SetUserName) */
	virtual FString GetUserName() const;

	/** Sets the custom user name. If set, GetUserName returns this name. */
	virtual void SetUserName(const FString& NewName);

	/** 
	 * Loop through all child by given Predicate 
	 *
	 * @param bIsRecursive		Should it loop recursively
	 */
	void ForEachChild(TFunctionRef<void(UDMXPixelMappingBaseComponent*)> Predicate, bool bIsRecursive);

	/**
	 * Looking for the first child by given Class
	 *
	 * @return An instance of the templated Component
	 */
	template <typename TComponentClass>
	TComponentClass* GetFirstChildOfClass()
	{
		TComponentClass* FoundObject = nullptr;
		ForEachComponentOfClass<TComponentClass>([this, &FoundObject](TComponentClass* InObject)
			{
				FoundObject = InObject;
				return;
			}, true);

		return FoundObject;
	}

	/** 
	 * Loop through all templated child class by given Predicate
	 *
	 * @param bIsRecursive		Should it loop recursively
	 */
	template <typename TComponentClass>
	void ForEachChildOfClass(TComponentPredicateType<TComponentClass> Predicate, bool bIsRecursive)
	{
		ForEachChild([&Predicate](UDMXPixelMappingBaseComponent* InComponent) {
			if (TComponentClass* CastComponent = Cast<TComponentClass>(InComponent))
			{
				Predicate(CastComponent);
			}
		}, bIsRecursive);
	}

	/** Get Pixel Mapping asset UObject */
	UDMXPixelMapping* GetPixelMapping() const;

	/** Get root component of the component tree */
	const UDMXPixelMappingRootComponent* GetRootComponent() const;

	/** Get the root component and not allow a null option. */
	const UDMXPixelMappingRootComponent* GetRootComponentChecked() const;

	/**
	 * Get renderer component associated with current component
	 * It could be this component itself if this component is root component
	 * It could be parent component if that is pixel component
	 * It could be nullptr if that is Root component
	 */
	UDMXPixelMappingRendererComponent* GetRendererComponent();

	/*----------------------------------------------------------
		Public blueprint accessible function
	----------------------------------------------------------*/

	/** Reset all sending DMX channels to 0 for this component and all children */
	UFUNCTION(BlueprintCallable, Category = "DMX|PixelMapping")
	virtual void ResetDMX(EDMXPixelMappingResetDMXMode ResetMode = EDMXPixelMappingResetDMXMode::SendDefaultValues) {};

	/** Send DMX values of this component and all children. */
	UFUNCTION(BlueprintCallable, Category = "DMX|PixelMapping")
	virtual void SendDMX() {};

	/** Renders this component and all children */
	UFUNCTION(BlueprintCallable, Category = "DMX|PixelMapping")
	virtual void Render() {};

	/** Render downsample texture and send DMX for this component and all children */
	UFUNCTION(BlueprintCallable, Category = "DMX|PixelMapping")
	virtual void RenderAndSendDMX() {};

	/**
	 * Recursively looking for the first parent by given Class
	 *
	 * @return An instance of the templated Component
	 */
	template <typename TComponentClass>
	static TComponentClass* GetFirstParentByClass(UDMXPixelMappingBaseComponent* InComponent)
	{
		if (!InComponent->WeakParent.IsValid())
		{
			return nullptr;
		}

		if (TComponentClass* SearchComponentbyType = Cast<TComponentClass>(InComponent->GetParent()))
		{
			return SearchComponentbyType;
		}
		else
		{
			return GetFirstParentByClass<TComponentClass>(InComponent->GetParent());
		}
	}

	/** Recursively loop through all child by given Component and Predicate */
	static void ForComponentAndChildren(UDMXPixelMappingBaseComponent* Component, TComponentPredicate Predicate);

	/** Array of children belong to this component */
	UPROPERTY()
	TArray<TObjectPtr<UDMXPixelMappingBaseComponent>> Children;

	/** Returns the parent. May be nullptr when the the component is creating or destroying */
	UDMXPixelMappingBaseComponent* GetParent() const { return WeakParent.Get(); }

	/** Returns the parent. May be nullptr when the the component is creating or destroying */
	bool HasValidParent() const { return WeakParent.IsValid(); }

#if WITH_EDITOR
	/** Set the components parent */
	void SetParent(const TWeakObjectPtr<UDMXPixelMappingBaseComponent>& NewParent) { WeakParent = NewParent; }
#endif 
	/** Parent component */
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Leads to entangled references. Use GetParent() or WeakParent instead."))
	TObjectPtr<UDMXPixelMappingBaseComponent> Parent_DEPRECATED;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bExpanded = true;
#endif

protected:
	/** Called when the component was added to a parent */
	virtual void NotifyAddedToParent();

	/** Called when the component was added to a parent */
	virtual void NotifyRemovedFromParent();

	/** Custom user name for the component. Should be used if set. */
	UPROPERTY()
	FString UserName;

private:
	/** Parent component */
	UPROPERTY(NonTransactional)
	TWeakObjectPtr<UDMXPixelMappingBaseComponent> WeakParent;

	/** Delegate Broadcast when a component was added */
	static FDMXPixelMappingOnComponentAdded OnComponentAdded;

	/** Delegate Broadcast when a component was removed */
	static FDMXPixelMappingOnComponentRemoved OnComponentRemoved;

	/** Delegate Broadcast when a component was renamed */
	static FDMXPixelMappingOnComponentRenamed OnComponentRenamed;
};
