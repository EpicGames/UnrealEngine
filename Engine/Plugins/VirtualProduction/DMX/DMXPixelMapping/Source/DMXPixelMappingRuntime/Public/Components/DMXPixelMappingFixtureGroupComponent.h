// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingOutputComponent.h"
#include "Library/DMXEntityReference.h"
#include "DMXPixelMappingFixtureGroupComponent.generated.h"

class UDMXLibrary;
class UDMXPixelMappingLayoutScript;

class STextBlock;
class SUniformGridPanel;


/**
 * Container component for Fixture Items
 */
UCLASS()
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingFixtureGroupComponent
	: public UDMXPixelMappingOutputComponent
{
	GENERATED_BODY()
public:
	/** Default Constructor */
	UDMXPixelMappingFixtureGroupComponent();

	//~ Begin UObject implementation
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
#endif
	//~ End UObject implementation

	//~ Begin UDMXPixelMappingBaseComponent implementation
	virtual const FName& GetNamePrefix() override;
	virtual void AddChild(UDMXPixelMappingBaseComponent* InComponent) override;
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;
	virtual void ResetDMX(EDMXPixelMappingResetDMXMode ResetMode = EDMXPixelMappingResetDMXMode::SendDefaultValues) override;
	virtual void SendDMX() override;
	virtual FString GetUserName() const override;
	//~ End UDMXPixelMappingBaseComponent implementation

	//~ Begin UDMXPixelMappingOutputComponent implementation
	virtual void SetPosition(const FVector2D& NewPosition) override;
	virtual void SetPositionRotated(FVector2D NewRotatedPosition) override;
	virtual void SetSize(const FVector2D& NewSize) override;
	virtual void SetRotation(double NewRotation) override;
#if WITH_EDITOR
	virtual bool IsExposedToTemplate() { return true; }
	virtual const FText GetPaletteCategory() override;
#endif
	//~ End UDMXPixelMappingOutputComponent implementation

	/** Returns a delegate broadcasted when the DMX Library changed */
	FSimpleMulticastDelegate& GetOnDMXLibraryChanged() { return OnDMXLibraryChangedDelegate; }

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	virtual void QueueDownsample() override;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Fixture List")
	TObjectPtr<UDMXLibrary> DMXLibrary;

	/** Layout script for the children of this component (hidden in customizations and displayed in its own panel). */
	UPROPERTY(EditAnywhere, Instanced, Category = "Layout")
	TObjectPtr<UDMXPixelMappingLayoutScript> LayoutScript;

private:
	/** Returns a delegate broadcasted when the DMX Library changed */
	FSimpleMulticastDelegate OnDMXLibraryChangedDelegate;

	static const FVector2D MinGroupSize;
};
