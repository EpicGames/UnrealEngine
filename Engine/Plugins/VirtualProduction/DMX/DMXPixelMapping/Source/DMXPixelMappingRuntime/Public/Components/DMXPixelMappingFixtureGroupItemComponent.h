// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingOutputDMXComponent.h"

#include "DMXPixelMappingFixtureGroupComponent.h"
#include "DMXPixelMappingMatrixCellComponent.h"
#include "Library/DMXEntityReference.h"

#include "Misc/Attribute.h"

#include "DMXPixelMappingFixtureGroupItemComponent.generated.h"

enum class EDMXColorMode : uint8;
struct FDMXAttributeName;
class FEditPropertyChain;
class STextBlock;
class UTextureRenderTarget2D;
class UDMXModulator;
class UDMXPixelMappingColorSpace;
namespace UE::DMXPixelMapping::Rendering::PixelMapRenderer { class FPixelMapRenderElement; }

/** 
 * A component that holds a single Fixture Patch in the Pixel Mapping, and actually sends DMX.
 */
UCLASS(AutoExpandCategories = ("Output Settings"))
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingFixtureGroupItemComponent
	: public UDMXPixelMappingOutputDMXComponent
{
	GENERATED_BODY()

public:
	/** Default Constructor */
	UDMXPixelMappingFixtureGroupItemComponent();

protected:
	//~ Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR
	//~ End UObject interface

public:
	//~ Begin UDMXPixelMappingBaseComponent implementation
	virtual const FName& GetNamePrefix() override;
	virtual void ResetDMX(EDMXPixelMappingResetDMXMode ResetMode = EDMXPixelMappingResetDMXMode::SendDefaultValues) override;
	virtual void SendDMX() override;
	virtual FString GetUserName() const override;
	//~ End UDMXPixelMappingBaseComponent implementation

	//~ Begin UDMXPixelMappingOutputComponent implementation
#if WITH_EDITOR
	virtual bool IsVisible() const override;
#endif // WITH_EDITOR	
	virtual bool IsOverParent() const override;
	virtual void SetPosition(const FVector2D& NewPosition) override;
	virtual void SetSize(const FVector2D& NewSize) override;
	//~ End UDMXPixelMappingOutputComponent implementation

	/** Returns the pixel map render element for the matrix cell */
	TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement> GetOrCreatePixelMapRenderElement();

	/** Check if a Component can be moved under another one (used for copy/move/duplicate) */
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;

	/** Index of the cell pixel in downsample target buffer */
	int32 DownsamplePixelIndex;

private:
	/** Updates the rendered element */
	void UpdateRenderElement();

#if WITH_EDITOR
	/** Called post edit change properties of the color space member */
	void OnColorSpacePostEditChangeProperties(FPropertyChangedEvent& PropertyChangedEvent);
#endif // WITH_EDITOR

	/** Helper that returns the renderer component this component belongs to */
	UDMXPixelMappingRendererComponent* GetRendererComponent() const;

	/** Pixel map render element for the matrix cell */
	TSharedPtr<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement> PixelMapRenderElement;


	//////////////////////
	// Deprecated Members
	
public:
	//~ Begin UDMXPixelMappingOutputComponent implementation
	UE_DEPRECATED(5.3, "No longer in use. Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	virtual void QueueDownsample() override;
	UE_DEPRECATED(5.3, "No longer in use. Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	virtual int32 GetDownsamplePixelIndex() const override { return DownsamplePixelIndex; }
	//~ End UDMXPixelMappingOutputComponent implementation
	
	//~ Begin UDMXPixelMappingOutputDMXComponent implementation
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.3, "Deprecated for performance reasons. Instead use 'Get DMX Pixel Mapping Renderer Component' and Render only once each tick.")
	virtual void RenderWithInputAndSendDMX() override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//~ End UDMXPixelMappingOutputDMXComponent implementation

	/** DEPRECATED 5.2 */
	UE_DEPRECATED(5.2, "Instead use UDMXPixelMappingColorSpace::Update and UDMXPixelMappingColorSpace::GetAttributeNameToValueMap using the ColorSpace member of this class.")
	TMap<FDMXAttributeName, float> CreateAttributeValues() const;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	EDMXColorMode ColorMode_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeRExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeGExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeBExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool bMonochromeExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeRInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeGInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeBInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool bMonochromeInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName AttributeR_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName AttributeG_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName AttributeB_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName MonochromeIntensity_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};
