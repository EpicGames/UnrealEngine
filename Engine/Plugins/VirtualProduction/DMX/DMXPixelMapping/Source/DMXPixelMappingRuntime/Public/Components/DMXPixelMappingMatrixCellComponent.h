// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingOutputComponent.h"
#include "DMXAttribute.h"
#include "DMXPixelMappingRenderElement.h"
#include "DMXTypes.h"
#include "Library/DMXEntityReference.h"

#include "DMXPixelMappingMatrixCellComponent.generated.h"


enum class EDMXColorMode : uint8;
namespace UE::DMXPixelMapping::Rendering::PixelMapRenderer { class FPixelMapRenderElement; }
class SUniformGridPanel;
class UDMXPixelMappingColorSpace;
class UTextureRenderTarget2D;


/**
 * Matrix pixel component
 */
UCLASS()
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingMatrixCellComponent
	: public UDMXPixelMappingOutputComponent
{
	GENERATED_BODY()

	/** Let the matrix component edit size and position */
	friend class UDMXPixelMappingMatrixComponent;

public:
	/** Default Constructor */
	UDMXPixelMappingMatrixCellComponent();

	//~ Begin UObject implementation
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject implementation

	//~ Begin UDMXPixelMappingBaseComponent implementation
	virtual const FName& GetNamePrefix() override;
	virtual FString GetUserName() const override;
	//~ End UDMXPixelMappingBaseComponent implementation

	//~ Begin UDMXPixelMappingOutputComponent implementation	
	virtual bool IsOverParent() const override;
	virtual int32 GetDownsamplePixelIndex() const override { return DownsamplePixelIndex; }
	virtual void QueueDownsample() override;
	//~ End UDMXPixelMappingOutputComponent implementation

	void SetCellCoordinate(FIntPoint InCellCoordinate);
	const FIntPoint& GetCellCoordinate() { return CellCoordinate; }

	/** Check if a Component can be moved under another one (used for copy/move/duplicate) */
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;

	/** Returns the pixel map render element for the matrix cell */
	TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement> GetOrCreatePixelMapRenderElement();

	/** Gets the color of this cell */
	FLinearColor GetPixelMapColor() const;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Cell Settings")
	int32 CellID;

private:
	/** Updates the rendered element. Automatically called on property changes */
	void UpdateRenderElement();

	/** Helper that returns the renderer component this component belongs to */
	UDMXPixelMappingRendererComponent* GetRendererComponent() const;

	/** Pixel map render element for the matrix cell */
	TSharedPtr<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement> PixelMapRenderElement;

	UPROPERTY()
	FIntPoint CellCoordinate;

	/** Index of the cell pixel in downsample target buffer */
	int32 DownsamplePixelIndex;
};
