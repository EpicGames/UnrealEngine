// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingOutputComponent.h"
#include "DMXPixelMappingOutputDMXComponent.h"
#include "Library/DMXEntityReference.h"
#include "Library/DMXEntityFixtureType.h"

#include "DMXPixelMappingMatrixComponent.generated.h"

enum class EDMXColorMode : uint8;
class UDMXLibrary;
class UDMXPixelMappingColorSpace;
class UDMXPixelMappingLayoutScript;
class UDMXPixelMappingMatrixCellComponent;


/**
 * DMX Matrix group component
 */
UCLASS()
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingMatrixComponent
	: public UDMXPixelMappingOutputDMXComponent
{
	GENERATED_BODY()

	DECLARE_EVENT_TwoParams(UDMXPixelMappingMatrixComponent, FDMXPixelMappingOnMatrixChanged, UDMXPixelMapping* /** PixelMapping */, UDMXPixelMappingMatrixComponent* /** AddedComponent */);

	/** Helper callback for loop through all component child */
	using ChildCallback = TFunctionRef<void(UDMXPixelMappingMatrixCellComponent*)>;

public:
	/** Default Constructor */
	UDMXPixelMappingMatrixComponent();

	/** Gets an Event broadcast when a the matrix (and by that its num cells) changed */
	UE_DEPRECATED(5.4, "OnMatrixChanged is removed without replacement. This is handled internally and should not require external handling.")
	static FDMXPixelMappingOnMatrixChanged& GetOnMatrixChanged()
	{
		static FDMXPixelMappingOnMatrixChanged OnMatrixChanged_DEPRECATED;
		return OnMatrixChanged_DEPRECATED;
	}

	// ~Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR
	// ~End UObject interface
	
	/**  Logs properties that were changed in underlying fixture patch or fixture type  */
	void LogInvalidProperties();

public:
	// ~Begin UDMXPixelMappingBaseComponent interface
	virtual const FName& GetNamePrefix() override;
	virtual void ResetDMX(EDMXPixelMappingResetDMXMode ResetMode = EDMXPixelMappingResetDMXMode::SendDefaultValues) override;
	virtual void SendDMX() override;
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;
	virtual FString GetUserName() const override;
	// ~End UDMXPixelMappingBaseComponent interface

	// ~Begin UDMXPixelMappingOutputComponent interface
	virtual void SetPosition(const FVector2D& NewPosition) override;
	virtual void SetPositionRotated(FVector2D NewRotatedPosition) override;
	virtual void SetSize(const FVector2D& NewSize) override;
	virtual void SetRotation(double NewRotation) override;
#if WITH_EDITOR
	virtual const FText GetPaletteCategory() override;
#endif // WITH_EDITOR
	virtual bool IsOverParent() const override;
	// ~End UDMXPixelMappingOutputComponent interface
	
	//~ Begin UDMXPixelMappingOutputDMXComponent implementation
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.3, "Deprecated for performance reasons. Instead use 'Get DMX Pixel Mapping Renderer Component' and Render only once each tick.")
	virtual void RenderWithInputAndSendDMX() override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//~ End UDMXPixelMappingOutputDMXComponent implementation

	/** Handles changes in size or in matrix */
	void HandleMatrixChanged();

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	virtual void QueueDownsample() override;

protected:
	/** Called when the fixture type in use changed */
	void OnFixtureTypeChanged(const UDMXEntityFixtureType* FixtureType);

	/** Called when the fixture patch in use changed */
	void OnFixturePatchChanged(const UDMXEntityFixturePatch* FixturePatch);

#if WITH_EDITORONLY_DATA
	/** Children available PreEditUndo, useful to hide all removed ones in post edit undo */
	TArray<UDMXPixelMappingBaseComponent*> PreEditUndoMatrixCellChildren;
#endif // WITH_EDITORONLY_DATA

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FDMXEntityFixturePatchRef FixturePatchMatrixRef_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	FIntPoint CoordinateGrid;

	UPROPERTY()
	FVector2D CellSize;

	UPROPERTY()
	EDMXPixelMappingDistribution Distribution;

	/** Inverts the cell ordering along the X-axis. Useful when the matrix is facing away from the screen. */
	UPROPERTY(EditAnywhere, Category = "Matrix")
	bool bInvertCellsX = false;

	/** Inverts the cell ordering along the Y-axis. Useful when the matrix is facing away from the screen . */
	UPROPERTY(EditAnywhere, Category = "Matrix")
	bool bInvertCellsY = false;

	/** Layout script for the children of this component (hidden in customizations and displayed in its own panel). */
	UPROPERTY(EditAnywhere, Instanced, Category = "Layout")
	TObjectPtr<UDMXPixelMappingLayoutScript> LayoutScript;

	//////////////////////
	// Deprecated Members
	
public:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	EDMXColorMode ColorMode_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeRExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeGExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeBExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool bMonochromeExpose_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeRInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeGInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool AttributeBInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	bool bMonochromeInvert_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName AttributeR_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName AttributeG_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName AttributeB_DEPRECATED;

	UPROPERTY(Meta = (DeprecatedProperty, DeprecationMessage = "Deprecated in favor of DMXPixelMappingColorSpace. See ColorSpace member."))
	FDMXAttributeName MonochromeIntensity_DEPRECATED;
#endif // WITH_EDITORONLY_DATA
};
