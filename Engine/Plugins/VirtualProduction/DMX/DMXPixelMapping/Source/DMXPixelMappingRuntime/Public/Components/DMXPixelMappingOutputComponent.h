// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingBaseComponent.h"
#include "Components/DMXPixelMappingComponentGeometryCache.h"

#include "DMXPixelMappingOutputComponent.generated.h"

struct FDMXPixelMappingLayoutToken;
class SBox;
class UDMXEntityFixturePatch;
class UDMXPixelMappingComponentGeometryCache;
class UDMXPixelMappingRendererComponent;
namespace UE::DMXPixelMapping::Rendering::PixelMapRenderer { class FPixelMapRenderElement; }



/**
 * Base class for all Designer and configurable components
 */
UCLASS(Abstract)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingOutputComponent
	: public UDMXPixelMappingBaseComponent
{
	GENERATED_BODY()

public:
	/** Default Constructor */
	UDMXPixelMappingOutputComponent();

	//~ Begin UObject Interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

public:
	//~ Begin DMXPixelMappingBaseComponent interface
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;
	//~ End DMXPixelMappingBaseComponent interface

	/*----------------------------------
		UDMXPixelMappingOutputComponent Interface
	----------------------------------*/
#if WITH_EDITOR
	/** Whether component should be part of Palette view */
	virtual bool IsExposedToTemplate() { return false; }

	/** Returns the text of palette category*/
	virtual const FText GetPaletteCategory();

	/** Whether component should be visible */
	virtual bool IsVisible() const;

	/** Whether component can be re-sized or re-position at the editor */
	virtual bool IsLockInDesigner() const { return bLockInDesigner; }

	/** Sets the ZOrder in the UI */
	virtual void SetZOrder(int32 NewZOrder);

	/** Returns the UI ZOrder */
	virtual int32 GetZOrder() const { return ZOrder; }

	/** Returns an editor color for the widget */
	virtual FLinearColor GetEditorColor() const;
#endif // WITH_EDITOR

	/** Returns true if the the component's over all its parents. */
	virtual bool IsOverParent() const;

	/** Returns true if the component is over specified position */
	virtual bool IsOverPosition(const FVector2D& Position) const;

	/** Returns true if the component overlaps the other */
	UE_DEPRECATED(5.4, "Removed without replacement. GetEdges and test against other if this method is needed.")
	virtual bool OverlapsComponent(UDMXPixelMappingOutputComponent* Other) const;

	/** Get pixel index in downsample texture */
	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	virtual int32 GetDownsamplePixelIndex() const { return 0; }

	/** Queue rendering to downsample rendering target */
	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	virtual void QueueDownsample() {}

	/** Returns the absolute position, without rotation */
	FVector2D GetPosition() const;

	/** Sets the absolute position of the component, without rotation. */
	virtual void SetPosition(const FVector2D& NewPosition);

	/** Returns the absolute position with rotation. */
	FVector2D GetPositionRotated() const;

	/** Sets the the absolute position with rotation. */
	virtual void SetPositionRotated(FVector2D NewRotatedPosition);

	/** Returns the edges of the component, absolute, rotated, clockwise order */
	void GetEdges(FVector2D& A, FVector2D& B, FVector2D& C, FVector2D& D) const;

	/** Get the absolute size */
	FVector2D GetSize() const;

	/** Sets the absolue size */
	virtual void SetSize(const FVector2D& NewSize);

	/** Gets the absolute rotation, in degrees */
	double GetRotation() const;

	/** Sets the absolute rotation, in degrees */
	virtual void SetRotation(double NewRotation);

	/** Invalidates the pixel map, effectively causing the renderer component to aquire a new pixel map */
	void InvalidatePixelMapRenderer();

	/** Helper that returns render component if available */
	UDMXPixelMappingRendererComponent* FindRendererComponent() const;
	/** Updates children to match the size of this instance */

#if WITH_EDITOR
	/** Z-orders this component and its children topmost */
	void ZOrderTopmost();
#endif // WITH_EDITOR

public:
#if WITH_EDITORONLY_DATA
	/** ZOrder in the UI */
	UPROPERTY()
	int32 ZOrder = 1;

	/** The color displayed in editor */
	UE_DEPRECATED(5.4, "Should no longer be publicly accessed, instead call GetEditorColor.")
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Editor Settings")
	FLinearColor EditorColor = FLinearColor::Blue;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	// Property Name getters
	static FName GetLockInDesignerPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, bLockInDesigner); }
	static FName GetVisibleInDesignerPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, bVisibleInDesigner); }
	static FName GetPositionXPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, PositionX); }
	static FName GetPositionYPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, PositionY); }
	static FName GetSizeXPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, SizeX); }
	static FName GetSizeYPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, SizeY); }
	static FName GetRotationPropertyName() { return GET_MEMBER_NAME_CHECKED(UDMXPixelMappingOutputComponent, Rotation); }
#endif

protected:
	UPROPERTY(BlueprintReadOnly, Category = "Transform", Meta = (AllowPrivateAccess = true))
	float PositionX = 0.f;

	UPROPERTY(BlueprintReadOnly, Category = "Transform", Meta = (AllowPrivateAccess = true))
	float PositionY = 0.f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Transient, Category = "Transform", Meta = (EditCondition = "!bLockInDesigner", DisplayName = "Position (with Rotation)"))
	FVector2D EditorPositionWithRotation = FVector2D::ZeroVector;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transform", Meta = (ClampMin = 0.0001, EditCondition = "!bLockInDesigner"))
	float SizeX = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transform", Meta = (ClampMin = 0.0001, EditCondition = "!bLockInDesigner"))
	float SizeY = 1.f;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Transform", Meta = (UIMin = -180, UIMax = 180, EditCondition = "!bLockInDesigner"))
	double Rotation = 0.0;

	UPROPERTY(Transient)
	FDMXPixelMappingComponentGeometryCache CachedGeometry;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Editor Settings")
	bool bLockInDesigner;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Editor Settings")
	bool bVisibleInDesigner;
#endif
};
