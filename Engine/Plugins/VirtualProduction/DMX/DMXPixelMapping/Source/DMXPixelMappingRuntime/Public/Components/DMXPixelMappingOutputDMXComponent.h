// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DMXPixelMappingOutputComponent.h"
#include "DMXPixelMappingRenderElement.h"
#include "Library/DMXEntityReference.h"

#include "DMXPixelMappingOutputDMXComponent.generated.h"

enum class EDMXPixelBlendingQuality : uint8;
class UDMXModulator;
class UDMXPixelMappingColorSpace;


/** Base class for components that contain a fixture patch to send DMX. */
UCLASS(Abstract)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingOutputDMXComponent
	: public UDMXPixelMappingOutputComponent
{
	GENERATED_BODY()

public:
	//~ Begin UDMXPixelMappingOutputComponent interface
#if WITH_EDITOR
	virtual FLinearColor GetEditorColor() const override;
#endif // WITH_EDITOR	
	//~ End UDMXPixelMappingOutputComponent interface

	/** The quality level to use when averaging colors during downsampling. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Quality")
	EDMXPixelBlendingQuality CellBlendingQuality = EDMXPixelBlendingQuality::High;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Fixture Patch", Meta = (ShowOnlyInnerProperties))
	FDMXEntityFixturePatchRef FixturePatchRef;

#if WITH_EDITORONLY_DATA
	/** If set, the color of the patch is displayed, instead of a custom editor color */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Editor Settings", Meta = (AllowPrivateAccess = true))
	bool bUsePatchColor = false;
#endif

	/** Sets which color space Pixel Mapping sends */
	UPROPERTY(Transient, EditAnywhere, NoClear, Category = "Color Space", Meta = (DisplayPriority = 2, DisplayName = "Output Mode", ShowDisplayNames))
	TSubclassOf<UDMXPixelMappingColorSpace> ColorSpaceClass;

	/** The Color Space currently in use */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Color Space")
	TObjectPtr<UDMXPixelMappingColorSpace> ColorSpace;

	/** Modulators applied to the output before sending DMX */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Transient, Category = "Output Modulators", meta = (DisplayName = "Output Modulators"))
	TArray<TSubclassOf<UDMXModulator>> ModulatorClasses;

	/** The actual modulator instances */
	UPROPERTY()
	TArray<TObjectPtr<UDMXModulator>> Modulators;

	/** Render input texture for downsample texture, donwsample and send DMX for this component */
	UE_DEPRECATED(5.3, "Deprecated for performance reasons. Instead use 'Get DMX Pixel Mapping Renderer Component' and Render only once each tick.")
	UFUNCTION(BlueprintCallable, Category = "DMX|PixelMapping", Meta = (DeprecatedFunction, DeprecationMessage = "Deprecated for performance reasons. Instead use 'Get DMX Pixel Mapping Renderer Component' and Render only once each tick"))
	virtual void RenderWithInputAndSendDMX() PURE_VIRTUAL(UDMXPixelMappingOutputDMXComponent::RenderWithInputAndSendDMX);

	/** Check if a Component can be moved under another one (used for copy/move/duplicate) */
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override
	{
		return false;
	}
};
