// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingOutputComponent.h"
#include "DMXPixelMappingPreprocessRenderer.h"
#include "IDMXPixelMappingRenderer.h"
#include "Library/DMXEntityReference.h"
#include "Templates/SubclassOf.h"

#include "DMXPixelMappingRendererComponent.generated.h"

enum class EDMXPixelMappingRendererType : uint8;
class UDMXPixelMappingLayoutScript;
class UDMXPixelMappingPixelMapRenderer;
class UDMXPixelMappingPreprocessRenderer;
class UMaterialInterface;
class UTexture;
class UTextureRenderTarget2D;
class UUserWidget;
class UWorld;
namespace UE::DMXPixelMapping::Rendering { class FPixelMapRenderElement; }


UENUM(BlueprintType)
enum class EDMXPixelMappingRendererPixelFormat : uint8
{
	Auto UMETA(ToolTip = "Use the format of the Input Texture, use 8-bit for Input Materials and UMG."),
	RGBA8 UMETA(DisplayName = "Low Precision 8-bit (RGBA8)"),
	RGBA16F UMETA(DisplayName = "High Precision 16-bit (RGBA16F)")
};


/** 
 * Component for rendering input texture.  
 */
UCLASS(BlueprintType)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingRendererComponent
	: public UDMXPixelMappingOutputComponent
{
	GENERATED_BODY()

public:
	/** Default Constructor */
	UDMXPixelMappingRendererComponent();

	//~ Begin UObject implementation
protected:
	virtual void PostInitProperties() override;
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR
	//~ End UObject implementation

public:
	//~ Begin UDMXPixelMappingBaseComponent implementation
	virtual const FName& GetNamePrefix() override;
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;
	virtual void ResetDMX(EDMXPixelMappingResetDMXMode ResetMode = EDMXPixelMappingResetDMXMode::SendDefaultValues) override;
	virtual void SendDMX() override;
	virtual void Render() final;
	virtual void RenderAndSendDMX() final;
	virtual FString GetUserName() const override;
	//~ End UDMXPixelMappingBaseComponent implementation

	/** Returns the pixel mapping preprocess renderer */
	UDMXPixelMappingPreprocessRenderer* GetPreprocessRenderer() const { return PreprocessRenderer; }

	/** Updates the preprocess renderer from the current render type and source texture */
	void UpdatePreprocessRenderer();

	/**
	 * Invalidates the pixel map, effectively causing a rebuild of the mapping on the next Render call.
	 * Does not rebuild the renderer self. Use InitializeRenderer to rebuild the renderers.
	 */
	void InvalidatePixelMapRenderer();

	/** Gets the rendered input texture, or nullptr if no input texture is currently rendered */
	UTexture* GetRenderedInputTexture() const;

	/**
	 * Returns the Modulators of the component corresponding to the patch specified.
	 * Note, this node does a lookup on all fixture patches in use, hence may be slow and shouldn't be called on tick.
	 */
	UFUNCTION(BlueprintCallable, Category = "DMX")
	bool GetPixelMappingComponentModulators(FDMXEntityFixturePatchRef FixturePatchRef, TArray<UDMXModulator*>& DMXModulators);

	/** Returns a copy of the current pixel map render elements */
	TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>> GetPixelMapRenderElements() const;

	/** Selects the input type. It can be a Texture, a Material or a User Widget. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Render Settings")
	EDMXPixelMappingRendererType RendererType;

	/** The texture used for pixel mapping */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Render Settings")
	TObjectPtr<UTexture> InputTexture;

	/** The material used for pixel mapping */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Render Settings", Meta = (DisplayName = "User Interface Material"))
	TObjectPtr<UMaterialInterface> InputMaterial;

	/** The UMG widget used for pixel mapping */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Render Settings")
	TSubclassOf<UUserWidget> InputWidget;

	/** The dynamic range of the renderer */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Render Settings")
	EDMXPixelMappingRendererPixelFormat PixelFormat = EDMXPixelMappingRendererPixelFormat::Auto;

	/** The exposure of the pixel mapping renderer. This property affects the DMX output. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings", Meta = (ClampMin = "0", UIMin = "0", UIMax = "1", DisplayName = "Exposure"))
	float Brightness = 1.f;

	/** If true, children are positioned relative to the size of this renderer */
	UPROPERTY(EditAnywhere, Category = "Layout")
	bool bChildrenFollowSize = true;

	/** Layout script for the children of this component (hidden in customizations and displayed in its own panel). */
	UPROPERTY(EditAnywhere, Instanced, Category = "Layout")
	TObjectPtr<UDMXPixelMappingLayoutScript> LayoutScript;

private:
	/** Called when a component was added to or removed from the pixel mapping */
	void OnComponentAddedOrRemoved(UDMXPixelMapping* PixelMapping, UDMXPixelMappingBaseComponent* Component);

	/** When bChildrenFollowSize is true, rearranges children relatively to the current size */
	void LetChildrenFollowSize();

	/** Returns the pixel format depending on the current dynamic range setting */
	EPixelFormat GetFormatFromDynamicRange() const;

	/** Tries to get any world. If with editor, returns the editor world, in game returns GWorld */
	UWorld* TryGetWorld() const;

	/** Elements rendered during pixel mapping */
	TArray<TSharedRef<UE::DMXPixelMapping::Rendering::FPixelMapRenderElement>> PixelMapRenderElements;

	/** The user widget instance currently in use */
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UUserWidget> UserWidget;

	/** True once the pixel map was built. Reset to false when the pixel map renderer is invalidated */
	bool bInvalidatePixelMap = true;

	/** Renderer responsible to preprocess the input texture / material / user widget */
	UPROPERTY(VisibleAnywhere, Instanced, Category = "Filtering")
	TObjectPtr<UDMXPixelMappingPreprocessRenderer> PreprocessRenderer;

	/** Renderer responsible to pixel map */
	UPROPERTY()
	TObjectPtr<UDMXPixelMappingPixelMapRenderer> PixelMapRenderer;

	/** The current rect in which children are laid out. Useful to compute UV position and size of children. */
	UPROPERTY()
	FVector2D LayoutRect;

	//////////////////////
	// Deprecated Members

public:
	UE_DEPRECATED(5.3, "Since a pixel mapping asset doesn't have a clear world context, GetWorld() is no longer supported.")
	UWorld* GetWorld() const;

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TSharedPtr<IDMXPixelMappingRenderer>& GetRenderer() { return PixelMappingRenderer_DEPRECATED; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	void RenderEditorPreviewTexture();

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	UTextureRenderTarget2D* GetPreviewRenderTarget();
#endif

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	FIntPoint GetPixelPosition(int32 InIndex) const;

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	void CreateOrUpdateDownsampleBufferTarget();

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void AddPixelToDownsampleSet(FDMXPixelMappingDownsamplePixelParamsV2&& InDownsamplePixelParam);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	int32 GetDownsamplePixelNum();

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	void SetDownsampleBuffer(TArray<FLinearColor>&& InDownsampleBuffer, FIntRect InRect);

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	bool GetDownsampleBufferPixel(const int32 InDownsamplePixelIndex, FLinearColor& OutLinearColor);

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	bool GetDownsampleBufferPixels(const int32 InDownsamplePixelIndexStart, const int32 InDownsamplePixelIndexEnd, TArray<FLinearColor>& OutLinearColors);

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	bool ResetColorDownsampleBufferPixel(const int32 InDownsamplePixelIndex);

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	bool ResetColorDownsampleBufferPixels(const int32 InDownsamplePixelIndexStart, const int32 InDownsamplePixelIndexEnd);

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	void EmptyDownsampleBuffer();

private:
	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	int32 GetTotalDownsamplePixelCount();

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	bool IsPixelRangeValid(const int32 InDownsamplePixelIndexStart, const int32 InDownsamplePixelIndexEnd) const;

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	void ResizePreviewRenderTarget(uint32 InSizeX, uint32 InSizeY);

	UE_DEPRECATED(5.3, "Please use UDMXPixelMappingPixelMapRenderer to render the pixel map")
	UTextureRenderTarget2D* CreateRenderTarget(const FName& InBaseName);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<IDMXPixelMappingRenderer> PixelMappingRenderer_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> DownsampleBufferTarget_DEPRECATED;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> PreviewRenderTarget_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	TArray<FLinearColor> DownsampleBuffer_DEPRECATED;
	int32 DownsamplePixelCount_DEPRECATED = 0;
	mutable FCriticalSection DownsampleBufferCS_DEPRECATED;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<FDMXPixelMappingDownsamplePixelParamsV2> DownsamplePixelParams_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	static const FLinearColor ClearTextureColor_DEPRECATED;
	bool bWasEverRendered_DEPRECATED = false;

public:
	static const FIntPoint MaxDownsampleBufferTargetSize_DEPRECATED;
};
