// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingBaseComponent.h"

#include "DMXPixelMappingRootComponent.generated.h"

class UDMXPixelMappingRendererComponent;


/**
 * Root component in the components tree
 */
UCLASS()
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingRootComponent
	: public UDMXPixelMappingBaseComponent
{
	GENERATED_BODY()

protected:
	// ~Begin UObject Interface
	virtual void PostLoad() override;
	virtual void BeginDestroy() override;
	// ~End UObject Interface

public:
	//~ Begin UDMXPixelMappingBaseComponent implementation
	virtual const FName& GetNamePrefix() override;
	virtual void AddChild(UDMXPixelMappingBaseComponent* InComponent) override;
	virtual void RemoveChild(UDMXPixelMappingBaseComponent* InComponent) override;
	virtual void ResetDMX(EDMXPixelMappingResetDMXMode ResetMode = EDMXPixelMappingResetDMXMode::SendDefaultValues) override;
	virtual void SendDMX() override;
	virtual void Render() override;
	virtual void RenderAndSendDMX() override;
	virtual FString GetUserName() const override;
	//~ End UDMXPixelMappingBaseComponent implementation

	/** Check if a Component can be moved under another one (used for copy/move/duplicate) */
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;

	/** Finds a renderer component by its name. Faster than UDMXPixelMappingBaseComponent::FindComponentByName */
	UDMXPixelMappingRendererComponent* FindRendererComponentByName(const FName& Name) const;

private:
	/** Called when a component in the PixelMapping was renamed */
	void OnComponentRenamed(UDMXPixelMappingBaseComponent* RenamedComponent);

	/** Chached Renderer Component names and their actual object pointer. Useful to speed up access (see DMXPixelMappingSubsystem) */
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UDMXPixelMappingRendererComponent>> CachedRendererComponentsByName;

	/** Holds the last rendered frame number, to avoid rendering multiple times in a single frame */
	uint32 LastRenderedFrameNumber = 0;
};
