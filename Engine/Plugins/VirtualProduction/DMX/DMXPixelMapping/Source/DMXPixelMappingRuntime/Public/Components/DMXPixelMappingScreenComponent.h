// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/DMXPixelMappingOutputDMXComponent.h"

#include "DMXProtocolTypes.h"
#include "IO/DMXOutputPortReference.h"
#include "Library/DMXEntityFixtureType.h"
#include "Styling/SlateBrush.h"

#include "DMXPixelMappingScreenComponent.generated.h"

enum class EDMXCellFormat : uint8;
enum class EDMXPixelBlendingQuality : uint8;
class FDMXPixelMappingComponentWidget;
class FDMXOutputPort;
class SDMXPixelMappingScreenComponentBox;
class UTextureRenderTarget2D;
class UDMXPixelMappingRendererComponent;

/** DMX Screen component. Fully deprecated as of 5.5. */
class UE_DEPRECATED(5.5, "DMXScreenComponent is no longer supported in favor of a fully Fixture Patch based workflow.") UDMXPixelMappingScreenComponent;
UCLASS(Deprecated, meta = (DeprecationMessage = "Deprecated 5.5. DMXScreenComponent is no longer supported in favor of a fully Fixture Patch based workflow."))
class DMXPIXELMAPPINGRUNTIME_API UDEPRECATED_DMXPixelMappingScreenComponent
	: public UDMXPixelMappingOutputDMXComponent
{
	GENERATED_BODY()

public:
	using ForEachPixelCallback = TFunctionRef<void(const int32 /* IndexXY */, const int32 /* IndexX */, const int32 /* IndexY */)>;

public:
	/** Default Constructor */
	UDEPRECATED_DMXPixelMappingScreenComponent();

	//~ Begin UObject implementation
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedChainEvent) override;
#endif // WITH_EDITOR
	//~ End UObject implementation

	//~ Begin UDMXPixelMappingBaseComponent implementation
	virtual const FName& GetNamePrefix() override;
	virtual void SendDMX() override;
	//~ End UDMXPixelMappingBaseComponent implementation

	//~ Begin UDMXPixelMappingOutputComponent implementation
#if WITH_EDITOR
	virtual const FText GetPaletteCategory() override;
	virtual bool IsExposedToTemplate() { return true; }
#endif // WITH_EDITOR
	//~ End UDMXPixelMappingOutputComponent implementation

public:
	virtual void QueueDownsample() override;
	//~ End UDMXPixelMappingOutputComponent implementation

	//~ Begin UDMXPixelMappingOutputDMXComponent implementation
	virtual void RenderWithInputAndSendDMX() override;
	//~ End UDMXPixelMappingOutputDMXComponent implementation

	/** Check if a Component can be moved under another one (used for copy/move/duplicate) */
	virtual bool CanBeMovedTo(const UDMXPixelMappingBaseComponent* Component) const override;

	/** Calculate screen size of the single pixel in screen component */
	const FVector2D GetScreenPixelSize() const;

	/** Loop through X and Y pixels in screen component */  
	void ForEachPixel(ForEachPixelCallback InCallback);

private:
	/** Prepare the final color to send */
	void AddColorToSendBuffer(const FColor& Color, TArray<uint8>& OutDMXSendBuffer);

public:
	/** If true, outputs to all DMX Output Ports */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Settings")
	bool bSendToAllOutputPorts;

	/** The port this render component outputs to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Common Settings", Meta = (DisplayName = "DMX Output Ports", EditCondition = "!bSendToAllOutputPorts"))
	TArray<FDMXOutputPortReference> OutputPortReferences;

	/** Returns the output Ports of the renderer component */
	TSet<FDMXOutputPortSharedRef> GetOutputPorts() const { return OutputPorts; }

	/** Get range of the downsample pixel positions */
	const TTuple<int32, int32> GetPixelDownsamplePositionRange() const { return PixelDownsamplePositionRange; }

private:
	/** Helper that returns the renderer component this component belongs to */
	UDMXPixelMappingRendererComponent* GetRendererComponent() const;

private:
	/** The output port instances, generated from OutputPortReferences */
	TSet<FDMXOutputPortSharedRef> OutputPorts;

public:
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Mapping Settings", meta = (DisplayName = "X Cells", ClampMin = "1", ClampMax = "1000", UIMin = "1", UIMax = "100", DisplayPriority = "1"))
	int32 NumXCells;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Mapping Settings", meta = (DisplayName = "Y Cells", ClampMin = "1", ClampMax = "1000", UIMin = "1", UIMax = "100", DisplayPriority = "1"))
	int32 NumYCells;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Output Ports instead."))
	FDMXProtocolName ProtocolName_DEPRECATED;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (ClampMin = "1", ClampMax = "100000", UIMin = "1", UIMax = "100000", DisplayPriority = "1"))
	int32 LocalUniverse;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (ClampMin = "1", ClampMax = "512", UIMin = "1", UIMax = "512", DisplayPriority = "1"))
	int32 StartAddress;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (DisplayName = "Color Space", DisplayPriority = "1"))
	EDMXCellFormat PixelFormat;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (DisplayPriority = "1"))
	EDMXPixelMappingDistribution Distribution;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (DisplayPriority = "1"))
	bool bIgnoreAlphaChannel;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", DisplayPriority = "2"))
	float PixelIntensity;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1", DisplayPriority = "2"))
	float AlphaIntensity;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (DisplayPriority = "3"))
	bool bShowAddresses;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Patch Settings", meta = (DisplayPriority = "3"))
	bool bShowUniverse;
#endif

private:
#if WITH_EDITOR
	/** Screen Component box used in the ComponentWidget */
	TSharedPtr<SDMXPixelMappingScreenComponentBox> ScreenComponentBox;
#endif // WITH_EDITOR

	/**Range of the downsample pixel positions */
	TTuple<int32, int32> PixelDownsamplePositionRange;

#if WITH_EDITORONLY_DATA
	FSlateBrush Brush;

	bool bIsUpdateWidgetRequested;
#endif

	static const FVector2D MinGridSize;
};
