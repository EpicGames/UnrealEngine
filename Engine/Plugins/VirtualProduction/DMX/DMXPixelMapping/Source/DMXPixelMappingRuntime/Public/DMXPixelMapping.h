// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tickable.h"
#include "UObject/Object.h"

#include "DMXPixelMapping.generated.h"

class SWidget;
class UDMXEntityFixturePatch;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingOutputComponent;
class UTexture;
enum class EDMXPixelMappingResetDMXMode : uint8;

/** 
 * DMX Pixel Mapping.
 *
 * Reads pixel colors of a Texture, Material or UMG widget and sends them as DMX.
 */
UCLASS(BlueprintType, Blueprintable)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMapping
	: public UObject
	, public FTickableGameObject
{
	using TComponentPredicate = TFunctionRef<void(UDMXPixelMappingBaseComponent*)>;

	template <typename Type>
	using TComponentPredicateType = TFunctionRef<void(Type*)>;


	GENERATED_BODY()
public:
	UDMXPixelMapping();

	/** Starts to send DMX */
	void StartSendingDMX();

	/** Stops sending DMX  */
	void StopSendingDMX();

	/** Pauses sending DMX */
	void PauseSendingDMX();

	/** Returns true if the pixel mapping is playing DMX. Note this returns false while it's paused */
	bool IsSendingDMX() const { return bIsSendingDMX; }

	/** Returns true if the pixel mapping is paused from playing DMX */
	bool IsPaused() const { return bIsPaused; }

	/** Sets how pixel mapping resets the channels it sends to when Stop Sending DMX is called. */
	void SetResetDMXMode(EDMXPixelMappingResetDMXMode InResetDMXMode) { ResetDMXMode = InResetDMXMode; }

	/** Returns how the DMX channels used by pixel mapping are reset when playing the pixel mapping is stopped */
	EDMXPixelMappingResetDMXMode GetResetDMXMode() const { return ResetDMXMode; }

	//~ Begin UObject implementation
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	//~ End UObject implementation

	/** Get root component of the component tree */
	UFUNCTION(BlueprintPure, Category = "DMX|PixelMapping")
	UDMXPixelMappingRootComponent* GetRootComponent() const { return RootComponent; }

	/** Checks if nested objects are created, otherwise create them. */
	void CreateOrLoadObjects();

	/** Recurcevly preload all components in tree. */
	void PreloadWithChildren();

	/** Destroys invalid components. Useful when fixture type or fixture patch changed */
	void DestroyInvalidComponents();

	/** Returns the first component that corresponds to the patch or null if none present */
	UDMXPixelMappingBaseComponent* FindComponent(UDMXEntityFixturePatch* FixturePatch) const;

	/** Find the component by name. */
	UDMXPixelMappingBaseComponent* FindComponent(const FName& Name) const;

#if WITH_EDITOR
	/** Find the component by widget. */
	UE_DEPRECATED(5.4, "Component widgets are no longer supported since 5.1. This function will always return nullptr.")
	UDMXPixelMappingOutputComponent* FindComponent(TSharedPtr<SWidget> InWidget) const { return nullptr;}
#endif // WITH_EDITOR

	/**
	 * Looking for the first component of class by given name
	 * @param InName        The name to search
	 * @return				An instance of the templated Component
	 */
	template <typename TComponentClass>
	TComponentClass* FindComponentOfClass(const FName& InName) const
	{
		TComponentClass* FoundComponent = nullptr;

		ForEachComponent([&](UDMXPixelMappingBaseComponent* InComponent) {
			if (TComponentClass* CastComponent = Cast<TComponentClass>(InComponent))
			{
				if (CastComponent->GetFName() == InName)
				{
					FoundComponent = CastComponent;
					return;
				}
			}
		});

		return FoundComponent;
	}

	/**
	 * Get all component by given class
	 * @param OutComponents        Found components
	 */
	template <typename TComponentClass>
	void GetAllComponentsOfClass(TArray<TComponentClass*>& OutComponents) const
	{
		ForEachComponent([&OutComponents](UDMXPixelMappingBaseComponent* InComponent) {
			if (TComponentClass* CastComponent = Cast<TComponentClass>(InComponent))
			{
				OutComponents.Add(CastComponent);
			}
		});
	}

	/**
	 * Iterates through components by class with given Predicate callback
	 */
	template <typename TComponentClass>
	void ForEachComponentOfClass(TComponentPredicateType<TComponentClass> Predicate) const
	{
		ForEachComponent([&Predicate](UDMXPixelMappingBaseComponent* InComponent) {
			if (TComponentClass* CastComponent = Cast<TComponentClass>(InComponent))
			{
				Predicate(CastComponent);
			}
		});
	}

	/**
	 * Get array of FName pointers by given class
	 * @param OutComponents        Found components
	 */
	template <typename TComponentClass>
	void GetAllComponentsNamesOfClass(TArray<TSharedPtr<FName>>& InComponentNames) const
	{
		ForEachComponent([&InComponentNames](UDMXPixelMappingBaseComponent* InComponent) {
			if (TComponentClass* CastComponent = Cast<TComponentClass>(InComponent))
			{
				InComponentNames.Add(MakeShared<FName>(CastComponent->GetFName()));
			}
		});
	}

	/**
	 * Recursively Iterates through all compnents
	 * @param Predicate        Callback function
	 */
	void ForEachComponent(TComponentPredicate Predicate) const;

	/** 
	 * Removes the Component from the hierarchy and all sub Components.
	 * @param InComponent        Component for remove
	 */
	void RemoveComponent(UDMXPixelMappingBaseComponent* InComponent);

	//~ Begin FTickableGameObject interface
	virtual bool IsTickableInEditor() const { return true; }
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool IsTickable() const override;
	virtual void Tick(float DeltaTime) override;
	TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(UDMXPixelMapping, STATGROUP_Tickables);
	}
	//~ End FTickableGameObject interface

public:
	/** Holds the reference to root component */
	UPROPERTY()
	TObjectPtr<UDMXPixelMappingRootComponent> RootComponent;

#if WITH_EDITORONLY_DATA
	////////////////////////////////////
	// Per Asset Editor User Settings

	/** If true, grid snapping is enabled */
	UPROPERTY(NonTransactional)
	bool bGridSnappingEnabled = false;

	/** The number of columns in the grid */
	UPROPERTY(NonTransactional)
	int32 SnapGridColumns = 10;

	/** The number of rows in the grid */
	UPROPERTY(NonTransactional)
	int32 SnapGridRows = 10;

	/** The color of the grid snapping grid */
	UPROPERTY(NonTransactional)
	FLinearColor SnapGridColor;

	/** Font size for the component labels in the designer view */
	UPROPERTY(NonTransactional)
	float ComponentLabelFontSize = 8.f;

	/** Exposure of the designer view */
	UPROPERTY(NonTransactional)
	float DesignerExposure = 1.f;

	/** If true, new components use the fixture patch color instead of the default pixel mapping color. */
	UPROPERTY(NonTransactional)
	bool bNewComponentsUsePatchColor = true;

	/** If true, editor is set to scale children with parent. This is forwarded from the editor module (DMXPixelMappingEditorSettings) to be accessible in the runtime module. */
	UPROPERTY(Transient, NonTransactional)
	bool bEditorScaleChildrenWithParent = false;

	/** Holds the Thumbnail image for this asset */
	UPROPERTY()
	TObjectPtr<UTexture> ThumbnailImage;
#endif // WITH_EDITORONLY_DATA

private:
	/** True while the pixel mapping is sending DMX on tick */
	bool bIsSendingDMX = false;

	/** True while the pixel mapping is paused */
	bool bIsPaused = false;

	/** Defines how DMX is reset when playback is stopped */
	UPROPERTY()
	EDMXPixelMappingResetDMXMode ResetDMXMode;

};
