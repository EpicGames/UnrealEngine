// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"

struct FDMXAttributeName;
class UDMXEntityFixturePatch;
class UDMXModulator;
class UDMXPixelMappingBaseComponent;

#if WITH_EDITOR
struct FPropertyChangedChainEvent;
#endif


class DMXPIXELMAPPINGRUNTIME_API FDMXPixelMappingRuntimeUtils
{
public:
	/** Adds a normalized attribute value to a DMX channel value map */
	static void ConvertNormalizedAttributeValueToChannelValue(UDMXEntityFixturePatch* InFixturePatch, const FDMXAttributeName& InAttributeName, float InNormalizedValue, TMap<int32, uint8>& InOutChannelToValueMap);

#if WITH_EDITOR
	/** Handles property changes for modulator classes in one place for Matrix and GroupItems, useless for other classes. */
	static void HandleModulatorPropertyChange(UDMXPixelMappingBaseComponent* Component, FPropertyChangedChainEvent& PropertyChangedChainEvent, const TArray<TSubclassOf<UDMXModulator>>& ModulatorClasses, TArray<UDMXModulator*>& InOutModulators);
#endif 

private:
	/** No instances can be created */
	FDMXPixelMappingRuntimeUtils() = delete;
};
