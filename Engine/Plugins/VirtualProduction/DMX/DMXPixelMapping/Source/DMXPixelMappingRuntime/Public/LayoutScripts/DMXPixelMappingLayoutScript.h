// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/Object.h"

#include "DMXPixelMappingLayoutScript.generated.h"

class UDMXEntityFixturePatch;
class UDMXLibrary;
class UDMXPixelMappingOutputComponent;
class UTextureRenderTarget2D;


/** The position and size of a comoponent in a layout. */
USTRUCT(BlueprintType)
struct DMXPIXELMAPPINGRUNTIME_API FDMXPixelMappingLayoutToken
{
	GENERATED_BODY()

	FDMXPixelMappingLayoutToken() = default;
	FDMXPixelMappingLayoutToken(TWeakObjectPtr<UDMXPixelMappingOutputComponent> OutputComponent);

	/** The position of the component on the X-Axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Layout Script")
	float PositionX = 0.f;

	/** The position of the component on the Y-Axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Layout Script")
	float PositionY = 0.f;

	/** The size of the component on the X-Axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Layout Script")
	float SizeX = 0.f;

	/** The size of the component on the Y-Axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Layout Script")
	float SizeY = 0.f;

	/** The rotation relative to the parent the component, applied after position */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Layout Script")
	double RelativeRotation = 0.0;

	/** The output component to which the token is applied */
	UPROPERTY(BlueprintReadWrite, Category = "Pixel Mapping Layout")
	TWeakObjectPtr<UDMXPixelMappingOutputComponent> Component;

	/** The Fixture ID of the Component's Fixture Patch, or 0 if the component has no Fixture Patch, or the Fixture ID is not set. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Layout Script")
	int32 FixtureID = 0;

private:
	/** Initializes the FixtureID member */
	void InitializeFixtureID();
};


/** 
 * Allows scripting of Pixel Mapping Component Layouts.
 * Override the Layout function to implement a layout. 
 * 
 * Note, layout scripts are applied in parent's unrotated space. 
 */
UCLASS(BlueprintType, Blueprintable, EditInlineNew, Abstract)
class DMXPIXELMAPPINGRUNTIME_API UDMXPixelMappingLayoutScript
	: public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Lays out children of the selection according to OutTokens. Tokens that are not returned remain unchanged.
	 * Called when the script is loaded (unless set otherwise in editor) and when Properties were changed. 
	 * 
	 * @param InTokens		The child components of the current selection, as layout tokens. 
	 * @param OutTokens		The layoyut of a component.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "DMX")
	void Layout(const TArray<FDMXPixelMappingLayoutToken>& InTokens, TArray<FDMXPixelMappingLayoutToken>& OutTokens);
	virtual void Layout_Implementation(const TArray<FDMXPixelMappingLayoutToken>& InTokens, TArray<FDMXPixelMappingLayoutToken>& OutTokens) {}

	/** Sets the number of Tokens that will be passed when Layout is called */
	virtual void SetNumTokens(int32 NewNumTokens) { NumTokens = NewNumTokens; }

	/** Sets the position of the parent component where the components that are being layouted reside in. */
	virtual void SetParentComponentPosition(const FVector2D NewPosition) { ParentComponentPosition = NewPosition; };

	/** Sets the size of the parent component where the components that are being layouted reside in. */
	virtual void SetParentComponentSize(const FVector2D& NewSize) { ParentComponentSize = NewSize; };

	/** Sets the size of the parent component where the components that are being layouted reside in. */
	virtual void SetParentComponentRotation(double NewRotation) { ParentRotation = NewRotation; };

	/** Sets the Texture Size */
	virtual void SetTextureSize(const FVector2D& NewTextureSize) { TextureSize = NewTextureSize; }

protected:
	/** The number of tokens in the Layout. */
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Layout Script")
	int32 NumTokens = 0;

	/** The position of the parent component where the components that are being layouted reside in. */
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Layout Script")
	FVector2D ParentComponentPosition;

	/** The size of the parent component where the components that are being layouted reside in. */
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Layout Script")
	FVector2D ParentComponentSize;

	/** 
	 * The rotation of the parent component where the components that are being layouted reside in.
	 * Note, the position property does not consider this rotation to ease calculations.
	 */
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Layout Script")
	double ParentRotation;

	/** The size of the texture this component is mapping. */
	UPROPERTY(BlueprintReadOnly, Transient, Category = "Layout Script")
	FVector2D TextureSize;
};
