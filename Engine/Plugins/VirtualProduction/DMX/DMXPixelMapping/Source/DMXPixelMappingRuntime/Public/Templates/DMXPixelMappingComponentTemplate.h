// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Library/DMXEntityReference.h"

#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/StrongObjectPtr.h"

struct UDMXEntityFixturePatchRef;
class UDMXPixelMappingBaseComponent;
class UDMXPixelMappingRootComponent;
class UClass;


/**
 * The Component template represents a Component or a set of Components to create.
 */
class DMXPIXELMAPPINGRUNTIME_API FDMXPixelMappingComponentTemplate
	: public TSharedFromThis<FDMXPixelMappingComponentTemplate>
{
public:
	/** Constructor for a template that can be specified by class */
	explicit FDMXPixelMappingComponentTemplate(TSubclassOf<UDMXPixelMappingBaseComponent> InComponentClass);

	/** Constructor for a group items template that can hold fixture patches */
	FDMXPixelMappingComponentTemplate(TSubclassOf<UDMXPixelMappingBaseComponent> InComponentClass, const FDMXEntityFixturePatchRef& InFixturePatchRef);

	/** Virtual Destructor */
	virtual ~FDMXPixelMappingComponentTemplate() {}

#if WITH_EDITOR
	/** Gets the category for the Component */
	FText GetCategory() const;
#endif 

	/** 
	 * Creates an instance of the Component according to the class specified during construction. 
	 * Returns nullptr if the ComponentType is not a ComponentType. 
	 */
	template <typename ComponentType>
	ComponentType* CreateComponent(UDMXPixelMappingRootComponent* InRootComponent)
	{
		return Cast<ComponentType>(CreateComponentInternal(InRootComponent));
	}

	/** The name of the Component template. */
	FText Name;

private:
	/** Internal create components without typecasting */
	UDMXPixelMappingBaseComponent* CreateComponentInternal(UDMXPixelMappingRootComponent* InRootComponent);

	/** The Component class that will be created by this template */
	TWeakObjectPtr<UClass> ComponentClass;

	/** Fixture Patch References, only for group item classes */
	FDMXEntityFixturePatchRef FixturePatchRef;
};
